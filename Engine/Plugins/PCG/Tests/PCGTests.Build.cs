// Copyright Epic Games, Inc. All Rights Reserved.
using UnrealBuildTool;
using System.IO;

public class PCGTests : TestModuleRules
{
	static PCGTests()
	{
		if (InTestMode)
		{
			TestMetadata = new Metadata();
			TestMetadata.TestName = "PCG";
			TestMetadata.TestShortName = "PCG";
		}
	}
	public PCGTests(ReadOnlyTargetRules Target) : base(Target, true)
	{
		bAllowUETypesInNamespaces = true;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"PCG"
			});

		// Allow tests to include PCG private headers (e.g. internal element settings classes
		// that are not part of PCG's public API).
		PrivateIncludePaths.Add(Path.Combine(GetModuleDirectory("PCG"), "Private"));
	}
}