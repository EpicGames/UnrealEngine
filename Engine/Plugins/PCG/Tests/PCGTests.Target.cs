// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.All)]
public class PCGTestsTarget : TestTargetRules
{
	public PCGTestsTarget(TargetInfo Target) : base(Target)
	{
		bCompileAgainstApplicationCore = true;
		bCompileAgainstEngine = true;
		bCompileAgainstCoreUObject = true;

		bMockEngineDefaults = true;
		bTestsRequireEngine = true;

		// Necessary for Regex. ICU needs platform file access.
		bCompileICU = true;
		bUsePlatformFileStub = false;

		// If we add tests that are in the PCG Module (because they use functionality that is not exposed) we can enable this option.
		// Note that it will gather ALL the tests that are in the engine by default, and if we execute the PCGTests program without any filter
		// it will run them all.
		// bWithLowLevelTestsOverride = true;
        
		bWarningsAsErrors = true;
	}
}
