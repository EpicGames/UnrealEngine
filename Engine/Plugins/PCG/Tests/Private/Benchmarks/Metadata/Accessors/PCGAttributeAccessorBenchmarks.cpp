// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGTestsCommon.h"

#include "Data/PCGPointArrayData.h"
#include "Data/PCGPointData.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include "Containers/StaticArray.h"

#include <catch2/catch_test_macros.hpp>
#include <catch2/benchmark/catch_benchmark.hpp>
#include <catch2/generators/catch_generators_range.hpp>

#include "TestHarness.h"

TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeAccessorBenchmarks", "[PCG][AttributeAccessors][!benchmark]")
{
	constexpr int32 Seed = 42;
	static const FPCGAttributePropertySelector PositionSelector = FPCGAttributePropertySelector::CreatePointPropertySelector(EPCGPointProperties::Position);
	static const FPCGAttributePropertySelector TransformPositionXSelector = FPCGAttributePropertySelector::CreatePointPropertySelector(EPCGPointProperties::Transform, NAME_None, {TEXT("Position"), TEXT("x")});
	static const FPCGAttributePropertySelector DensitySelector = FPCGAttributePropertySelector::CreatePointPropertySelector(EPCGPointProperties::Density);
	static const FName FloatAttributeName = "FloatAttr";
	static const FPCGAttributePropertySelector FloatAttributeSelector = FPCGAttributePropertySelector::CreateAttributeSelector(FloatAttributeName);

	// Preparing the data
	constexpr int32 NumPoints = 10000;
	UPCGPointData* PointData = CreateData<UPCGPointData>();
	UPCGPointArrayData* PointArrayData = CreateData<UPCGPointArrayData>();

	PointData->SetNumPoints(NumPoints);
	PointArrayData->SetNumPoints(NumPoints);
	
	PointData->AllocateProperties(EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::Density | EPCGPointNativeProperties::MetadataEntry);
	PointArrayData->AllocateProperties(EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::Density | EPCGPointNativeProperties::MetadataEntry);

	TPCGValueRange<FTransform> PointDataTransformRange = PointData->GetTransformValueRange(/*bAllocate=*/false);
	TPCGValueRange<float> PointDataDensityRange = PointData->GetDensityValueRange(/*bAllocate=*/false);
	TPCGValueRange<int64> PointDataMetadataEntryRange = PointData->GetMetadataEntryValueRange(/*bAllocate=*/false);
	FPCGMetadataAttribute<float>* PointDataFloatAttribute = PointData->MutableMetadata()->CreateAttribute<float>(FloatAttributeName, 0.f, true, false);

	TPCGValueRange<FTransform> PointArrayDataTransformRange = PointArrayData->GetTransformValueRange(/*bAllocate=*/false);
	TPCGValueRange<float> PointArrayDataDensityRange = PointArrayData->GetDensityValueRange(/*bAllocate=*/false);
	TPCGValueRange<int64> PointArrayDataMetadataEntryRange = PointArrayData->GetMetadataEntryValueRange(/*bAllocate=*/false);
	FPCGMetadataAttribute<float>* PointArrayDataFloatAttribute = PointArrayData->MutableMetadata()->CreateAttribute<float>(FloatAttributeName, 0.f, true, false);
	
	FRandomStream RandomStream(Seed);
	constexpr int32 Range = 1000;

	for (int32 i = 0; i < NumPoints; ++i)
	{
		const FTransform RandomTransform{RandomStream.VRand()};
		const float RandomDensity{RandomStream.FRand()};
		const float RandomFloatAttributeValue{RandomStream.FRand()};

		PointDataTransformRange[i] = RandomTransform;
		PointArrayDataTransformRange[i] = RandomTransform;

		PointDataDensityRange[i] = RandomDensity;
		PointArrayDataDensityRange[i] = RandomDensity;

		PointDataMetadataEntryRange[i] = PointData->MutableMetadata()->AddEntry();
		PointArrayDataMetadataEntryRange[i] = PointArrayData->MutableMetadata()->AddEntry();

		PointDataFloatAttribute->SetValue(PointDataMetadataEntryRange[i], RandomFloatAttributeValue);
		PointArrayDataFloatAttribute->SetValue(PointArrayDataMetadataEntryRange[i], RandomFloatAttributeValue);
	}
	
	auto [What, BasePointData] = GENERATE_COPY(table<FString, UPCGBasePointData*>({{TEXT("PointData"), PointData}, {TEXT("PointArrayData"), PointArrayData}}));

	DYNAMIC_SECTION(*What)
	{
		SECTION("Property accessor creation")
		{
			BENCHMARK("DensityAccessorCreation")
			{
				TUniquePtr<IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateAccessor(BasePointData, DensitySelector);
			};
		
			BENCHMARK("PositionAccessorCreation")
			{
				TUniquePtr<IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateAccessor(BasePointData, PositionSelector);
			};
		
			BENCHMARK("TransformPositionXAccessorCreation")
			{
				TUniquePtr<IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateAccessor(BasePointData, TransformPositionXSelector);
			};
		
			BENCHMARK("AttributeAccessorCreation")
			{
				TUniquePtr<IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateAccessor(BasePointData, FloatAttributeSelector);
			};
		}
	
		SECTION("Property reading single (1000 times)")
		{
			TUniquePtr<IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateAccessor(BasePointData, DensitySelector);
			TUniquePtr<IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateKeys(BasePointData, DensitySelector);
			CHECK(Keys.IsValid());
		
			BENCHMARK("DensityReadingSingle")
			{
				double Value;
				// Test the first then do not test the rest.
				REQUIRE(Accessor->Get(Value, *Keys));
				for (int32 i = 1; i < Range; ++i)
				{
					Accessor->Get(Value, i, *Keys);
				}
			};
		
			Accessor = PCGAttributeAccessorHelpers::CreateAccessor(BasePointData, PositionSelector);
			BENCHMARK("PositionReadingSingle")
			{
				FVector Value;
				// Test the first then do not test the rest.
				REQUIRE(Accessor->Get(Value, *Keys));
				for (int32 i = 1; i < Range; ++i)
				{
					Accessor->Get(Value, i, *Keys);
				}
			};
		
			Accessor = PCGAttributeAccessorHelpers::CreateAccessor(BasePointData, TransformPositionXSelector);
			BENCHMARK("TransformPositionXReadingSingle")
			{
				double Value;
				// Test the first then do not test the rest.
				REQUIRE(Accessor->Get(Value, *Keys));
				for (int32 i = 1; i < Range; ++i)
				{
					Accessor->Get(Value, i, *Keys);
				}
			};
		
			Accessor = PCGAttributeAccessorHelpers::CreateAccessor(BasePointData, FloatAttributeSelector);
			BENCHMARK("AttributeReadingSingle")
			{
				float Value;
				// Test the first then do not test the rest.
				REQUIRE(Accessor->Get(Value, *Keys));
				for (int32 i = 1; i < Range; ++i)
				{
					Accessor->Get(Value, i, *Keys);
				}
			};
		}

		SECTION("Property reading range (range 1000)")
		{
			TUniquePtr<IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateAccessor(BasePointData, DensitySelector);
			TUniquePtr<IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateKeys(BasePointData, DensitySelector);
			CHECK(Keys.IsValid());
			
			TStaticArray<uint8, sizeof(FVector)*Range> Buffer{};
		
			BENCHMARK("DensityReadingRange")
			{
				TArrayView<double> Values = MakeArrayView(reinterpret_cast<double*>(Buffer.GetData()), Range);
				REQUIRE(Accessor->GetRange(Values, 0, *Keys));
			};
		
			Accessor = PCGAttributeAccessorHelpers::CreateAccessor(BasePointData, PositionSelector);
			BENCHMARK("PositionReadingRange")
			{
				TArrayView<FVector> Values = MakeArrayView(reinterpret_cast<FVector*>(Buffer.GetData()), Range);
				REQUIRE(Accessor->GetRange(Values, 0, *Keys));
			};
		
			Accessor = PCGAttributeAccessorHelpers::CreateAccessor(BasePointData, TransformPositionXSelector);
			BENCHMARK("TransformPositionXReadingRange")
			{
				TArrayView<double> Values = MakeArrayView(reinterpret_cast<double*>(Buffer.GetData()), Range);
				REQUIRE(Accessor->GetRange(Values, 0, *Keys));
			};
		
			Accessor = PCGAttributeAccessorHelpers::CreateAccessor(BasePointData, FloatAttributeSelector);
			BENCHMARK("AttributeReadingRange")
			{
				TArrayView<float> Values = MakeArrayView(reinterpret_cast<float*>(Buffer.GetData()), Range);
				REQUIRE(Accessor->GetRange(Values, 0, *Keys));
			};
		}

		SECTION("Property writing single (1000 times)")
		{
			TUniquePtr<IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateAccessor(BasePointData, DensitySelector);
			TUniquePtr<IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateKeys(BasePointData, DensitySelector);
			CHECK(Keys.IsValid());
		
			BENCHMARK("DensityWritingSingle")
			{
				// Test the first then do not test the rest.
				REQUIRE(Accessor->Set<double>(0.0, *Keys));
				for (int32 i = 1; i < Range; ++i)
				{
					Accessor->Set<double>(i, i, *Keys);
				}
			};
		
			Accessor = PCGAttributeAccessorHelpers::CreateAccessor(BasePointData, PositionSelector);
			BENCHMARK("PositionWritingSingle")
			{
				// Test the first then do not test the rest.
				REQUIRE(Accessor->Set<FVector>(FVector::ZeroVector, *Keys));
				for (int32 i = 1; i < Range; ++i)
				{
					Accessor->Set<FVector>(FVector::OneVector * i, i, *Keys);
				}
			};
		
			Accessor = PCGAttributeAccessorHelpers::CreateAccessor(BasePointData, TransformPositionXSelector);
			BENCHMARK("TransformPositionXWritingSingle")
			{
				// Test the first then do not test the rest.
				REQUIRE(Accessor->Set<double>(0.0, *Keys));
				for (int32 i = 1; i < Range; ++i)
				{
					Accessor->Set<double>(i, i, *Keys);
				}
			};
		
			Accessor = PCGAttributeAccessorHelpers::CreateAccessor(BasePointData, FloatAttributeSelector);
			BENCHMARK("AttributeWritingSingle")
			{
				// Test the first then do not test the rest.
				REQUIRE(Accessor->Set<float>(0.0f, *Keys));
				for (int32 i = 1; i < Range; ++i)
				{
					Accessor->Set<float>(i, i, *Keys);
				}
			};
		}

		SECTION("Property writing range (range 1000)")
		{
			TUniquePtr<IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateAccessor(BasePointData, DensitySelector);
			TUniquePtr<IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateKeys(BasePointData, DensitySelector);
			CHECK(Keys.IsValid());
			
			TStaticArray<uint8, sizeof(FVector)*Range> Buffer{};

			for (int32 i = 0; i < Range; ++i)
			{
				reinterpret_cast<double*>(Buffer.GetData())[i] = i;
			}
			
			BENCHMARK("DensityWritingRange")
			{
				TArrayView<const double> Values = MakeArrayView(reinterpret_cast<double*>(Buffer.GetData()), Range);
				REQUIRE(Accessor->SetRange(Values, 0, *Keys));
			};
		
			Accessor = PCGAttributeAccessorHelpers::CreateAccessor(BasePointData, PositionSelector);
			for (int32 i = 0; i < Range; ++i)
			{
				reinterpret_cast<FVector*>(Buffer.GetData())[i] = FVector::OneVector * i;
			}
			
			BENCHMARK("PositionWritingRange")
			{
				TArrayView<const FVector> Values = MakeArrayView(reinterpret_cast<FVector*>(Buffer.GetData()), Range);
				REQUIRE(Accessor->SetRange(Values, 0, *Keys));
			};
		
			Accessor = PCGAttributeAccessorHelpers::CreateAccessor(BasePointData, TransformPositionXSelector);
			for (int32 i = 0; i < Range; ++i)
			{
				reinterpret_cast<double*>(Buffer.GetData())[i] = i;
			}
			
			BENCHMARK("TransformPositionXWritingRange")
			{
				TArrayView<const double> Values = MakeArrayView(reinterpret_cast<double*>(Buffer.GetData()), Range);
				REQUIRE(Accessor->SetRange(Values, 0, *Keys));
			};
		
			Accessor = PCGAttributeAccessorHelpers::CreateAccessor(BasePointData, FloatAttributeSelector);
			for (int32 i = 0; i < Range; ++i)
			{
				reinterpret_cast<float*>(Buffer.GetData())[i] = i;
			}
			
			BENCHMARK("AttributeWritingRange")
			{
				TArrayView<const float> Values = MakeArrayView(reinterpret_cast<float*>(Buffer.GetData()), Range);
				REQUIRE(Accessor->SetRange(Values, 0, *Keys));
			};
		}
	}
}