// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGTestsCommon.h"

#include "Data/PCGPointArrayData.h"
#include "Data/PCGPointData.h"

#include "Containers/StaticArray.h"

#include <catch2/catch_test_macros.hpp>
#include <catch2/benchmark/catch_benchmark.hpp>
#include <catch2/generators/catch_generators_range.hpp>

#include "TestHarness.h"

TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeGeneric::Benchmark", "[PCG][AttributeGeneric][!benchmark]")
{
	constexpr int32 Seed = 42;
	static const FName DoubleAttributeName = "DoubleAttr";
	static const FName DoubleGenericAttributeName = "DoubleGenericAttr";
	static const FName StringAttributeName = "StringAttr";
	static const FName StringGenericAttributeName = "StringGenericAttr";

	// Preparing the data
	constexpr int32 NumPoints = 4096;
	constexpr int32 ChunkSize = 256;

	SECTION("Double Attribute")
	{
		UPCGPointArrayData* PointArrayData = CreateData<UPCGPointArrayData>();

		PointArrayData->SetNumPoints(NumPoints);

		PointArrayData->AllocateProperties(EPCGPointNativeProperties::MetadataEntry);
		TPCGValueRange<int64> PointArrayDataMetadataEntryRange = PointArrayData->GetMetadataEntryValueRange(/*bAllocate=*/false);

		FPCGMetadataDomain* Domain = PointArrayData->MutableMetadata()->GetDefaultMetadataDomain();
		FPCGMetadataAttributeBase* GenericAttribute = Domain->CreateAttribute<double>(DoubleGenericAttributeName, 0.);
		FPCGMetadataAttribute<double>* TypedAttribute = Domain->CreateAttribute<double>(DoubleAttributeName, 0., /*bAllowsInterpolation=*/true, /*bOverrideParent=*/false);
		
		// Allocate entry keys and values
		TStaticArray<double, NumPoints> Values;
		for (int32 i = 0; i < NumPoints; ++i)
		{
			PointArrayDataMetadataEntryRange[i] = Domain->AddEntry();
			Values[i] = static_cast<double>(i);
		}

		TConstArrayView<PCGMetadataEntryKey> SingleKey = MakeConstArrayView(&PointArrayDataMetadataEntryRange[0], 1);
		BENCHMARK_ADVANCED("[Generic] Set single value") (Catch::Benchmark::Chronometer meter)
		{
			// Make sure to reset everytime so we don't stack the values.
			GenericAttribute->Reset();
			meter.measure([&](){ GenericAttribute->SetValues<double>(SingleKey, {1.0});});
		};
		
		BENCHMARK_ADVANCED("[Typed] Set single value") (Catch::Benchmark::Chronometer meter)
		{
			// Make sure to reset everytime so we don't stack the values.
			TypedAttribute->Reset();
			meter.measure([&](){ TypedAttribute->SetValues(SingleKey, {1.0});});
		};
		
		BENCHMARK_ADVANCED("[Generic] Set full range") (Catch::Benchmark::Chronometer meter)
		{
			// Make sure to reset everytime so we don't stack the values.
			GenericAttribute->Reset();
			meter.measure([&](){ GenericAttribute->SetValues<double>(PointArrayDataMetadataEntryRange, Values); });
		};
		
		BENCHMARK_ADVANCED("[Typed] Set full range") (Catch::Benchmark::Chronometer meter)
		{
			// Make sure to reset everytime so we don't stack the values.
			TypedAttribute->Reset();
			meter.measure([&](){ TypedAttribute->SetValues(PointArrayDataMetadataEntryRange, Values); });
		};
		
		TStaticArray<double, NumPoints> ReadValues;
		TStaticArray<const double*, NumPoints> ReadValuesPtr;
		
		BENCHMARK("[Generic] Get single value")
		{
			GenericAttribute->GetValuesFromItemKeys<double>(SingleKey, MakeArrayView(ReadValues.GetData(), 1));
		};
		
		BENCHMARK("[Generic] Get single value as pointer")
		{
			GenericAttribute->GetValuesFromItemKeys<const double*>(SingleKey, MakeArrayView(ReadValuesPtr.GetData(), 1));
		};
		
		BENCHMARK("[Typed] Get single value")
		{
			TypedAttribute->GetValuesFromItemKeys(SingleKey, MakeArrayView(ReadValues.GetData(), 1));
		};
		
		BENCHMARK("[Generic] Get full range")
		{
			GenericAttribute->GetValuesFromItemKeys<double>(PCGValueRangeHelpers::MakeConstValueRange(PointArrayDataMetadataEntryRange), ReadValues);
		};
		
		BENCHMARK("[Generic] Get full range as pointer")
		{
			GenericAttribute->GetValuesFromItemKeys<const double*>(PCGValueRangeHelpers::MakeConstValueRange(PointArrayDataMetadataEntryRange), ReadValuesPtr);
		};
		
		BENCHMARK("[Typed] Get full range")
		{
			TypedAttribute->GetValuesFromItemKeys(PCGValueRangeHelpers::MakeConstValueRange(PointArrayDataMetadataEntryRange), ReadValues);
		};
		
		// Finally test the worst case scenario where there is non-contiguous values.
		UPCGPointArrayData* ChildPointArrayData = CreateData<UPCGPointArrayData>();
		ChildPointArrayData->InitializeFromDataWithParams(FPCGInitializeFromDataParams{PointArrayData});
		TPCGValueRange<int64> ChildPointArrayDataMetadataEntryRange = ChildPointArrayData->GetMetadataEntryValueRange(/*bAllocate=*/false);

		FPCGMetadataDomain* ChildDomain = ChildPointArrayData->MutableMetadata()->GetDefaultMetadataDomain();
		FPCGMetadataAttributeBase* ChildGenericAttribute = ChildDomain->GetMutableAttribute(DoubleGenericAttributeName);
		FPCGMetadataAttribute<double>* ChildTypedAttribute = ChildDomain->GetMutableTypedAttribute<double>(DoubleAttributeName);
		
		CHECK_NOT_EQUAL(ChildGenericAttribute, nullptr);
		CHECK_NOT_EQUAL(ChildTypedAttribute, nullptr);
		
		// Allocate entry keys and values
		TStaticArray<double, NumPoints/2> ChildValues;
		TStaticArray<PCGMetadataEntryKey, NumPoints/2> ChildEntryKeys;
		for (int32 i = 0; i < NumPoints / 2; ++i)
		{
			ChildDomain->InitializeOnSet(ChildPointArrayDataMetadataEntryRange[2 * i]);
			ChildValues[i] = static_cast<double>(i + NumPoints * 2);
			ChildEntryKeys[i] = ChildPointArrayDataMetadataEntryRange[2 * i];
		}
		
		ChildGenericAttribute->SetValues<double>(ChildEntryKeys, ChildValues);
		ChildTypedAttribute->SetValues(ChildEntryKeys, ChildValues);
		
		CHECK_EQUAL(ChildGenericAttribute->GetNumberOfEntries(), NumPoints / 2);
		CHECK_EQUAL(ChildTypedAttribute->GetNumberOfEntries(), NumPoints / 2);
		
		BENCHMARK("[Generic] Get full range, with fragmented memory")
		{
			ChildGenericAttribute->GetValuesFromItemKeys<double>(PCGValueRangeHelpers::MakeConstValueRange(ChildPointArrayDataMetadataEntryRange), ReadValues);
		};
		
		BENCHMARK("[Generic] Get full range as pointer, with fragmented memory")
		{
			ChildGenericAttribute->GetValuesFromItemKeys<const double*>(PCGValueRangeHelpers::MakeConstValueRange(ChildPointArrayDataMetadataEntryRange), ReadValuesPtr);
		};
		
		BENCHMARK("[Typed] Get full range, with fragmented memory")
		{
			ChildTypedAttribute->GetValuesFromItemKeys(PCGValueRangeHelpers::MakeConstValueRange(ChildPointArrayDataMetadataEntryRange), ReadValues);
		};
	}
	
	SECTION("String Attribute")
	{
		UPCGPointArrayData* PointArrayData = CreateData<UPCGPointArrayData>();

		PointArrayData->SetNumPoints(NumPoints);

		PointArrayData->AllocateProperties(EPCGPointNativeProperties::MetadataEntry);
		TPCGValueRange<int64> PointArrayDataMetadataEntryRange = PointArrayData->GetMetadataEntryValueRange(/*bAllocate=*/false);

		FPCGMetadataDomain* Domain = PointArrayData->MutableMetadata()->GetDefaultMetadataDomain();
		FPCGMetadataAttributeBase* GenericAttribute = Domain->CreateAttribute<FString>(StringGenericAttributeName, FString{});
		FPCGMetadataAttribute<FString>* TypedAttribute = Domain->CreateAttribute<FString>(StringAttributeName, FString{}, /*bAllowsInterpolation=*/true, /*bOverrideParent=*/false);

		// Allocate entry keys and values
		TStaticArray<FString, NumPoints> Values;
		for (int32 i = 0; i < NumPoints; ++i)
		{
			PointArrayDataMetadataEntryRange[i] = Domain->AddEntry();
			Values[i] = FString::Printf(TEXT("This is a string with some characters and a value: %d"), i % 25);
		}

		TConstArrayView<PCGMetadataEntryKey> SingleKey = MakeConstArrayView(&PointArrayDataMetadataEntryRange[0], 1);
		BENCHMARK_ADVANCED("[Generic] Set single value") (Catch::Benchmark::Chronometer meter)
		{
			// Make sure to reset everytime so we don't stack the values.
			GenericAttribute->Reset();
			meter.measure([&](){ GenericAttribute->SetValues<FString>(SingleKey, {Values[0]});});
		};
		
		BENCHMARK_ADVANCED("[Typed] Set single value") (Catch::Benchmark::Chronometer meter)
		{
			// Make sure to reset everytime so we don't stack the values.
			TypedAttribute->Reset();
			meter.measure([&](){ TypedAttribute->SetValues(SingleKey, {Values[0]});});
		};
		
		BENCHMARK_ADVANCED("[Generic] Set full range") (Catch::Benchmark::Chronometer meter)
		{
			// Make sure to reset everytime so we don't stack the values.
			GenericAttribute->Reset();
			meter.measure([&](){ GenericAttribute->SetValues<FString>(PointArrayDataMetadataEntryRange, Values); });
		};
		
		BENCHMARK_ADVANCED("[Typed] Set full range") (Catch::Benchmark::Chronometer meter)
		{
			// Make sure to reset everytime so we don't stack the values.
			TypedAttribute->Reset();
			meter.measure([&](){ TypedAttribute->SetValues(PointArrayDataMetadataEntryRange, Values); });
		};
		
		TStaticArray<FString, NumPoints> ReadValues;
		TStaticArray<const FString*, NumPoints> ReadValuesPtr;
		
		BENCHMARK("[Generic] Get single value")
		{
			GenericAttribute->GetValuesFromItemKeys<FString>(SingleKey, MakeArrayView(ReadValues.GetData(), 1));
		};
		
		BENCHMARK("[Generic] Get single value pointer")
		{
			GenericAttribute->GetValuesFromItemKeys<const FString*>(SingleKey, MakeArrayView<const FString*>(ReadValuesPtr.GetData(), 1));
		};
		
		BENCHMARK("[Typed] Get single value")
		{
			TypedAttribute->GetValuesFromItemKeys(SingleKey, MakeArrayView(ReadValues.GetData(), 1));
		};
		
		BENCHMARK("[Generic] Get full range")
		{
			GenericAttribute->GetValuesFromItemKeys<FString>(PCGValueRangeHelpers::MakeConstValueRange(PointArrayDataMetadataEntryRange), ReadValues);
		};
		
		BENCHMARK("[Generic] Get full range as pointer")
		{
			GenericAttribute->GetValuesFromItemKeys<const FString*>(PCGValueRangeHelpers::MakeConstValueRange(PointArrayDataMetadataEntryRange), ReadValuesPtr);
		};
		
		BENCHMARK("[Typed] Get full range")
		{
			TypedAttribute->GetValuesFromItemKeys(PCGValueRangeHelpers::MakeConstValueRange(PointArrayDataMetadataEntryRange), ReadValues);
		};
		
		// Finally test the worst case scenario where there is non-contiguous values.
		UPCGPointArrayData* ChildPointArrayData = CreateData<UPCGPointArrayData>();
		ChildPointArrayData->InitializeFromDataWithParams(FPCGInitializeFromDataParams{PointArrayData});
		TPCGValueRange<int64> ChildPointArrayDataMetadataEntryRange = ChildPointArrayData->GetMetadataEntryValueRange(/*bAllocate=*/false);

		FPCGMetadataDomain* ChildDomain = ChildPointArrayData->MutableMetadata()->GetDefaultMetadataDomain();
		FPCGMetadataAttributeBase* ChildGenericAttribute = ChildDomain->GetMutableAttribute(StringGenericAttributeName);
		FPCGMetadataAttribute<FString>* ChildTypedAttribute = ChildDomain->GetMutableTypedAttribute<FString>(StringAttributeName);
		
		CHECK_NOT_EQUAL(ChildGenericAttribute, nullptr);
		CHECK_NOT_EQUAL(ChildTypedAttribute, nullptr);
		
		// Allocate entry keys and values
		TStaticArray<FString, NumPoints/2> ChildValues;
		TStaticArray<PCGMetadataEntryKey, NumPoints/2> ChildEntryKeys;
		for (int32 i = 0; i < NumPoints / 2; ++i)
		{
			ChildDomain->InitializeOnSet(ChildPointArrayDataMetadataEntryRange[2 * i]);
			ChildValues[i] = FString::Printf(TEXT("This is a string with some characters and a new value: %d"), i % 25);
			ChildEntryKeys[i] = ChildPointArrayDataMetadataEntryRange[2 * i];
		}
		
		ChildGenericAttribute->SetValues<FString>(ChildEntryKeys, ChildValues);
		ChildTypedAttribute->SetValues(ChildEntryKeys, ChildValues);
		
		CHECK_EQUAL(ChildGenericAttribute->GetNumberOfEntries(), NumPoints / 2);
		CHECK_EQUAL(ChildTypedAttribute->GetNumberOfEntries(), NumPoints / 2);
		
		BENCHMARK("[Generic] Get full range, with fragmented memory")
		{
			ChildGenericAttribute->GetValuesFromItemKeys<FString>(PCGValueRangeHelpers::MakeConstValueRange(ChildPointArrayDataMetadataEntryRange), ReadValues);
		};
		
		BENCHMARK("[Generic] Get full range as pointer, with fragmented memory")
		{
			ChildGenericAttribute->GetValuesFromItemKeys<const FString*>(PCGValueRangeHelpers::MakeConstValueRange(ChildPointArrayDataMetadataEntryRange), ReadValuesPtr);
		};
		
		BENCHMARK("[Typed] Get full range, with fragmented memory")
		{
			ChildTypedAttribute->GetValuesFromItemKeys(PCGValueRangeHelpers::MakeConstValueRange(ChildPointArrayDataMetadataEntryRange), ReadValues);
		};
	}
}