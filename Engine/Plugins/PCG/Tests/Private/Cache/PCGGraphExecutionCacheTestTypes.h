// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/PCGGraphPerExecutionCache.h"
#include "PCGDefaultExecutionSource.h"

#include "PCGGraphExecutionCacheTestTypes.generated.h"

/** Minimal cache entry used exclusively in PCGGraphExecutionCacheTests. */
USTRUCT()
struct FPCGTestCacheEntry : public FPCGPerExecutionCacheData
{
	GENERATED_BODY()

	using ValueType = int32;

	FPCGTestCacheEntry() = default;
	FPCGTestCacheEntry(ValueType InValue) : Value(InValue) {}

	ValueType GetValue() const { return Value; }

	ValueType Value = 0;
};

/** Execution state stub for cache tests — overrides only GetGenerationTaskId(). */
class FPCGTestExecutionState final : public FPCGDefaultExecutionState
{
public:
	explicit FPCGTestExecutionState(UPCGDefaultExecutionSource* InSource) : FPCGDefaultExecutionState(InSource) {}

	virtual FPCGTaskId GetGenerationTaskId() const override { return TaskId; }

	FPCGTaskId TaskId = InvalidPCGTaskId;
};

/** Execution source stub for cache tests — exposes a settable task ID. */
UCLASS()
class UPCGTestExecutionSource : public UPCGDefaultExecutionSource
{
	GENERATED_BODY()

public:
	UPCGTestExecutionSource()
	{
		State = MakeUnique<FPCGTestExecutionState>(this);
	}

	void SetTaskId(FPCGTaskId InTaskId)
	{
		static_cast<FPCGTestExecutionState&>(*State).TaskId = InTaskId;
	}
};
