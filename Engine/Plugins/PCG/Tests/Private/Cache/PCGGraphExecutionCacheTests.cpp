// Copyright Epic Games, Inc. All Rights Reserved.

// Test summary
// ┌─────────────────────────────────────────────────────┬───────────────────────────────────────────────────────────────-─┐
// │ Test name                                           │ What it covers                                                  │
// ├─────────────────────────────────────────────────────┼────────────────────────────────────────────────────────────────-┤
// │ ExecutionCache::GetExecutionCacheEntry              │ Returns unset optional when no entry exists or task is invalid; │
// │                                                     │ returns the stored value after SetExecutionCacheEntry.          │
// |                                                     |                                                                 |
// │ ExecutionCache::TaskIsolation                       │ Entries stored under task A are not visible under task B.       │
// |                                                     |                                                                 |
// │ ExecutionCache::CacheIdIsolation                    │ Entries stored under one typed ID are not visible under a       │
// │                                                     │ different ID for the same task.                                 │
// |                                                     |                                                                 |
// │ ExecutionCache::Clear                               │ After Clear(), the entries map is empty and no previously       │
// │                                                     │ stored value is retrievable.                                    │
// |                                                     |                                                                 |
// │ ExecutionCache::GetOrCreateWithNoTask               │ With an invalid task ID the lambda is called directly and its   │
// │                                                     │ result returned without being stored in the cache.              │
// |                                                     |                                                                 |
// │ ExecutionCache::CachingWithValidTask                │ First call (cache miss) invokes the lambda and stores the       │
// │                                                     │ result; subsequent calls return the cached value without        │
// │                                                     │ invoking the lambda again.                                      │
// |                                                     |                                                                 |
// │ ExecutionCache::ComponentGettersWithNoWorld         │ UPCGComponent data and bounds getters do not crash when there   │
// │                                                     │ is no world (and therefore no subsystem), and member-level      │
// │                                                     │ caches make repeated calls return the same pointer.             │
// |                                                     |                                                                 |
// │ ExecutionCache::GrowthDoesNotInvalidateEntries      │ Adding many entries after an initial write — growing both the   │
// │                                                     │ outer task map and the inner per-task GUID map — leaves the     │
// │                                                     │ original entry intact and readable.                             │
// |                                                     |                                                                 |
// │ ExecutionCache::                                    │ When InMakeEntry() itself calls GetOrCreateExecutionCacheValue  │
// │   ReentrantMakeEntryDoesNotCorruptPointers          │ and causes TMap rehashing, the outer call's Entry/StructData    │
// │                                                     │ pointers are refreshed so the computed value is stored          │
// │                                                     │ correctly in the cache.                                         │
// └─────────────────────────────────────────────────────┴───────────────────────────────────────────────────────────────-─┘

#include "PCGTestsCommon.h"

#include "PCGComponent.h"
#include "Cache/PCGGraphExecutionCacheTestTypes.h"
#include "Graph/PCGGraphPerExecutionCache.h"

#include "UObject/UObjectGlobals.h"

#include <catch2/catch_test_macros.hpp>

namespace PCGExecutionCacheTests
{
	/** Cache IDs that are not part of the production PCGPerExecutionCacheGuids namespace. */
	constexpr TPCGPerExecutionCacheId<FPCGTestCacheEntry> TestEntry { FGuid(0xC8A1F3E2, 0x4D7B9C05, 0xA362E817, 0x5F094D3B) };
	constexpr TPCGPerExecutionCacheId<FPCGTestCacheEntry> TestEntry2{ FGuid(0x1A2B3C4D, 0x5E6F7A8B, 0x9C0D1E2F, 0x3A4B5C6D) };
}

// ---------------------------------------------------------------------------
// FPCGPerExecutionCache — GetExecutionCacheEntry
// ---------------------------------------------------------------------------
// Verifies that GetExecutionCacheEntry returns an unset optional when no entry
// has been stored, and the correct value after SetExecutionCacheEntry.

TEST_CASE_METHOD(PCGTests::FPCGBaseTest, "PCG::ExecutionCache::GetExecutionCacheEntry", "[PCG][ExecutionCache]")
{
	FPCGPerExecutionCache Cache;

	UPCGTestExecutionSource* Source = NewObject<UPCGTestExecutionSource>();
	Source->SetTaskId(1);

	UPCGTestExecutionSource* NoTaskSource = NewObject<UPCGTestExecutionSource>();
	// NoTaskSource has TaskId == InvalidPCGTaskId by default.

	SECTION("Returns unset optional when no entry exists for the task")
	{
		const TOptional<int32> Result = Cache.GetExecutionCacheEntry(Source, PCGExecutionCacheTests::TestEntry);
		CHECK_FALSE(Result.IsSet());
	}

	SECTION("Returns unset optional when the source has no active task")
	{
		const TOptional<int32> Result = Cache.GetExecutionCacheEntry(NoTaskSource, PCGExecutionCacheTests::TestEntry);
		CHECK_FALSE(Result.IsSet());
	}

	SECTION("Returns the stored value after SetExecutionCacheEntry")
	{
		Cache.SetExecutionCacheEntry(Source, PCGExecutionCacheTests::TestEntry, 42, /*bValidateWritable=*/false);

		const TOptional<int32> Result = Cache.GetExecutionCacheEntry(Source, PCGExecutionCacheTests::TestEntry);
		REQUIRE(Result.IsSet());
		CHECK(Result.GetValue() == 42);
	}
}

// ---------------------------------------------------------------------------
// FPCGPerExecutionCache — task isolation
// ---------------------------------------------------------------------------
// Entries stored under one task ID must not be visible under a different task ID.

TEST_CASE_METHOD(PCGTests::FPCGBaseTest, "PCG::ExecutionCache::TaskIsolation", "[PCG][ExecutionCache]")
{
	FPCGPerExecutionCache Cache;

	UPCGTestExecutionSource* SourceA = NewObject<UPCGTestExecutionSource>();
	SourceA->SetTaskId(1);

	UPCGTestExecutionSource* SourceB = NewObject<UPCGTestExecutionSource>();
	SourceB->SetTaskId(2);

	Cache.SetExecutionCacheEntry(SourceA, PCGExecutionCacheTests::TestEntry, 10, /*bValidateWritable=*/false);

	const TOptional<int32> ResultA = Cache.GetExecutionCacheEntry(SourceA, PCGExecutionCacheTests::TestEntry);
	REQUIRE(ResultA.IsSet());
	CHECK(ResultA.GetValue() == 10);

	const TOptional<int32> ResultB = Cache.GetExecutionCacheEntry(SourceB, PCGExecutionCacheTests::TestEntry);
	CHECK_FALSE(ResultB.IsSet());
}

// ---------------------------------------------------------------------------
// FPCGPerExecutionCache — cache-ID isolation
// ---------------------------------------------------------------------------
// Entries stored under one typed ID must not be visible under a different ID
// for the same task.

TEST_CASE_METHOD(PCGTests::FPCGBaseTest, "PCG::ExecutionCache::CacheIdIsolation", "[PCG][ExecutionCache]")
{
	FPCGPerExecutionCache Cache;

	UPCGTestExecutionSource* Source = NewObject<UPCGTestExecutionSource>();
	Source->SetTaskId(1);

	Cache.SetExecutionCacheEntry(Source, PCGExecutionCacheTests::TestEntry, 7, /*bValidateWritable=*/false);

	const TOptional<int32> Result1 = Cache.GetExecutionCacheEntry(Source, PCGExecutionCacheTests::TestEntry);
	REQUIRE(Result1.IsSet());
	CHECK(Result1.GetValue() == 7);

	const TOptional<int32> Result2 = Cache.GetExecutionCacheEntry(Source, PCGExecutionCacheTests::TestEntry2);
	CHECK_FALSE(Result2.IsSet());
}

// ---------------------------------------------------------------------------
// FPCGPerExecutionCache — Clear
// ---------------------------------------------------------------------------
// After Clear(), no previously stored entries should be retrievable.

TEST_CASE_METHOD(PCGTests::FPCGBaseTest, "PCG::ExecutionCache::Clear", "[PCG][ExecutionCache]")
{
	FPCGPerExecutionCache Cache;

	UPCGTestExecutionSource* Source = NewObject<UPCGTestExecutionSource>();
	Source->SetTaskId(1);

	Cache.SetExecutionCacheEntry(Source, PCGExecutionCacheTests::TestEntry, 5, /*bValidateWritable=*/false);
	REQUIRE(Cache.GetExecutionCacheEntry(Source, PCGExecutionCacheTests::TestEntry).IsSet());

	Cache.Clear();

	CHECK(Cache.Entries.IsEmpty());
	CHECK_FALSE(Cache.GetExecutionCacheEntry(Source, PCGExecutionCacheTests::TestEntry).IsSet());
}

// ---------------------------------------------------------------------------
// FPCGPerExecutionCache — GetOrCreateExecutionCacheValue with no active task
// ---------------------------------------------------------------------------
// Verifies that with no active task the lambda is called directly and the
// result is returned without being stored in the cache.

TEST_CASE_METHOD(PCGTests::FPCGBaseTest, "PCG::ExecutionCache::GetOrCreateWithNoTask", "[PCG][ExecutionCache]")
{
	FPCGPerExecutionCache Cache;

	UPCGTestExecutionSource* NoTaskSource = NewObject<UPCGTestExecutionSource>();
	// NoTaskSource has TaskId == InvalidPCGTaskId by default.

	SECTION("Lambda is called and its value is returned")
	{
		bool bLambdaCalled = false;
		const FBox TestBox(FVector(-100.f), FVector(100.f));

		const FBox Result = Cache.GetOrCreateExecutionCacheValue(NoTaskSource, PCGPerExecutionCacheGuids::Bounds,
			[&bLambdaCalled, TestBox]()
			{
				bLambdaCalled = true;
				return TestBox;
			}, /*bValidateWritable=*/false);

		REQUIRE(bLambdaCalled);
		REQUIRE(Result == TestBox);
	}

	SECTION("Nothing is stored in the cache when there is no active task")
	{
		Cache.GetOrCreateExecutionCacheValue(NoTaskSource, PCGPerExecutionCacheGuids::Bounds,
			[]() { return FBox(FVector(-1.f), FVector(1.f)); }, /*bValidateWritable=*/false);

		CHECK(Cache.Entries.IsEmpty());
	}
}

// ---------------------------------------------------------------------------
// FPCGPerExecutionCache — caching behaviour with a valid task ID
// ---------------------------------------------------------------------------
// Uses a custom cache entry type (FPCGTestCacheEntry) to verify that:
//   - the first call invokes the lambda and stores the result, and
//   - subsequent calls return the cached value without invoking the lambda.

TEST_CASE_METHOD(PCGTests::FPCGBaseTest, "PCG::ExecutionCache::CachingWithValidTask", "[PCG][ExecutionCache]")
{
	FPCGPerExecutionCache Cache;
	constexpr FPCGTaskId TaskId = 1;

	UPCGTestExecutionSource* Source = NewObject<UPCGTestExecutionSource>();
	Source->SetTaskId(TaskId);

	int32 LambdaCallCount = 0;

	// First call: cache miss — lambda must be invoked and the value stored.
	const int32 First = Cache.GetOrCreateExecutionCacheValue(Source, PCGExecutionCacheTests::TestEntry,
		[&LambdaCallCount]() { ++LambdaCallCount; return 99; }, /*bValidateWritable=*/false);

	REQUIRE(LambdaCallCount == 1);
	REQUIRE(First == 99);

	SECTION("Lambda is not called again on a cache hit")
	{
		const int32 Second = Cache.GetOrCreateExecutionCacheValue(Source, PCGExecutionCacheTests::TestEntry,
			[&LambdaCallCount]() { ++LambdaCallCount; return 0; }, /*bValidateWritable=*/false);

		CHECK(LambdaCallCount == 1);
		CHECK(Second == 99);
	}

	SECTION("Entry is stored under the correct task ID")
	{
		REQUIRE(Cache.Entries.Contains(TaskId));
		CHECK(Cache.Entries[TaskId].Data.Contains(PCGExecutionCacheTests::TestEntry.Guid));
	}
}

// ---------------------------------------------------------------------------
// UPCGComponent getters — no world (component not placed in a level)
// ---------------------------------------------------------------------------
// Regression test: before the GetOrCreateExecutionCacheValue refactor some
// getters had an early "if (CurrentGenerationTask != InvalidPCGTaskId)"
// guard that returned nullptr without invoking the lambda.
//
// A bare AActor with no level/world is used as the owner so that
// GetOwner() is non-null (required by the data getter lambdas) while
// GetWorld() returns null (no subsystem reachable).  With no subsystem the
// static GetOrCreateExecutionCacheValue overload calls the lambda directly
// on every invocation.

TEST_CASE_METHOD(PCGTests::FPCGBaseTest, "PCG::ExecutionCache::ComponentGettersWithNoWorld", "[PCG][ExecutionCache]")
{
	AActor* Actor = NewObject<AActor>();
	UPCGComponent* Component = NewObject<UPCGComponent>(Actor);

	REQUIRE(Component->GetWorld() == nullptr);
	REQUIRE_FALSE(Component->IsGenerating());

	SECTION("Bounds getters do not crash")
	{
		// The actor has no components so the helpers return uninitialized
		// boxes, but must not crash.
		Component->GetGridBounds();
		Component->GetOriginalGridBounds();
		Component->GetLocalSpaceBounds();
		Component->GetOriginalLocalSpaceBounds();
		Component->GetTotalBounds();
	}

	SECTION("PCG data getters do not crash")
	{
		// All data getters must invoke the lambda path rather than
		// short-circuiting to null.  A bare actor produces no spatial data
		// so null returns are expected, but the calls must not crash.
		Component->GetPCGData();
		Component->GetInputPCGData();
		Component->GetActorPCGData();
		Component->GetOriginalActorPCGData();
		Component->GetLandscapePCGData();
		Component->GetLandscapeHeightPCGData();
	}

	SECTION("Data getter results are consistent across repeated calls")
	{
		// Component-level member caches (e.g. CachedActorData) must make
		// the same pointer be returned on repeated calls even without an
		// execution-cache task ID.
		UPCGData* const ActorData0 = Component->GetActorPCGData();
		UPCGData* const ActorData1 = Component->GetActorPCGData();
		CHECK(ActorData0 == ActorData1);

		UPCGData* const PCGData0 = Component->GetPCGData();
		UPCGData* const PCGData1 = Component->GetPCGData();
		CHECK(PCGData0 == PCGData1);
	}
}

// ---------------------------------------------------------------------------
// FPCGPerExecutionCache — cache growth does not invalidate existing entries
// ---------------------------------------------------------------------------
// TMap may rehash its internal storage when new elements are inserted.
// Verifies that entries written before a growth event are still present and
// hold their original values afterwards.
//
// Two growth axes are tested:
//   - Outer map growth: many distinct task IDs are added after the sentinel,
//     forcing the FPCGTaskId → FPCGPerExecutionCacheEntry map to rehash.
//   - Inner map growth: many distinct GUIDs are added under the sentinel's
//     task, forcing the per-task FGuid → TInstancedStruct map to rehash.

TEST_CASE_METHOD(PCGTests::FPCGBaseTest, "PCG::ExecutionCache::GrowthDoesNotInvalidateEntries", "[PCG][ExecutionCache]")
{
	constexpr int32    SentinelValue  = 12345;
	constexpr FPCGTaskId SentinelTaskId = 1;
	constexpr int32    NumExtraEntries = 128;

	SECTION("Outer map growth (many task IDs)")
	{
		FPCGPerExecutionCache Cache;

		UPCGTestExecutionSource* SentinelSource = NewObject<UPCGTestExecutionSource>();
		SentinelSource->SetTaskId(SentinelTaskId);
		Cache.SetExecutionCacheEntry(SentinelSource, PCGExecutionCacheTests::TestEntry, SentinelValue, /*bValidateWritable=*/false);

		// Insert entries under distinct task IDs to grow the outer TMap.
		for (int32 i = 0; i < NumExtraEntries; ++i)
		{
			UPCGTestExecutionSource* Source = NewObject<UPCGTestExecutionSource>();
			Source->SetTaskId(static_cast<FPCGTaskId>(SentinelTaskId + 1 + i));
			Cache.SetExecutionCacheEntry(Source, PCGExecutionCacheTests::TestEntry, i, /*bValidateWritable=*/false);
		}

		const TOptional<int32> Result = Cache.GetExecutionCacheEntry(SentinelSource, PCGExecutionCacheTests::TestEntry);
		REQUIRE(Result.IsSet());
		CHECK(Result.GetValue() == SentinelValue);
	}

	SECTION("Inner map growth (many GUIDs under one task)")
	{
		FPCGPerExecutionCache Cache;

		UPCGTestExecutionSource* Source = NewObject<UPCGTestExecutionSource>();
		Source->SetTaskId(SentinelTaskId);
		Cache.SetExecutionCacheEntry(Source, PCGExecutionCacheTests::TestEntry, SentinelValue, /*bValidateWritable=*/false);

		// Insert entries under distinct GUIDs in the same task to grow the inner TMap.
		for (int32 i = 0; i < NumExtraEntries; ++i)
		{
			const TPCGPerExecutionCacheId<FPCGTestCacheEntry> DynamicId{ FGuid(0xDEADBEEF, 0, 0, static_cast<uint32>(i)) };
			Cache.SetExecutionCacheEntry(Source, DynamicId, i, /*bValidateWritable=*/false);
		}

		const TOptional<int32> Result = Cache.GetExecutionCacheEntry(Source, PCGExecutionCacheTests::TestEntry);
		REQUIRE(Result.IsSet());
		CHECK(Result.GetValue() == SentinelValue);
	}
}

// ---------------------------------------------------------------------------
// FPCGPerExecutionCache — reentrant InMakeEntry does not corrupt cached pointers
// ---------------------------------------------------------------------------
// Regression test: when InMakeEntry() itself calls GetOrCreateExecutionCacheValue(),
// TMap rehashing can occur, invalidating the raw Entry/StructData pointers captured
// before the call.  The fix re-finds those pointers after InMakeEntry() returns so
// the computed value is written to the correct location in the cache.
//
// Two axes are tested:
//   - Inner TMap growth: the lambda inserts many GUIDs under the same task ID,
//     growing Entry->Data and potentially invalidating StructData.
//   - Outer TMap growth: the lambda inserts entries under many different task IDs,
//     growing Entries and potentially invalidating Entry.
//
// In both cases the outer entry must be stored correctly after the call.

TEST_CASE_METHOD(PCGTests::FPCGBaseTest, "PCG::ExecutionCache::ReentrantMakeEntryDoesNotCorruptPointers", "[PCG][ExecutionCache]")
{
	constexpr FPCGTaskId TaskId         = 1;
	constexpr int32      OuterValue     = 42;
	constexpr int32      InnerValue     = 99;
	// 128 reentrant insertions is more than enough to trigger TMap rehashing and
	// invalidate any raw pointers captured before the growth event.
	constexpr int32      NumInnerEntries = 128;

	SECTION("Inner TMap growth (same task, many GUIDs)")
	{
		FPCGPerExecutionCache Cache;

		UPCGTestExecutionSource* Source = NewObject<UPCGTestExecutionSource>();
		Source->SetTaskId(TaskId);

		// Build the dynamic IDs ahead of time so we can re-query them after the outer call.
		TArray<TPCGPerExecutionCacheId<FPCGTestCacheEntry>> DynamicIds;
		DynamicIds.Reserve(NumInnerEntries);
		for (int32 i = 0; i < NumInnerEntries; ++i)
		{
			DynamicIds.Add(TPCGPerExecutionCacheId<FPCGTestCacheEntry>{ FGuid(0xDEADBEEF, 0, 0, static_cast<uint32>(i)) });
		}

		// The lambda inserts many extra GUIDs under the same task, growing Entry->Data
		// and potentially invalidating the StructData pointer held by the outer call.
		const int32 Result = Cache.GetOrCreateExecutionCacheValue(Source, PCGExecutionCacheTests::TestEntry,
			[&Cache, Source, &DynamicIds]()
			{
				for (int32 i = 0; i < NumInnerEntries; ++i)
				{
					Cache.GetOrCreateExecutionCacheValue(Source, DynamicIds[i],
						[i]() { return InnerValue + i; }, /*bValidateWritable=*/false);
				}
				return OuterValue;
			}, /*bValidateWritable=*/false);

		CHECK(Result == OuterValue);

		const TOptional<int32> Cached = Cache.GetExecutionCacheEntry(Source, PCGExecutionCacheTests::TestEntry);
		REQUIRE(Cached.IsSet());
		CHECK(Cached.GetValue() == OuterValue);

		for (int32 i = 0; i < NumInnerEntries; ++i)
		{
			const TOptional<int32> InnerCached = Cache.GetExecutionCacheEntry(Source, DynamicIds[i]);
			REQUIRE(InnerCached.IsSet());
			CHECK(InnerCached.GetValue() == InnerValue + i);
		}
	}

	SECTION("Outer TMap growth (lambda inserts entries under many task IDs)")
	{
		FPCGPerExecutionCache Cache;

		UPCGTestExecutionSource* Source = NewObject<UPCGTestExecutionSource>();
		Source->SetTaskId(TaskId);

		// Build the inner sources ahead of time so we can re-query them after the outer call.
		TArray<UPCGTestExecutionSource*> InnerSources;
		InnerSources.Reserve(NumInnerEntries);
		for (int32 i = 0; i < NumInnerEntries; ++i)
		{
			UPCGTestExecutionSource* InnerSource = NewObject<UPCGTestExecutionSource>();
			InnerSource->SetTaskId(static_cast<FPCGTaskId>(TaskId + 1 + i));
			InnerSources.Add(InnerSource);
		}

		// The lambda inserts entries under many distinct task IDs, growing Entries and
		// potentially invalidating the Entry pointer held by the outer call.
		const int32 Result = Cache.GetOrCreateExecutionCacheValue(Source, PCGExecutionCacheTests::TestEntry,
			[&Cache, &InnerSources]()
			{
				for (int32 i = 0; i < NumInnerEntries; ++i)
				{
					Cache.GetOrCreateExecutionCacheValue(InnerSources[i], PCGExecutionCacheTests::TestEntry2,
						[i]() { return InnerValue + i; }, /*bValidateWritable=*/false);
				}
				return OuterValue;
			}, /*bValidateWritable=*/false);

		CHECK(Result == OuterValue);

		const TOptional<int32> Cached = Cache.GetExecutionCacheEntry(Source, PCGExecutionCacheTests::TestEntry);
		REQUIRE(Cached.IsSet());
		CHECK(Cached.GetValue() == OuterValue);

		for (int32 i = 0; i < NumInnerEntries; ++i)
		{
			const TOptional<int32> InnerCached = Cache.GetExecutionCacheEntry(InnerSources[i], PCGExecutionCacheTests::TestEntry2);
			REQUIRE(InnerCached.IsSet());
			CHECK(InnerCached.GetValue() == InnerValue + i);
		}
	}
}
