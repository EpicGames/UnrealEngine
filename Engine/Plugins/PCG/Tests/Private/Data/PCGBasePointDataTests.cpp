// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "TestHarness.h"

#include "PCGContext.h"
#include "Data/PCGBasePointData.h"
#include "Metadata/PCGMetadataAttributeTraits.h"
#include "Tests/PCGTestsCommon.h"

#include <catch2/catch_test_macros.hpp>
#include <catch2/generators/catch_generators_range.hpp>

namespace PCGBasePointDataTests
{
	template <class T>
	TArray<T> CreateValueArray(const T& InValue, int32 Size)
	{
		TArray<T> ValueArray;
		ValueArray.Reserve(Size);
		for (int32 Index = 0; Index < Size; ++Index)
		{
			ValueArray.Add(InValue);
		}

		return ValueArray;
	}

	template <class T>
	bool TestArraysEqual(const TArray<T>& ArrayA, const TArray<T>& ArrayB)
	{
		REQUIRE_EQUAL(ArrayA.Num(), ArrayB.Num());

		for (int32 i = 0; i < ArrayA.Num(); ++i)
		{
			REQUIRE(PCG::Private::MetadataTraits<T>::Equal(ArrayA[i], ArrayB[i]));
		}

		return true;
	}

	template <class T>
	bool TestRangeEqual(int32 RangeStartIndex, const TConstPCGValueRange<T>& Range, const TArray<T>& Array)
	{
		for (int32 i = 0; i < Array.Num(); ++i)
		{
			const int32 RangeIndex = RangeStartIndex + i;
			REQUIRE(PCG::Private::MetadataTraits<T>::Equal(Range[RangeIndex], Array[i]));
		}

		return true;
	}
};

TEST_CASE("PCG::BasePointData::API", "[PCG][BasePointData][API]")
{
	const FTransform Location(FVector(1.f, 2.f, 3.f));
	const float Density = 4.f;
	const int32 Seed = 5;

	FPCGPoint PCGPoint(Location, Density, Seed);
	PCGPoint.BoundsMin = FVector(-6.f, -6.f, -6.f);
	PCGPoint.BoundsMax = FVector(7.f, 7.f, 7.f);
	PCGPoint.Color = FVector4(8, 8, 8, 8);
	PCGPoint.MetadataEntry = 9;
	PCGPoint.Steepness = 10.f;

	const int32 NumPoints = 10;
	FPCGContext Context{};
	UPCGBasePointData* PointData = FPCGContext::NewPointData_AnyThread(&Context);
	PointData->SetNumPoints(NumPoints);
	PointData->AllocateProperties(EPCGPointNativeProperties::All);

	// Individual Properties
	SECTION("PerProperty")
	{
		int32 RangeStartIndex = GENERATE_COPY(range(0, NumPoints));
		int32 RangeSize = GENERATE_COPY(range(1, NumPoints - RangeStartIndex + 1));

		REQUIRE(RangeStartIndex + RangeSize <= NumPoints);

		FPCGPointOutputRange OutputRange{ PointData, RangeStartIndex, RangeSize };
		const FPCGPointInputRange InputRange{ PointData, RangeStartIndex, RangeSize };

		SECTION("Transform")
		{
			TArray<FTransform> SetTransforms = PCGBasePointDataTests::CreateValueArray(PCGPoint.Transform, OutputRange.RangeSize);
			PointData->SetTransformValuesOnRange(OutputRange, SetTransforms);

			TArray<FTransform> GetTransforms = PointData->GetTransformValuesFromRange(InputRange);

			PCGBasePointDataTests::TestArraysEqual(SetTransforms, GetTransforms);

			const TConstPCGValueRange<FTransform> ConstTransformRange = PointData->GetConstTransformValueRange();

			PCGBasePointDataTests::TestRangeEqual(RangeStartIndex, ConstTransformRange, SetTransforms);
		}

		SECTION("Density")
		{
			TArray<float> SetDensity = PCGBasePointDataTests::CreateValueArray(PCGPoint.Density, OutputRange.RangeSize);
			PointData->SetDensityValuesOnRange(OutputRange, SetDensity);

			TArray<float> GetDensity = PointData->GetDensityValuesFromRange(InputRange);

			PCGBasePointDataTests::TestArraysEqual(SetDensity, GetDensity);

			const TConstPCGValueRange<float> ConstDensityRange = PointData->GetConstDensityValueRange();

			PCGBasePointDataTests::TestRangeEqual(RangeStartIndex, ConstDensityRange, SetDensity);
		}

		SECTION("Seed")
		{
			TArray<int32> SetSeed = PCGBasePointDataTests::CreateValueArray(PCGPoint.Seed, OutputRange.RangeSize);
			PointData->SetSeedValuesOnRange(OutputRange, SetSeed);

			TArray<int32> GetSeed = PointData->GetSeedValuesFromRange(InputRange);

			PCGBasePointDataTests::TestArraysEqual(SetSeed, GetSeed);

			const TConstPCGValueRange<int32> ConstSeedRange = PointData->GetConstSeedValueRange();

			PCGBasePointDataTests::TestRangeEqual(RangeStartIndex, ConstSeedRange, SetSeed);
		}

		SECTION("BoundsMin")
		{
			TArray<FVector> SetBoundsMin = PCGBasePointDataTests::CreateValueArray(PCGPoint.BoundsMin, OutputRange.RangeSize);
			PointData->SetBoundsMinValuesOnRange(OutputRange, SetBoundsMin);

			TArray<FVector> GetBoundsMin = PointData->GetBoundsMinValuesFromRange(InputRange);

			PCGBasePointDataTests::TestArraysEqual(SetBoundsMin, GetBoundsMin);

			const TConstPCGValueRange<FVector> ConstBoundsMinRange = PointData->GetConstBoundsMinValueRange();

			PCGBasePointDataTests::TestRangeEqual(RangeStartIndex, ConstBoundsMinRange, SetBoundsMin);
		}

		SECTION("BoundsMax")
		{
			TArray<FVector> SetBoundsMax = PCGBasePointDataTests::CreateValueArray(PCGPoint.BoundsMax, OutputRange.RangeSize);
			PointData->SetBoundsMaxValuesOnRange(OutputRange, SetBoundsMax);

			TArray<FVector> GetBoundsMax = PointData->GetBoundsMaxValuesFromRange(InputRange);

			PCGBasePointDataTests::TestArraysEqual(SetBoundsMax, GetBoundsMax);

			const TConstPCGValueRange<FVector> ConstBoundsMaxRange = PointData->GetConstBoundsMaxValueRange();

			PCGBasePointDataTests::TestRangeEqual(RangeStartIndex, ConstBoundsMaxRange, SetBoundsMax);
		}

		SECTION("Color")
		{
			TArray<FVector4> SetColor = PCGBasePointDataTests::CreateValueArray(PCGPoint.Color, OutputRange.RangeSize);
			PointData->SetColorValuesOnRange(OutputRange, SetColor);

			TArray<FVector4> GetColor = PointData->GetColorValuesFromRange(InputRange);

			PCGBasePointDataTests::TestArraysEqual(SetColor, GetColor);

			const TConstPCGValueRange<FVector4> ConstColorRange = PointData->GetConstColorValueRange();

			PCGBasePointDataTests::TestRangeEqual(RangeStartIndex, ConstColorRange, SetColor);
		}

		SECTION("Steepness")
		{
			TArray<float> SetSteepness = PCGBasePointDataTests::CreateValueArray(PCGPoint.Steepness, OutputRange.RangeSize);
			PointData->SetSteepnessValuesOnRange(OutputRange, SetSteepness);

			TArray<float> GetSteepness = PointData->GetSteepnessValuesFromRange(InputRange);

			PCGBasePointDataTests::TestArraysEqual(SetSteepness, GetSteepness);

			const TConstPCGValueRange<float> ConstSteepnessRange = PointData->GetConstSteepnessValueRange();

			PCGBasePointDataTests::TestRangeEqual(RangeStartIndex, ConstSteepnessRange, SetSteepness);
		}

		SECTION("Metadata Entry")
		{
			TArray<int64> SetMetadataEntry = PCGBasePointDataTests::CreateValueArray(PCGPoint.MetadataEntry, OutputRange.RangeSize);
			PointData->SetMetadataEntryValuesOnRange(OutputRange, SetMetadataEntry);

			TArray<int64> GetMetadataEntry = PointData->GetMetadataEntryValuesFromRange(InputRange);

			PCGBasePointDataTests::TestArraysEqual(SetMetadataEntry, GetMetadataEntry);

			const TConstPCGValueRange<int64> ConstMetadataEntryRange = PointData->GetConstMetadataEntryValueRange();

			PCGBasePointDataTests::TestRangeEqual(RangeStartIndex, ConstMetadataEntryRange, SetMetadataEntry);
		}
	}

	// Full Points
	SECTION("FullPoints")
	{
		int32 RangeStartIndex = GENERATE_COPY(range(0, NumPoints));
		int32 RangeSize = NumPoints - RangeStartIndex;
		int32 Index = GENERATE_COPY(range(0, RangeSize));

		REQUIRE(RangeStartIndex + RangeSize <= NumPoints);
		REQUIRE(Index < RangeStartIndex + RangeSize);

		FPCGPointOutputRange OutputRange{ PointData, RangeStartIndex, RangeSize };
		const FPCGPointInputRange InputRange{ PointData, RangeStartIndex, RangeSize };
			
		// Points should be default value so different
		FPCGPoint PointFromRange = PointData->GetPointFromRange(InputRange, Index);
		REQUIRE_FALSE(PCGTestsCommon::PointsAreIdentical(PCGPoint, PointFromRange));

		// Do SetPointOnRange
		PointData->SetPointOnRange(OutputRange, Index, PCGPoint);

		// Get result using GetPointFromRange
		PointFromRange = PointData->GetPointFromRange(InputRange, Index);

		// Compare
		REQUIRE(PCGTestsCommon::PointsAreIdentical(PCGPoint, PointFromRange));

		// Get result using FConstPCGPointValueRanges for extra validation that proper index was used
		FConstPCGPointValueRanges PointValueRange(PointData);
		PointFromRange = PointValueRange.GetPoint(RangeStartIndex + Index);

		REQUIRE(PCGTestsCommon::PointsAreIdentical(PCGPoint, PointFromRange));
	}
}
