// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "TestHarness.h"

#include "PCGData.h"
#include "PCGModule.h"
#include "PCGParamData.h"
#include "Compute/Data/PCGProxyForGPUData.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGDynamicMeshData.h"
#include "Data/PCGLandscapeData.h"
#include "Data/PCGLandscapeSplineData.h"
#include "Data/PCGPolygon2DData.h"
#include "Data/PCGPolyLineData.h"
#include "Data/PCGPointData.h"
#include "Data/PCGPointArrayData.h"
#include "Data/PCGPrimitiveData.h"
#include "Data/PCGRenderTargetData.h"
#include "Data/PCGSpatialData.h"
#include "Data/PCGSplineData.h"
#include "Data/PCGStaticMeshResourceData.h"
#include "Data/PCGTextureData.h"
#include "Data/PCGTexture2DArrayData.h"
#include "Data/PCGVirtualTextureData.h"
#include "Data/PCGVolumeData.h"

#include "PCGTestsCommon.h"

#include <catch2/catch_test_macros.hpp>
#include <catch2/generators/catch_generators.hpp>

class FPCGDataTypeRegistryTestBase : public PCGTests::FPCGBaseTest
{
	using PCGTests::FPCGBaseTest::FPCGBaseTest;

protected:
	static bool IsRegistered(const FPCGDataTypeBaseId& ID)
	{
		return FPCGModule::GetConstDataTypeRegistry().IsRegistered(ID);
	}
};

TEST_CASE_METHOD(FPCGDataTypeRegistryTestBase, "PCG::DataTypeRegistry", "[PCG][DataTypeRegistry]")
{
	const FPCGDataTypeRegistry& Registry = FPCGModule::GetConstDataTypeRegistry();

	SECTION("ByClass")
	{
		TArray<FPCGDataTypeIdentifier> Identifiers;

		// Not using generators here because this is a sequential testing (we need to verify that ALL identifiers are unique)
		TTuple<const TCHAR*, FPCGDataTypeIdentifier> TestValues[] =
		{
			{TEXT("Any"), FPCGDataTypeIdentifier::Construct<UPCGData>()},
			{TEXT("Spatial"), FPCGDataTypeIdentifier::Construct<UPCGSpatialData>()},
			{TEXT("Point"), FPCGDataTypeIdentifier::Construct<UPCGBasePointData>()},
			{TEXT("Volume"), FPCGDataTypeIdentifier::Construct<UPCGVolumeData>()},
			{TEXT("Primitive"), FPCGDataTypeIdentifier::Construct<UPCGPrimitiveData>()},
			{TEXT("Polyline"), FPCGDataTypeIdentifier::Construct<UPCGPolyLineData>()},
			{TEXT("Spline"), FPCGDataTypeIdentifier::Construct<UPCGSplineData>()},
			{TEXT("Polygon2D"), FPCGDataTypeIdentifier::Construct<UPCGPolygon2DData>()},
			{TEXT("Landscape spline"), FPCGDataTypeIdentifier::Construct<UPCGLandscapeSplineData>()},
			{TEXT("Landscape"), FPCGDataTypeIdentifier::Construct<UPCGLandscapeData>()},
			{TEXT("Base texture"), FPCGDataTypeIdentifier::Construct<UPCGTexture2DSingleBaseData>()},
			{TEXT("Texture"), FPCGDataTypeIdentifier::Construct<UPCGTextureData>()},
			{TEXT("Texture 2D Array"), FPCGDataTypeIdentifier::Construct<UPCGTexture2DArrayData>()},
			{TEXT("Proxy for GPU"), FPCGDataTypeIdentifier::Construct<UPCGProxyForGPUData>()},
			{TEXT("Static mesh resource"), FPCGDataTypeIdentifier::Construct<UPCGStaticMeshResourceData>()},
			{TEXT("Virtual Texture"), FPCGDataTypeIdentifier::Construct<UPCGVirtualTextureData>()},
			{TEXT("Render target"), FPCGDataTypeIdentifier::Construct<UPCGRenderTargetData>()},
			{TEXT("Settings"), FPCGDataTypeIdentifier::Construct<UPCGSettings>()},
			{TEXT("Param"), FPCGDataTypeIdentifier::Construct<UPCGParamData>()},
			{TEXT("DynamicMesh"), FPCGDataTypeIdentifier::Construct<UPCGDynamicMeshData>()},
		};

		for (const auto& [TypeName, Identifier] : TestValues)
		{
			CAPTURE(TypeName);
			REQUIRE(Identifier.IsValid());
			REQUIRE_FALSE(Identifiers.ContainsByPredicate([&Identifier](const FPCGDataTypeIdentifier& Other) { return Identifier.IsSameType(Other); }));
			Identifiers.Add(Identifier);
		}
	}

	SECTION("ByDefaultType")
	{
		// Test invalid first
		SECTION("None type is invalid")
		{
			REQUIRE_FALSE(FPCGDataTypeIdentifier{ EPCGDataType::None }.IsValid());
		}

		TArray<FPCGDataTypeIdentifier> Identifiers;

		// Not using generators here because this is a sequential testing (we need to verify that ALL identifiers are unique)
		TTuple<const TCHAR*, FPCGDataTypeIdentifier> TestValues[] =
		{
			{TEXT("Any"), FPCGDataTypeIdentifier(EPCGDataType::Any)},
			{TEXT("Spatial"), FPCGDataTypeIdentifier(EPCGDataType::Spatial)},
			{TEXT("Point"), FPCGDataTypeIdentifier(EPCGDataType::Point)},
			{TEXT("Volume"), FPCGDataTypeIdentifier(EPCGDataType::Volume)},
			{TEXT("Primitive"), FPCGDataTypeIdentifier(EPCGDataType::Primitive)},
			{TEXT("Polyline"), FPCGDataTypeIdentifier(EPCGDataType::PolyLine)},
			{TEXT("Spline"), FPCGDataTypeIdentifier(EPCGDataType::Spline)},
			{TEXT("Polygon2D"), FPCGDataTypeIdentifier(EPCGDataType::Polygon2D)},
			{TEXT("Landscape spline"), FPCGDataTypeIdentifier(EPCGDataType::LandscapeSpline)},
			{TEXT("Landscape"), FPCGDataTypeIdentifier(EPCGDataType::Landscape)},
			{TEXT("Base texture"), FPCGDataTypeIdentifier(EPCGDataType::BaseTexture)},
			{TEXT("Texture"), FPCGDataTypeIdentifier(EPCGDataType::Texture)},
			{TEXT("Proxy for GPU"), FPCGDataTypeIdentifier(EPCGDataType::ProxyForGPU)},
			{TEXT("Static mesh resource"), FPCGDataTypeIdentifier(EPCGDataType::StaticMeshResource)},
			{TEXT("Virtual Texture"), FPCGDataTypeIdentifier(EPCGDataType::VirtualTexture)},
			{TEXT("Render target"),FPCGDataTypeIdentifier(EPCGDataType::RenderTarget)},
			{TEXT("Settings"), FPCGDataTypeIdentifier(EPCGDataType::Settings)},
			{TEXT("Param"), FPCGDataTypeIdentifier(EPCGDataType::Param)},
			{TEXT("PointOrParam"), FPCGDataTypeIdentifier(EPCGDataType::PointOrParam)},
			{TEXT("DynamicMesh"), FPCGDataTypeIdentifier(EPCGDataType::DynamicMesh)}
		};

		for (const auto& [TypeName, Identifier] : TestValues)
		{
			CAPTURE(TypeName);
			REQUIRE(Identifier.IsValid());

			const bool bRegistered = Algo::AllOf(Identifier.GetIds(), [](const FPCGDataTypeBaseId ID) { return IsRegistered(ID); });
			REQUIRE(bRegistered);

			REQUIRE_FALSE(Identifiers.ContainsByPredicate([&Identifier](const FPCGDataTypeIdentifier& Other) { return Identifier.IsSameType(Other); }));
			Identifiers.Add(Identifier);
		}
	}

	SECTION("Compatibility")
	{
		auto [TypeA, TypeB, ExpectedCompatibility] = GENERATE(
			table<FPCGDataTypeIdentifier, FPCGDataTypeIdentifier, EPCGDataTypeCompatibilityResult>({
			{EPCGDataType::Any, EPCGDataType::None, EPCGDataTypeCompatibilityResult::UnknownType},
			{EPCGDataType::None, EPCGDataType::None, EPCGDataTypeCompatibilityResult::UnknownType},
			{EPCGDataType::None, EPCGDataType::Any, EPCGDataTypeCompatibilityResult::UnknownType},
		
			{EPCGDataType::Any, EPCGDataType::Spatial, EPCGDataTypeCompatibilityResult::RequireFilter},
			{EPCGDataType::Any, EPCGDataType::Concrete, EPCGDataTypeCompatibilityResult::RequireConversion},
			{EPCGDataType::Any, EPCGDataType::Point, EPCGDataTypeCompatibilityResult::RequireFilter},
		
			{EPCGDataType::None, EPCGDataType::None, EPCGDataTypeCompatibilityResult::UnknownType},
		
			{EPCGDataType::Spatial, EPCGDataType::Any, EPCGDataTypeCompatibilityResult::Compatible},
			{EPCGDataType::Point, EPCGDataType::Any, EPCGDataTypeCompatibilityResult::Compatible},
			{EPCGDataType::Any, EPCGDataType::Any, EPCGDataTypeCompatibilityResult::Compatible},
		
			{EPCGDataType::Point, EPCGDataType::Point, EPCGDataTypeCompatibilityResult::Compatible},
		
			{EPCGDataType::Spatial, EPCGDataType::Concrete, EPCGDataTypeCompatibilityResult::RequireConversion},
			{EPCGDataType::Spatial, EPCGDataType::Point, EPCGDataTypeCompatibilityResult::RequireConversion},
			{EPCGDataType::Spatial, EPCGDataType::Volume, EPCGDataTypeCompatibilityResult::RequireFilter},
			{EPCGDataType::Spatial, EPCGDataType::Surface, EPCGDataTypeCompatibilityResult::RequireFilter},
			{EPCGDataType::Spatial, EPCGDataType::Spline, EPCGDataTypeCompatibilityResult::RequireFilter},
		
			{EPCGDataType::Composite, EPCGDataType::Concrete, EPCGDataTypeCompatibilityResult::RequireConversion},
		
			{EPCGDataType::Spline, EPCGDataType::Point, EPCGDataTypeCompatibilityResult::RequireConversion},
			{EPCGDataType::Volume, EPCGDataType::Point, EPCGDataTypeCompatibilityResult::RequireConversion},
		
			{EPCGDataType::Spline, EPCGDataType::Surface, EPCGDataTypeCompatibilityResult::RequireConversion},
			{EPCGDataType::Polygon2D, EPCGDataType::Surface, EPCGDataTypeCompatibilityResult::RequireConversion},
		
			{EPCGDataType::Spline, EPCGDataType::LandscapeSpline, EPCGDataTypeCompatibilityResult::NotCompatible},
			{EPCGDataType::Spline, EPCGDataType::PolyLine, EPCGDataTypeCompatibilityResult::Compatible},
			{EPCGDataType::LandscapeSpline, EPCGDataType::PolyLine, EPCGDataTypeCompatibilityResult::Compatible},
		
			{EPCGDataType::Landscape, EPCGDataType::Texture, EPCGDataTypeCompatibilityResult::NotCompatible},
			{EPCGDataType::Landscape, EPCGDataType::Surface, EPCGDataTypeCompatibilityResult::Compatible},
			{EPCGDataType::Surface, EPCGDataType::Landscape, EPCGDataTypeCompatibilityResult::RequireFilter},

			// Explicit construction of overlapping types
			{EPCGDataType::Surface, FPCGDataTypeIdentifier::Construct(EPCGDataType::Surface, EPCGDataType::Landscape), EPCGDataTypeCompatibilityResult::Compatible},
			{EPCGDataType::Landscape, FPCGDataTypeIdentifier::Construct(EPCGDataType::Surface, EPCGDataType::Landscape), EPCGDataTypeCompatibilityResult::Compatible},
			{EPCGDataType::Surface, FPCGDataTypeIdentifier::Construct(EPCGDataType::Landscape, EPCGDataType::Surface), EPCGDataTypeCompatibilityResult::Compatible},
			{EPCGDataType::Landscape, FPCGDataTypeIdentifier::Construct(EPCGDataType::Landscape, EPCGDataType::Surface), EPCGDataTypeCompatibilityResult::Compatible},
		
			{EPCGDataType::RenderTarget, EPCGDataType::Texture, EPCGDataTypeCompatibilityResult::NotCompatible},
			{EPCGDataType::BaseTexture, EPCGDataType::Texture, EPCGDataTypeCompatibilityResult::RequireFilter},
			{EPCGDataType::Texture, EPCGDataType::BaseTexture, EPCGDataTypeCompatibilityResult::Compatible},
			{EPCGDataType::BaseTexture, EPCGDataType::RenderTarget, EPCGDataTypeCompatibilityResult::RequireFilter},
			{EPCGDataType::RenderTarget, EPCGDataType::BaseTexture, EPCGDataTypeCompatibilityResult::Compatible},
			{EPCGDataType::VirtualTexture, EPCGDataType::Texture, EPCGDataTypeCompatibilityResult::NotCompatible},
			{EPCGDataType::VirtualTexture, EPCGDataType::Surface, EPCGDataTypeCompatibilityResult::Compatible},
		
			{EPCGDataType::Point, EPCGDataType::Param, EPCGDataTypeCompatibilityResult::RequireConversion},
			{EPCGDataType::Param, EPCGDataType::Point, EPCGDataTypeCompatibilityResult::RequireConversion},
		
			{EPCGDataType::PointOrParam, EPCGDataType::Param, EPCGDataTypeCompatibilityResult::RequireFilter},
			{EPCGDataType::PointOrParam, EPCGDataType::Point, EPCGDataTypeCompatibilityResult::RequireFilter},
			{EPCGDataType::Point, EPCGDataType::PointOrParam, EPCGDataTypeCompatibilityResult::Compatible},
			{EPCGDataType::Param, EPCGDataType::Param, EPCGDataTypeCompatibilityResult::Compatible},


			{EPCGDataType::Spline | EPCGDataType::Point, EPCGDataType::Spline, EPCGDataTypeCompatibilityResult::RequireFilter},
			{EPCGDataType::Spline | EPCGDataType::Point, EPCGDataType::Point, EPCGDataTypeCompatibilityResult::RequireFilter},
			{EPCGDataType::Spline | EPCGDataType::Point, EPCGDataType::Param, EPCGDataTypeCompatibilityResult::NotCompatible},
			{EPCGDataType::Spline | EPCGDataType::Point, EPCGDataType::Spline | EPCGDataType::Landscape, EPCGDataTypeCompatibilityResult::RequireFilter},
		
			{EPCGDataType::Param, EPCGDataType::Spline | EPCGDataType::Point, EPCGDataTypeCompatibilityResult::NotCompatible},
			{EPCGDataType::Spline, EPCGDataType::Spline | EPCGDataType::Point, EPCGDataTypeCompatibilityResult::Compatible},
			{EPCGDataType::Point, EPCGDataType::Spline | EPCGDataType::Point, EPCGDataTypeCompatibilityResult::Compatible},
			{EPCGDataType::Spline | EPCGDataType::Landscape, EPCGDataType::Spline | EPCGDataType::Point, EPCGDataTypeCompatibilityResult::RequireFilter},
		}));

		CAPTURE(*FString::Printf(TEXT("Compatiblity between %s and %s"), *TypeA.ToString(), *TypeB.ToString()));
		const EPCGDataTypeCompatibilityResult Result = Registry.IsCompatible(TypeA, TypeB);
		REQUIRE_EQUAL(Result, ExpectedCompatibility);
	}

	SECTION("Aggregation")
	{
		auto [Type, ExpectedIdentifier] = GENERATE(
			table<FPCGDataTypeIdentifier, FPCGDataTypeIdentifier>(
				{
					{
						FPCGDataTypeIdentifier{EPCGDataType::Point} | EPCGDataType::None, 
						FPCGDataTypeIdentifier::Construct<UPCGPointData>()
					},
					{ 
						EPCGDataType::Point | EPCGDataType::Param, 
						FPCGDataTypeIdentifier::Construct<UPCGPointData, UPCGParamData>()
					},
					{
						EPCGDataType::Point | EPCGDataType::Param,
						FPCGDataTypeIdentifier::Construct<UPCGParamData, UPCGPointData>()
					},
					{
						EPCGDataType::PointOrParam,
						FPCGDataTypeIdentifier::Construct<UPCGPointArrayData, UPCGParamData>()
					},
					{
						EPCGDataType::PointOrParam,
						FPCGDataTypeIdentifier::Construct<UPCGParamData, UPCGBasePointData>()
					},
					{
						EPCGDataType::Spline | EPCGDataType::Volume,
						FPCGDataTypeIdentifier::Construct<UPCGSplineData, UPCGVolumeData>()
					},
					{
						EPCGDataType::Texture | EPCGDataType::Point | EPCGDataType::Landscape,
						FPCGDataTypeIdentifier::Construct<UPCGTextureData, UPCGBasePointData, UPCGLandscapeData>()
					},
				}));

		REQUIRE(Type.IsSameType(ExpectedIdentifier));
	}

	SECTION("Intersection")
	{
		auto [Types, ExpectedIdentifier] = GENERATE(
			table<TArray<EPCGDataType>, FPCGDataTypeIdentifier>(
		{
			{ 
				TArray{EPCGDataType::Point, EPCGDataType::None}, 
				FPCGDataTypeIdentifier{}
			},
			{ 
				TArray{EPCGDataType::Point, EPCGDataType::Param}, 
				FPCGDataTypeIdentifier{}
			},
			{
				TArray{EPCGDataType::Spline, EPCGDataType::Volume},
				FPCGDataTypeIdentifier{}
			},
			{
				TArray{EPCGDataType::Surface, EPCGDataType::Point, EPCGDataType::Landscape},
				FPCGDataTypeIdentifier{}
			},
			{
				TArray{EPCGDataType::Texture,  EPCGDataType::Texture},
				FPCGDataTypeIdentifier{EPCGDataType::Texture}
			},
			{
				TArray{EPCGDataType::Point,  EPCGDataType::PointOrParam},
				FPCGDataTypeIdentifier{EPCGDataType::Point}
			},
			{
				TArray{EPCGDataType::Param,  EPCGDataType::PointOrParam},
				FPCGDataTypeIdentifier{EPCGDataType::Param}
			},
			{
				TArray{EPCGDataType::Surface,  EPCGDataType::Landscape},
				FPCGDataTypeIdentifier{EPCGDataType::Landscape}
			},
			{
				TArray{EPCGDataType::Surface,  EPCGDataType::Texture},
				FPCGDataTypeIdentifier{EPCGDataType::Texture}
			},
			{
				TArray{EPCGDataType::Any,  EPCGDataType::Spatial},
				FPCGDataTypeIdentifier{EPCGDataType::Spatial}
			},
		}));

		FPCGDataTypeIdentifier TypeIdentifier = Types[0];
		for (const EPCGDataType Type : Types)
		{
			TypeIdentifier &= Type;
		}

		REQUIRE(TypeIdentifier.IsSameType(ExpectedIdentifier));
	}

	SECTION("Composition")
	{
		auto [Types, ExpectedIdentifier] = GENERATE(
		table<TArray<FPCGDataTypeIdentifier>, FPCGDataTypeIdentifier>(
	{
		{
			TArray{FPCGDataTypeIdentifier{EPCGDataType::Point}, FPCGDataTypeIdentifier{EPCGDataType::None}},
			FPCGDataTypeIdentifier{EPCGDataType::Point}
		},
		{
			TArray{FPCGDataTypeIdentifier{EPCGDataType::Any}, FPCGDataTypeIdentifier{EPCGDataType::Spatial}},
			FPCGDataTypeIdentifier{EPCGDataType::Any}
		},
		{
			TArray{FPCGDataTypeIdentifier{EPCGDataType::Spatial}, FPCGDataTypeIdentifier{EPCGDataType::Concrete}},
			FPCGDataTypeIdentifier{EPCGDataType::Spatial}
		},
		{
			TArray{FPCGDataTypeIdentifier{EPCGDataType::Spatial}, (FPCGDataTypeIdentifier{EPCGDataType::Composite})},
			FPCGDataTypeIdentifier{EPCGDataType::Spatial}
		},
		{
			TArray{FPCGDataTypeIdentifier{EPCGDataType::Concrete}, FPCGDataTypeIdentifier{EPCGDataType::Composite}},
			FPCGDataTypeIdentifier{EPCGDataType::Spatial}
		},
		{
			TArray{FPCGDataTypeIdentifier{EPCGDataType::Texture}, FPCGDataTypeIdentifier{EPCGDataType::Composite}},
			FPCGDataTypeIdentifier{EPCGDataType::Texture | EPCGDataType::Composite}
		},
		{
			TArray{FPCGDataTypeIdentifier{EPCGDataType::Concrete}, FPCGDataTypeIdentifier{EPCGDataType::Point}},
			FPCGDataTypeIdentifier{EPCGDataType::Concrete}
		},
		{
			TArray{FPCGDataTypeIdentifier{EPCGDataType::Surface}, FPCGDataTypeIdentifier{EPCGDataType::Texture}},
			FPCGDataTypeIdentifier{EPCGDataType::Surface}
		},
		{
			TArray{FPCGDataTypeIdentifier{EPCGDataType::Spline}, FPCGDataTypeIdentifier{EPCGDataType::LandscapeSpline}, FPCGDataTypeIdentifier{EPCGDataType::Polygon2D}},
			FPCGDataTypeIdentifier{EPCGDataType::PolyLine}
		},
		{
			TArray{FPCGDataTypeIdentifier{EPCGDataType::Spline}, FPCGDataTypeIdentifier{EPCGDataType::LandscapeSpline}, FPCGDataTypeIdentifier{EPCGDataType::Polygon2D}, FPCGDataTypeIdentifier{EPCGDataType::Point} },
			FPCGDataTypeIdentifier::Construct<UPCGPolyLineData, UPCGPointData>()
		},
		{
			TArray{FPCGDataTypeIdentifier{EPCGDataType::Texture}, FPCGDataTypeIdentifier{EPCGDataType::RenderTarget}},
			FPCGDataTypeIdentifier{EPCGDataType::BaseTexture}
		},
		{
			TArray{FPCGDataTypeIdentifier::Construct<UPCGTexture2DArrayData>(), FPCGDataTypeIdentifier{EPCGDataType::Texture}, FPCGDataTypeIdentifier{EPCGDataType::RenderTarget}, FPCGDataTypeIdentifier{EPCGDataType::VirtualTexture}, FPCGDataTypeIdentifier{EPCGDataType::Landscape}},
			FPCGDataTypeIdentifier{EPCGDataType::Surface}
		},
		{
			TArray{FPCGDataTypeIdentifier{EPCGDataType::Point}, FPCGDataTypeIdentifier{EPCGDataType::Param}},
			FPCGDataTypeIdentifier::Construct<UPCGPointData, UPCGParamData>()
		},
		{
			TArray{FPCGDataTypeIdentifier{EPCGDataType::Point}, FPCGDataTypeIdentifier{EPCGDataType::Param}},
			FPCGDataTypeIdentifier::Construct<UPCGParamData, UPCGPointData>()
		},
		{
			TArray{FPCGDataTypeIdentifier{EPCGDataType::Spline}, FPCGDataTypeIdentifier{EPCGDataType::Volume}},
			FPCGDataTypeIdentifier::Construct<UPCGSplineData, UPCGVolumeData>()
		},
		{
			TArray{FPCGDataTypeIdentifier{EPCGDataType::Texture}, FPCGDataTypeIdentifier{EPCGDataType::Point}, FPCGDataTypeIdentifier{EPCGDataType::Landscape}},
			FPCGDataTypeIdentifier::Construct<UPCGTextureData, UPCGPointData, UPCGLandscapeData>()
		},
	}));

		const FPCGDataTypeIdentifier CompositionType = FPCGDataTypeIdentifier::Compose(Types);
		
		const FString TypeName = CompositionType.ToString();
		const FString ExpectedTypeName = ExpectedIdentifier.ToString();

		CAPTURE(*FString::Printf(TEXT("Composition of %s gives %s."), *TypeName, *ExpectedTypeName));
		REQUIRE(CompositionType.IsSameType(ExpectedIdentifier));
	}

	/**
	 * We should never have duplicates in the IDs when we construct/combine/compose the same types.
	 */
	SECTION("Unicity")
	{
		SECTION("Construct with same class in template gives just 1 ID")
		{
			const FPCGDataTypeIdentifier SameTypeWithClassTpl = FPCGDataTypeIdentifier::Construct<UPCGData, UPCGData>();
			REQUIRE_EQUAL(SameTypeWithClassTpl.GetIds().Num(), 1);
		}

		SECTION("Construct with child classes in template gives just 1 ID")
		{
			const FPCGDataTypeIdentifier ChildTypesWithClassTpl = FPCGDataTypeIdentifier::Construct<UPCGData, UPCGSpatialData, UPCGPointData>();
			REQUIRE_EQUAL(ChildTypesWithClassTpl.GetIds().Num(), 1);
		}

		SECTION("Construct with child classes in template gives the wider one")
		{
			const FPCGDataTypeIdentifier ChildTypesWithClassTpl = FPCGDataTypeIdentifier::Construct<UPCGData, UPCGSpatialData, UPCGPointData>();
			REQUIRE_EQUAL(ChildTypesWithClassTpl.GetId(), FPCGDataTypeInfo::AsId());
		}

		SECTION("Construct with same class gives just 1 ID")
		{
			const FPCGDataTypeIdentifier SameTypeWithClass = FPCGDataTypeIdentifier::Construct(UPCGPointData::StaticClass(), UPCGPointData::StaticClass());
			REQUIRE_EQUAL(SameTypeWithClass.GetIds().Num(), 1);
		}

		SECTION("Construct with child base ids gives just 1 ID")
		{
			const FPCGDataTypeIdentifier ChildTypeWithBaseIds = FPCGDataTypeIdentifier::Construct(FPCGDataTypeInfoLandscape::AsId(), FPCGDataTypeInfoSurface::AsId(), FPCGDataTypeInfoLandscape::AsId());
			REQUIRE_EQUAL(ChildTypeWithBaseIds.GetIds().Num(), 1);
		}

		SECTION("Construct with child base ids gives the wider one")
		{
			const FPCGDataTypeIdentifier ChildTypeWithBaseIds = FPCGDataTypeIdentifier::Construct(FPCGDataTypeInfoLandscape::AsId(), FPCGDataTypeInfoSurface::AsId(), FPCGDataTypeInfoLandscape::AsId());
			REQUIRE_EQUAL(ChildTypeWithBaseIds.GetId(), FPCGDataTypeInfoSurface::AsId());
		}

		SECTION("Construct with same ids gives just 1 ID")
		{
			const FPCGDataTypeIdentifier SameTypeWithClassTpl = FPCGDataTypeIdentifier::Construct<UPCGData, UPCGData>();
			const FPCGDataTypeIdentifier SameTypeWithIds = FPCGDataTypeIdentifier::Construct(SameTypeWithClassTpl, SameTypeWithClassTpl, SameTypeWithClassTpl);
			REQUIRE_EQUAL(SameTypeWithIds.GetIds().Num(), 1);
		}

		SECTION("Bitwise Or with same default type gives just 1 ID")
		{
			const FPCGDataTypeIdentifier SameTypeWithClass = FPCGDataTypeIdentifier::Construct(UPCGPointData::StaticClass(), UPCGPointData::StaticClass());
			const FPCGDataTypeIdentifier SameTypeWithDefaultTypeOr = SameTypeWithClass | EPCGDataType::Point;
			REQUIRE_EQUAL(SameTypeWithDefaultTypeOr.GetIds().Num(), 1);
		}

		SECTION("Bitwise Or with same ids gives just 1 ID")
		{
			const FPCGDataTypeIdentifier SameTypeWithClass = FPCGDataTypeIdentifier::Construct(UPCGPointData::StaticClass(), UPCGPointData::StaticClass());
			const FPCGDataTypeIdentifier SameTypeWithOr = SameTypeWithClass | SameTypeWithClass;
			REQUIRE_EQUAL(SameTypeWithOr.GetIds().Num(), 1);
		}

		SECTION("Self Bitwise Or")
		{
			const FPCGDataTypeIdentifier SameTypeWithClass = FPCGDataTypeIdentifier::Construct(UPCGPointData::StaticClass(), UPCGPointData::StaticClass());
			
			FPCGDataTypeIdentifier SameTypeOr = SameTypeWithClass;
			SameTypeOr |= SameTypeWithClass;
			CAPTURE("Bitwise Or with self gives just 1 ID");
			REQUIRE_EQUAL(SameTypeOr.GetIds().Num(), 1);
	
			SameTypeOr |= EPCGDataType::Point;
			CAPTURE("Bitwise Or on self with default type gives just 1 ID");
			REQUIRE_EQUAL(SameTypeOr.GetIds().Num(), 1);
		}

		SECTION("Compose with same type gives just 1 ID")
		{
			const FPCGDataTypeIdentifier SameTypeWithClass = FPCGDataTypeIdentifier::Construct(UPCGPointData::StaticClass(), UPCGPointData::StaticClass());
			const FPCGDataTypeIdentifier SameTypeCompose = SameTypeWithClass.Compose({SameTypeWithClass, SameTypeWithClass});
			REQUIRE_EQUAL(SameTypeCompose.GetIds().Num(), 1);
		}
	}

	SECTION("Wider")
	{
		// Test: Left is wider than right. Result in the bool
		auto [LeftIdentifier, RightIdentifier, bIsWider] = GENERATE(
			table<FPCGDataTypeIdentifier, FPCGDataTypeIdentifier, bool>(
			{
				{EPCGDataType::Any, EPCGDataType::Concrete, true},
				{EPCGDataType::Concrete, EPCGDataType::Any, false},
				{EPCGDataType::PointOrParam, EPCGDataType::Point, true},
				{EPCGDataType::PointOrParam, EPCGDataType::Param, true},
				{EPCGDataType::Param, EPCGDataType::PointOrParam, false},
				{EPCGDataType::Point, EPCGDataType::PointOrParam, false},
				{EPCGDataType::PolyLine, EPCGDataType::Spline, true},
				{EPCGDataType::Spline, EPCGDataType::PolyLine, false},
				{EPCGDataType::Spline, EPCGDataType::LandscapeSpline, true},
				{EPCGDataType::LandscapeSpline, EPCGDataType::Spline, true},
				{EPCGDataType::Spline | EPCGDataType::Point, EPCGDataType::Point | EPCGDataType::LandscapeSpline, true}
			}));

		REQUIRE_EQUAL(LeftIdentifier.IsWider(RightIdentifier), bIsWider);
	}

	SECTION("Union")
	{
		auto [Identifiers, ExpectedIdentifier] = GENERATE(
			table<FPCGDataTypeIdentifier, FPCGDataTypeIdentifier>(
			{
				{FPCGDataTypeIdentifier{EPCGDataType::Point} | EPCGDataType::None, EPCGDataType::Point},
				{FPCGDataTypeIdentifier{EPCGDataType::Any} | EPCGDataType::Concrete, EPCGDataType::Any},
				{FPCGDataTypeIdentifier{EPCGDataType::Param} | EPCGDataType::Spatial, EPCGDataType::Any},
				{FPCGDataTypeIdentifier{EPCGDataType::Param} | EPCGDataType::Point, EPCGDataType::Any},
				{FPCGDataTypeIdentifier{EPCGDataType::Texture} | EPCGDataType::RenderTarget, EPCGDataType::BaseTexture},
				{FPCGDataTypeIdentifier{EPCGDataType::Landscape} | EPCGDataType::RenderTarget, EPCGDataType::Surface},
				{FPCGDataTypeIdentifier{EPCGDataType::PointOrParam}, EPCGDataType::Any},
				{FPCGDataTypeIdentifier{EPCGDataType::Point} | EPCGDataType::Landscape | EPCGDataType::Polygon2D, EPCGDataType::Concrete},
				{FPCGDataTypeIdentifier{EPCGDataType::Point} | EPCGDataType::Landscape | EPCGDataType::Polygon2D | EPCGDataType::Composite, EPCGDataType::Spatial},
			}));

		const FPCGDataTypeIdentifier Union = Registry.GetIdentifiersUnion({Identifiers});
		REQUIRE(Union.IsSameType(ExpectedIdentifier));
	}

	SECTION("IsChildOf")
	{
		auto [LHS, RHS, bIsChildOf] = GENERATE(
			table<FPCGDataTypeIdentifier, FPCGDataTypeIdentifier, bool>(
			{
				{FPCGDataTypeInfo::AsId(), FPCGDataTypeInfo::AsId(), true},
				{FPCGDataTypeInfoPoint::AsId(), FPCGDataTypeInfo::AsId(), true},
				{FPCGDataTypeInfo::AsId(), FPCGDataTypeInfoPoint::AsId(), false},
				{FPCGDataTypeInfoSpline::AsId(), FPCGDataTypeInfoPolyline::AsId(), true},
				{FPCGDataTypeInfoSpline::AsId(), FPCGDataTypeInfoLandscapeSpline::AsId(), false},
				{FPCGDataTypeIdentifier::Construct(FPCGDataTypeInfoSpline::AsId(), FPCGDataTypeInfoLandscapeSpline::AsId()), FPCGDataTypeInfoPolyline::AsId(), true},
				{FPCGDataTypeIdentifier::Construct(FPCGDataTypeInfoParam::AsId(), FPCGDataTypeInfoPoint::AsId()), FPCGDataTypeIdentifier::Construct(FPCGDataTypeInfoParam::AsId(), FPCGDataTypeInfoSpatial::AsId()), true},
			}));

		REQUIRE_EQUAL(LHS.IsChildOf(RHS), bIsChildOf);
	}
}