// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "TestHarness.h"

#include <catch2/generators/catch_generators.hpp>

#include "PCGContext.h"
#include "Data/PCGPointData.h"
#include "Data/PCGSplineData.h"
#include "Elements/Grammar/PCGSelectGrammar.h"
#include "Tests/PCGTestsCommon.h"
#include "PCGTestsCommon.h"


class FPCGSelectGrammarBaseTest: public PCGTests::FPCGBaseTest
{
public:
	FPCGSelectGrammarBaseTest()
		: PCGTests::FPCGBaseTest()
	{
		PCGTestsCommon::GenerateSettings<UPCGSelectGrammarSettings>(TestData);
		Settings = CastChecked<UPCGSelectGrammarSettings>(TestData.Settings);
		Element = TestData.Settings->GetElement();
	}

	template <typename PCGDataType> requires std::is_base_of_v<UPCGData, PCGDataType>
	TArray<TTuple<const PCGDataType*, const FPCGMetadataAttribute<FString>*>> Execute(const FPCGAttributeIdentifier& GrammarAttributeId)
	{
		Context = TestData.InitializeTestContext();

		while (!Element->Execute(Context.Get()))
		{
		}

		const TArray<FPCGTaggedData>& Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
		const TArray<FPCGTaggedData>& Outputs = Context->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);

		CHECK_EQUAL(Inputs.Num(), Outputs.Num());

		TArray<TTuple<const PCGDataType*, const FPCGMetadataAttribute<FString>*>> Result;
		for (const FPCGTaggedData& OutputData : Outputs)
		{
			const PCGDataType* Data = Cast<const PCGDataType>(OutputData.Data);
			CHECK_NOT_EQUAL(Data, nullptr);
			const FPCGMetadataAttribute<FString>* Attribute = Data->ConstMetadata()->template GetConstTypedAttribute<FString>(GrammarAttributeId);
			CHECK_NOT_EQUAL(Attribute, nullptr);
			Result.Emplace(Data, Attribute);
		}

		return Result;
	};

	void EmplaceData(const UPCGData* Data, const FName PinLabel, bool bResetData)
	{
		if (bResetData)
		{
			TestData.Reset(Settings);
		}

		TestData.InputData.TaggedData.Emplace_GetRef(Data).Pin = PinLabel;
	}

	PCGTestsCommon::FTestData TestData;
	TUniquePtr<FPCGContext> Context;
	UPCGSelectGrammarSettings* Settings;
	FPCGElementPtr Element;
};

TEST_CASE_METHOD(FPCGSelectGrammarBaseTest, "PCG::SelectGrammar", "[PCG][SelectGrammar]")
{
	const FName GrammarAttributeName = "Grammar";
	const FName KeyAttributeName = "Key";
	const FName DoubleAttributeName = "Double";
	const FName Keys[2] = {"KeyA", "KeyB"};
	const FString Grammars[3] = {TEXT("[A,B]"), TEXT("[A]*"), TEXT("{A,B}*")};

	Settings->bCriteriaAsInput = false;
	Settings->Criteria.Add(FPCGSelectGrammarCriterion{Keys[0], EPCGSelectGrammarComparator::LessThan, 250.0, 0.0, Grammars[0]});
	Settings->Criteria.Add(FPCGSelectGrammarCriterion{Keys[0], EPCGSelectGrammarComparator::LessThan, 450.0, 0.0, Grammars[1]});
	Settings->Criteria.Add(FPCGSelectGrammarCriterion{Keys[0], EPCGSelectGrammarComparator::Select, 0.0, 0.0, Grammars[2]});
	Settings->Criteria.Add(FPCGSelectGrammarCriterion{Keys[1], EPCGSelectGrammarComparator::LessThan, 150.0, 0.0, Grammars[1]});
	Settings->Criteria.Add(FPCGSelectGrammarCriterion{Keys[1], EPCGSelectGrammarComparator::Select, 0.0, 0.0, Grammars[0]});
	
	SECTION("PointData")
	{
		constexpr int32 NumPoints = 5;
		
		UPCGBasePointData* InputData = FPCGContext::NewPointData_AnyThread(nullptr);
		InputData->AllocateProperties(EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::MetadataEntry | EPCGPointNativeProperties::Density);
		InputData->SetNumPoints(NumPoints);
		
		InputData->SetBoundsMin(FVector(-50, -50, -50));
		InputData->SetBoundsMax(FVector(50, 50, 50));
		TPCGValueRange<FTransform> TransformsRange = InputData->GetTransformValueRange();
		TPCGValueRange<float> DensityRange = InputData->GetDensityValueRange();
		TPCGValueRange<PCGMetadataEntryKey> MetadataEntryRange = InputData->GetMetadataEntryValueRange();

		FPCGMetadataAttribute<FName>* KeyAttribute = InputData->MutableMetadata()->CreateAttribute<FName>(KeyAttributeName, NAME_None, false, true);
		FPCGMetadataAttribute<FName>* KeyAttributeData = InputData->MutableMetadata()->CreateAttribute<FName>(FPCGAttributeIdentifier(KeyAttributeName, PCGMetadataDomainID::Data), Keys[0], false, true);
		FPCGMetadataAttribute<double>* DoubleAttributeData = InputData->MutableMetadata()->CreateAttribute<double>(FPCGAttributeIdentifier(DoubleAttributeName, PCGMetadataDomainID::Data), 150.0, false, true);

		for (int32 i = 0; i < NumPoints; ++i)
		{
			TransformsRange[i].SetScale3D(FVector::OneVector * (i + 1));
			DensityRange[i] = i;
			MetadataEntryRange[i] = InputData->MutableMetadata()->AddEntry();
			KeyAttribute->SetValue(MetadataEntryRange[i], Keys[i % std::size(Keys)]);
		}

		EmplaceData(InputData, PCGPinConstants::DefaultInputLabel, /*bResetData=*/true);
		
		SECTION("Hardcoded Key: KeyA ; Comparator: $ScaledLocalSize.X")
		{
			Settings->bKeyAsAttribute = false;
			Settings->Key = Keys[0];
			Settings->ComparedValueAttribute.Update(TEXT("$ScaledLocalSize.X"));
			Settings->OutputGrammarAttribute.SetAttributeName(GrammarAttributeName);

			TArray<const FString*> ExpectedGrammars = {&Grammars[0], &Grammars[0], &Grammars[1], &Grammars[1], &Grammars[2]};

			TArray<TTuple<const UPCGBasePointData*, const FPCGMetadataAttribute<FString>*>> DataAndAttributes = Execute<UPCGBasePointData>(FPCGAttributeIdentifier{GrammarAttributeName});
			CHECK_EQUAL(DataAndAttributes.Num(), 1);
			const UPCGBasePointData* OutputData = DataAndAttributes[0].Get<0>();
			const FPCGMetadataAttribute<FString>* Attribute = DataAndAttributes[0].Get<1>();
			CHECK(OutputData);
			CHECK(Attribute);

			for (int32 i = 0; i < NumPoints; ++i)
			{
				const FString ActualGrammar = Attribute->GetValueFromItemKey(OutputData->GetMetadataEntry(i));
				REQUIRE_EQUAL(ActualGrammar, *ExpectedGrammars[i]);
			}
		}

		SECTION("Attribute Key ; Comparator: $ScaledLocalSize.X")
		{
			Settings->bKeyAsAttribute = true;
			Settings->KeyAttribute.SetAttributeName(KeyAttributeName); 
			Settings->ComparedValueAttribute.Update(TEXT("$ScaledLocalSize.X"));
			Settings->OutputGrammarAttribute.SetAttributeName(GrammarAttributeName);

			TArray<const FString*> ExpectedGrammars = {&Grammars[0], &Grammars[0], &Grammars[1], &Grammars[0], &Grammars[2]};

			TArray<TTuple<const UPCGBasePointData*, const FPCGMetadataAttribute<FString>*>> DataAndAttributes = Execute<UPCGBasePointData>(FPCGAttributeIdentifier{GrammarAttributeName});
			CHECK_EQUAL(DataAndAttributes.Num(), 1);
			const UPCGBasePointData* OutputData = DataAndAttributes[0].Get<0>();
			const FPCGMetadataAttribute<FString>* Attribute = DataAndAttributes[0].Get<1>();
			CHECK(OutputData);
			CHECK(Attribute);

			for (int32 i = 0; i < NumPoints; ++i)
			{
				const FString ActualGrammar = Attribute->GetValueFromItemKey(OutputData->GetMetadataEntry(i));
				REQUIRE_EQUAL(ActualGrammar, *ExpectedGrammars[i]);
			}
		}

		SECTION("Attribute Key ; Comparator: $Density")
		{
			Settings->bKeyAsAttribute = true;
			Settings->KeyAttribute.SetAttributeName(KeyAttributeName); 
			Settings->ComparedValueAttribute.SetPointProperty(EPCGPointProperties::Density);
			Settings->OutputGrammarAttribute.SetAttributeName(GrammarAttributeName);

			TArray<const FString*> ExpectedGrammars = {&Grammars[0], &Grammars[1], &Grammars[0], &Grammars[1], &Grammars[0]};

			TArray<TTuple<const UPCGBasePointData*, const FPCGMetadataAttribute<FString>*>> DataAndAttributes = Execute<UPCGBasePointData>(FPCGAttributeIdentifier{GrammarAttributeName});
			CHECK_EQUAL(DataAndAttributes.Num(), 1);
			const UPCGBasePointData* OutputData = DataAndAttributes[0].Get<0>();
			const FPCGMetadataAttribute<FString>* Attribute = DataAndAttributes[0].Get<1>();
			CHECK(OutputData);
			CHECK(Attribute);

			for (int32 i = 0; i < NumPoints; ++i)
			{
				const FString ActualGrammar = Attribute->GetValueFromItemKey(OutputData->GetMetadataEntry(i));
				REQUIRE_EQUAL(ActualGrammar, *ExpectedGrammars[i]);
			}
		}

		SECTION("Attribute Key ; Comparator: @Data.Double ; Output Grammar: @Data.Grammar")
		{
			Settings->bKeyAsAttribute = true;
			Settings->KeyAttribute = FPCGAttributePropertyInputSelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(KeyAttributeName, TEXT("Data"));
			Settings->ComparedValueAttribute = FPCGAttributePropertyInputSelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(DoubleAttributeName, TEXT("Data"));
			Settings->OutputGrammarAttribute = FPCGAttributePropertyOutputSelector::CreateAttributeSelector<FPCGAttributePropertyOutputSelector>(GrammarAttributeName, TEXT("Data"));

			TArray<TTuple<const UPCGBasePointData*, const FPCGMetadataAttribute<FString>*>> DataAndAttributes = Execute<UPCGBasePointData>(FPCGAttributeIdentifier{GrammarAttributeName, PCGMetadataDomainID::Data});
			CHECK_EQUAL(DataAndAttributes.Num(), 1);
			const FPCGMetadataAttribute<FString>* Attribute = DataAndAttributes[0].Get<1>();
			CHECK(Attribute);

			REQUIRE(Attribute->GetMetadataDomain()->GetDomainID() == PCGMetadataDomainID::Data);
			REQUIRE_EQUAL(Attribute->GetValueFromItemKey(PCGFirstEntryKey), Grammars[0]);
		}
	}
	
	SECTION("SplineData")
	{
		constexpr int32 NumPoints = 5;

		TArray<FSplinePoint> SplinePoints1;
		TArray<FSplinePoint> SplinePoints2;
		for (int32 i = 0; i < NumPoints; ++i)
		{
			SplinePoints1.Emplace(i, FVector::OneVector * i * 100);
			SplinePoints2.Emplace(i, FVector::OneVector * i * -50);
		}

		auto PrepareData = [this, KeyAttributeName, DoubleAttributeName, &Keys](const TArray<FSplinePoint>& SplinePoints, int32 Index)
		{
			UPCGSplineData* InputData = NewObject<UPCGSplineData>();
			InputData->Initialize(SplinePoints, /*bClosedLoop=*/false, FTransform::Identity);

			InputData->MutableMetadata()->CreateAttribute<FName>(FPCGAttributeIdentifier(KeyAttributeName, PCGMetadataDomainID::Data), Keys[Index], false, true);
			InputData->MutableMetadata()->CreateAttribute<double>(FPCGAttributeIdentifier(DoubleAttributeName, PCGMetadataDomainID::Data), InputData->GetLength(), false, true);

			EmplaceData(InputData, PCGPinConstants::DefaultInputLabel, /*bResetData=*/Index==0);
		};

		TestData.InputData.TaggedData.Reset();
		PrepareData(SplinePoints1, 0);
		PrepareData(SplinePoints2, 1);

		SECTION("Attribute Key ; Comparator: @Data.Double ; Output Grammar: @Data.Grammar")
		{
			Settings->bKeyAsAttribute = true;
			Settings->KeyAttribute = FPCGAttributePropertyInputSelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(KeyAttributeName, TEXT("Data"));
			Settings->ComparedValueAttribute = FPCGAttributePropertyInputSelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(DoubleAttributeName, TEXT("Data"));
			Settings->OutputGrammarAttribute = FPCGAttributePropertyOutputSelector::CreateAttributeSelector<FPCGAttributePropertyOutputSelector>(GrammarAttributeName, TEXT("Data"));

			TArray<TTuple<const UPCGSplineData*, const FPCGMetadataAttribute<FString>*>> DataAndAttributes = Execute<UPCGSplineData>(FPCGAttributeIdentifier{GrammarAttributeName, PCGMetadataDomainID::Data});
			CHECK_EQUAL(DataAndAttributes.Num(), 2);
			const FPCGMetadataAttribute<FString>* Attribute1 = DataAndAttributes[0].Get<1>();
			const FPCGMetadataAttribute<FString>* Attribute2 = DataAndAttributes[1].Get<1>();
			CHECK(Attribute1);
			CHECK(Attribute2);

			REQUIRE(Attribute1->GetMetadataDomain()->GetDomainID() == PCGMetadataDomainID::Data);
			REQUIRE_EQUAL(Attribute1->GetValueFromItemKey(PCGFirstEntryKey), Grammars[2]);

			REQUIRE(Attribute2->GetMetadataDomain()->GetDomainID() == PCGMetadataDomainID::Data);
			REQUIRE_EQUAL(Attribute2->GetValueFromItemKey(PCGFirstEntryKey), Grammars[0]);
		}
	}
}
