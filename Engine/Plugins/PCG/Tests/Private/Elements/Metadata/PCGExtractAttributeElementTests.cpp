// Copyright Epic Games, Inc. All Rights Reserved.

// =============================================================================
// PCG Extract Attribute Element Tests
//
// Tests for UPCGExtractAttributeSettings (PCGExtractAttribute.cpp), the node
// that pulls a single value out of an attribute at a given entry index and
// writes it as a single-entry UPCGParamData.
//
// Test summary:
//
// Normal path & edge cases:
//  - AtIndex::Basic::FirstEntry:                  Index=0 of 3 entries, double attribute
//  - AtIndex::Basic::MiddleEntry:                 Index=1 of 3 entries
//  - AtIndex::Basic::LastEntry:                   Index=2 of 3 entries
//  - AtIndex::Basic::PropertyToSource:            Index=0 of 3 entries, target $Transform on points, write into @Source
//  - AtIndex::Basic::CrossDomain:                 Write from/to data domain
//  - AtIndex::OOB::Negative:                      Index=-1 → error, no output
//  - AtIndex::OOB::TooLarge:                      Index=N → error, no output
//  - AtIndex::OOB::EmptyInput:                    0 entries, Index=0 → error
//  - AtIndex::RegressionTest::DiscardExtraNames:  Output attribute should discard any extra names
//  - AtIndex::RegressionTest::DiscardDataDomain:  Output attribute should write into the element domain.
//
// Cross-type axis (one mid index per element type):
//  - AtIndex::CrossType::Int32:         int32 attribute extraction
//  - AtIndex::CrossType::Double:        double attribute extraction
//  - AtIndex::CrossType::FVector:       FVector attribute extraction
//  - AtIndex::CrossType::FString:       FString attribute extraction
//
// Input data type axis:
//  - AtIndex::Input::ParamData:         UPCGParamData input → param data output
//  - AtIndex::Input::PointData:         UPCGBasePointData input → param data
//                                        output (single entry)
//
// Output shape:
//  - AtIndex::Output::IsParamData:      Output is always a UPCGParamData with
//                                        exactly one entry
// =============================================================================

#include "CoreMinimal.h"
#include "TestHarness.h"

#include <catch2/generators/catch_generators.hpp>

#include "PCGCommon.h"
#include "PCGContext.h"
#include "PCGData.h"
#include "PCGParamData.h"
#include "PCGTestsCommon.h"
#include "Data/PCGBasePointData.h"
#include "Elements/Metadata/PCGExtractAttribute.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/PCGMetadataAttribute.h"
#include "Metadata/PCGMetadataAttributeTpl.h"
#include "Metadata/PCGMetadataAttributeTraits.h"
#include "Metadata/PCGMetadataDomain.h"

class FPCGExtractAttributeTest : public PCGTests::FPCGSingleElementBaseTest<UPCGExtractAttributeSettings>
{
public:
	inline static const FName SourceAttrName = TEXT("Source");
	inline static const FName OutputAttrName = TEXT("OutResult");

	// ------------------------------------------------------------------
	// Input factories
	// ------------------------------------------------------------------

	// Build a UPCGParamData with one attribute of type T, filled with one entry per
	// value. Works for both basic types (int32, double, FVector, FString, …) and
	// non-basic structs — CreateAttribute<T> picks the right overload and the typed
	// return upcasts cleanly to FPCGMetadataAttributeBase*.
	template <typename T>
	UPCGParamData* MakeParam(const TArray<T>& PerEntry, FName InAttrName = SourceAttrName, FPCGMetadataDomainID DomainID = PCGMetadataDomainID::Default)
	{
		UPCGParamData* ParamData = NewObject<UPCGParamData>();
		check(ParamData && ParamData->Metadata);

		FPCGMetadataDomain* Domain = ParamData->MutableMetadata()->GetMetadataDomain(DomainID);
		REQUIRE(Domain);
		REQUIRE((Domain->SupportsMultiEntries() || PerEntry.Num() <= 1));

		FPCGMetadataAttributeBase* Attr = Domain->CreateAttribute<T>(
			InAttrName, T{}, /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false);
		check(Attr);

		for (const T& Value : PerEntry)
		{
			const PCGMetadataEntryKey Key = Domain->AddEntry();
			Attr->SetValue<T>(Key, Value);
		}

		return ParamData;
	}

	// Build a UPCGBasePointData with one attribute of type T, one entry per point.
	// Same FPCGMetadataAttributeBase* + templated SetValue<T> trick as MakeParam.
	template <typename T>
	UPCGBasePointData* MakePointData(const TArray<T>& PerPoint, FName InAttrName = SourceAttrName)
	{
		UPCGBasePointData* PointData = CreatePointData();
		PointData->SetNumPoints(PerPoint.Num());
		PointData->AllocateProperties(EPCGPointNativeProperties::MetadataEntry);

		FPCGMetadataAttributeBase* Attr = PointData->MutableMetadata()->CreateAttribute<T>(
			InAttrName, T{}, /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false);
		check(Attr);

		TPCGValueRange<PCGMetadataEntryKey> EntryRange = PointData->GetMetadataEntryValueRange();
		for (int32 i = 0; i < PerPoint.Num(); ++i)
		{
			PointData->MutableMetadata()->InitializeOnSet(EntryRange[i]);
			Attr->SetValue<T>(EntryRange[i], PerPoint[i]);
		}
		return PointData;
	}

	// ------------------------------------------------------------------
	// Configuration / execution
	// ------------------------------------------------------------------

	void ConfigureAtIndex(int32 InIndex, const FString& InAttrName = SourceAttrName.ToString(), const FString& InOutputName = OutputAttrName.ToString())
	{
		TypedSettings->Index = InIndex;
		TypedSettings->InputSource.Update(InAttrName);
		TypedSettings->OutputAttributeName.Update(InOutputName);
	}

	void AddInput(UPCGData* Data)
	{
		FPCGTaggedData& Tagged = InputData.TaggedData.Emplace_GetRef();
		Tagged.Data = Data;
		Tagged.Pin = PCGPinConstants::DefaultInputLabel;
	}

	// ------------------------------------------------------------------
	// Output assertions
	// ------------------------------------------------------------------

	const UPCGData* RequireSingleOutput()
	{
		REQUIRE(Context->OutputData.TaggedData.Num() == 1);
		const UPCGData* OutData = Context->OutputData.TaggedData[0].Data;
		REQUIRE(OutData);
		return OutData;
	}

	void ExpectNoOutput()
	{
		CHECK(Context->OutputData.TaggedData.Num() == 0);
	}

	// Read a basic-type single-attribute value at the given metadata entry key.
	template <typename T>
	T ReadAttr(const UPCGData* OutData, FName AttrName, PCGMetadataEntryKey EntryKey, FPCGMetadataDomainID DomainID = PCGMetadataDomainID::Default)
	{
		const UPCGMetadata* OutMetadata = OutData->ConstMetadata();
		REQUIRE(OutMetadata);
		const FPCGMetadataDomain* OutDomain = OutMetadata->GetConstMetadataDomain(DomainID);
		REQUIRE(OutDomain);
		const FPCGMetadataAttributeBase* Attr = OutDomain->GetConstAttribute(AttrName);
		REQUIRE(Attr);
		return Attr->GetValueFromItemKey<T>(EntryKey);
	}
};

// =============================================================================
// Normal path & edge cases
// =============================================================================

TEST_CASE_METHOD(FPCGExtractAttributeTest, "PCG::ExtractAttribute::AtIndex::Basic", "[PCG][ExtractAttribute][AtIndex]")
{
	// Same source data, sweep across first / middle / last indexes.
	const TArray<double> Source = {10.5, 20.5, 30.5};

	auto [Index, Expected] = GENERATE(
		table<int32, double>({
			{0, 10.5},
			{1, 20.5},
			{2, 30.5}
		}));

	DYNAMIC_SECTION("Index " << Index)
	{
		ConfigureAtIndex(Index);
		AddInput(MakeParam<double>(Source));

		ExecuteElement();

		CHECK(NumErrors == 0);

		const UPCGData* OutData = RequireSingleOutput();
		const UPCGParamData* OutParam = Cast<const UPCGParamData>(OutData);
		REQUIRE(OutParam);
		CHECK(OutParam->Metadata->GetLocalItemCount() == 1);

		CHECK(ReadAttr<double>(OutData, OutputAttrName, PCGFirstEntryKey) == Expected);
	}
}

TEST_CASE_METHOD(FPCGExtractAttributeTest, "PCG::ExtractAttribute::AtIndex::Basic::PropertyToSource", "[PCG][ExtractAttribute][AtIndex]")
{
	UPCGBasePointData* PointData = CreatePointData();
	PointData->SetNumPoints(3);
	const FTransform Transform = FTransform{ FRotator::MakeFromEuler(FVector{-10, 20, 30}), FVector{1, 2, 3}, FVector{0.1, 0.2, 0.3} };
	PointData->SetTransform(Transform);

	TypedSettings->Index = 0;
	TypedSettings->InputSource = FPCGAttributePropertySelector::CreatePointPropertySelector<FPCGAttributePropertyInputSelector>(EPCGPointProperties::Transform);

	const FName OutputAttributeName = "Transform";

	AddInput(PointData);
	ExecuteElement();

	CHECK(NumErrors == 0);

	const UPCGData* OutData = RequireSingleOutput();
	const UPCGParamData* OutParam = Cast<const UPCGParamData>(OutData);
	REQUIRE(OutParam);
	CHECK(OutParam->Metadata->GetLocalItemCount() == 1);
	CHECK(ReadAttr<FTransform>(OutData, OutputAttributeName, PCGFirstEntryKey).Equals(Transform));
}

TEST_CASE_METHOD(FPCGExtractAttributeTest, "PCG::ExtractAttribute::AtIndex::CrossDomain", "[PCG][ExtractAttribute][AtIndex]")
{
	// Same source data, sweep across first / middle / last indexes.
	const TArray<double> Source = { 10.5 };

	auto [Index, Expected, InputSource, InputAttributeName, InputDomain, OutputTarget, OutputAttributeName, OutputDomain] = GENERATE(
		table<int32, double, FString, FName, FPCGMetadataDomainID, FString, FName, FPCGMetadataDomainID>({
			{0, 10.5, TEXT("@Data.Source"), "Source", PCGMetadataDomainID::Data, TEXT("@Source"), "Source", PCGMetadataDomainID::Data},
			{0, 10.5, TEXT("@Data.Source"), "Source", PCGMetadataDomainID::Data, TEXT("Target"), "Target", PCGMetadataDomainID::Elements},
			{0, 10.5, TEXT("Source"), "Source", PCGMetadataDomainID::Elements, TEXT("@Data.Target"), "Target", PCGMetadataDomainID::Data}
			}));

	DYNAMIC_SECTION("" << InputSource << "->" << OutputTarget)
	{
		ConfigureAtIndex(Index, InputSource, OutputTarget);
		AddInput(MakeParam<double>(Source, InputAttributeName, InputDomain));

		ExecuteElement();

		CHECK(NumErrors == 0);

		const UPCGData* OutData = RequireSingleOutput();
		const UPCGParamData* OutParam = Cast<const UPCGParamData>(OutData);
		REQUIRE(OutParam);
		const FPCGMetadataDomain* Domain = OutParam->ConstMetadata()->GetConstMetadataDomain(OutputDomain);
		REQUIRE(Domain);
		CHECK(Domain->GetLocalItemCount() == 1);

		CHECK(ReadAttr<double>(OutData, OutputAttributeName, PCGFirstEntryKey, OutputDomain) == Expected);
	}
}

TEST_CASE_METHOD(FPCGExtractAttributeTest, "PCG::ExtractAttribute::AtIndex::OOB", "[PCG][ExtractAttribute][AtIndex]")
{
	auto [Source, Index] = GENERATE(
		table<TArray<double>, int32>({
			{TArray<double>{1.0, 2.0, 3.0}, -1},   // Negative
			{TArray<double>{1.0, 2.0, 3.0},  3},   // OOB+
			{TArray<double>{1.0, 2.0, 3.0}, 99}    // OOB+ far
		}));

	DYNAMIC_SECTION("Index " << Index << " into " << Source.Num() << " entries")
	{
		ConfigureAtIndex(Index);
		AddInput(MakeParam<double>(Source));

		FSuppressErrorsScope Suppress(*this);
		ExecuteElement();

		CHECK(NumErrors >= 1);
		ExpectNoOutput();
	}
}

TEST_CASE_METHOD(FPCGExtractAttributeTest, "PCG::ExtractAttribute::AtIndex::OOB::EmptyInput", "[PCG][ExtractAttribute][AtIndex]")
{
	ConfigureAtIndex(0);
	AddInput(MakeParam<double>({}));

	FSuppressErrorsScope Suppress(*this);
	ExecuteElement();

	CHECK(NumErrors >= 1);
	ExpectNoOutput();
}

TEST_CASE_METHOD(FPCGExtractAttributeTest, "PCG::ExtractAttribute::AtIndex::RegressionTest::DiscardExtraNames", "[PCG][ExtractAttribute][AtIndex]")
{
	// Same source data, sweep across first / middle / last indexes.
	const TArray<double> Source = { 10.5, 20.5, 30.5 };

	auto [Index, Expected, OutputTarget, ExpectedOutputName] = GENERATE(
		table<int32, double, FString, FName>({
			{0, 10.5, TEXT("@Source."), SourceAttrName},
			{1, 20.5, TEXT("Bla."), "Bla"},
			{2, 30.5, TEXT("Blou.Bla"), "Blou"}
			}));

	DYNAMIC_SECTION("Index " << Index)
	{
		TypedSettings->Index = Index;
		TypedSettings->InputSource = FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(SourceAttrName);
		TypedSettings->OutputAttributeName.Update(OutputTarget);

		AddInput(MakeParam<double>(Source));

		ExecuteElement();

		CHECK(NumErrors == 0);

		const UPCGData* OutData = RequireSingleOutput();
		const UPCGParamData* OutParam = Cast<const UPCGParamData>(OutData);
		REQUIRE(OutParam);
		CHECK(OutParam->Metadata->GetLocalItemCount() == 1);

		CHECK(ReadAttr<double>(OutData, ExpectedOutputName, PCGFirstEntryKey) == Expected);
	}
}

TEST_CASE_METHOD(FPCGExtractAttributeTest, "PCG::ExtractAttribute::AtIndex::RegressionTest::DiscardDataDomain", "[PCG][ExtractAttribute][AtIndex]")
{
	// Same source data, sweep across first / middle / last indexes.
	const TArray<double> Source = { 10.5, 20.5, 30.5 };

	auto [Index, Expected, OutputTarget, ExpectedOutputName] = GENERATE(
		table<int32, double, FString, FName>({
			{0, 10.5, TEXT("@Data.Blou"), "Blou"},
			{1, 20.5, TEXT("@Data.Bla"), "Bla"}
			}));

	DYNAMIC_SECTION("Index " << Index)
	{
		TypedSettings->Index = Index;
		TypedSettings->InputSource = FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(SourceAttrName);
		TypedSettings->OutputAttributeName.Update(OutputTarget);
		TypedSettings->bForceOutputAttributeToBeInElementsDomain = true;

		AddInput(MakeParam<double>(Source));

		ExecuteElement();

		CHECK(NumErrors == 0);

		const UPCGData* OutData = RequireSingleOutput();
		const UPCGParamData* OutParam = Cast<const UPCGParamData>(OutData);
		REQUIRE(OutParam);
		CHECK(OutParam->Metadata->GetLocalItemCount() == 1);
		const FPCGMetadataDomain* DataDomain = OutParam->Metadata->GetConstMetadataDomain(PCGMetadataDomainID::Data);

		// If the data domain has no attribute it would have never been initialized. So it is either null or it should not have the attribute.
		CHECK((DataDomain == nullptr || !DataDomain->HasAttribute(ExpectedOutputName)));

		CHECK(ReadAttr<double>(OutData, ExpectedOutputName, PCGFirstEntryKey) == Expected);
	}
}

// =============================================================================
// Cross-type axis
// =============================================================================

TEST_CASE_METHOD(FPCGExtractAttributeTest, "PCG::ExtractAttribute::AtIndex::CrossType::Int32", "[PCG][ExtractAttribute][AtIndex][CrossType]")
{
	ConfigureAtIndex(/*Index=*/1);
	AddInput(MakeParam<int32>({10, 20, 30}));

	ExecuteElement();

	CHECK(NumErrors == 0);
	const UPCGData* OutData = RequireSingleOutput();
	CHECK(ReadAttr<int32>(OutData, OutputAttrName, PCGFirstEntryKey) == 20);
}

TEST_CASE_METHOD(FPCGExtractAttributeTest, "PCG::ExtractAttribute::AtIndex::CrossType::Double", "[PCG][ExtractAttribute][AtIndex][CrossType]")
{
	ConfigureAtIndex(/*Index=*/1);
	AddInput(MakeParam<double>({1.5, 2.5, 3.5}));

	ExecuteElement();

	CHECK(NumErrors == 0);
	const UPCGData* OutData = RequireSingleOutput();
	CHECK(ReadAttr<double>(OutData, OutputAttrName, PCGFirstEntryKey) == 2.5);
}

TEST_CASE_METHOD(FPCGExtractAttributeTest, "PCG::ExtractAttribute::AtIndex::CrossType::FVector", "[PCG][ExtractAttribute][AtIndex][CrossType]")
{
	const FVector Expected(4.0, 5.0, 6.0);

	ConfigureAtIndex(/*Index=*/1);
	AddInput(MakeParam<FVector>({FVector(1.0, 2.0, 3.0), Expected, FVector(7.0, 8.0, 9.0)}));

	ExecuteElement();

	CHECK(NumErrors == 0);
	const UPCGData* OutData = RequireSingleOutput();
	CHECK(ReadAttr<FVector>(OutData, OutputAttrName, PCGFirstEntryKey) == Expected);
}

TEST_CASE_METHOD(FPCGExtractAttributeTest, "PCG::ExtractAttribute::AtIndex::CrossType::FString", "[PCG][ExtractAttribute][AtIndex][CrossType]")
{
	ConfigureAtIndex(/*Index=*/1);
	AddInput(MakeParam<FString>({TEXT("alpha"), TEXT("bravo"), TEXT("charlie")}));

	ExecuteElement();

	CHECK(NumErrors == 0);
	const UPCGData* OutData = RequireSingleOutput();
	CHECK(ReadAttr<FString>(OutData, OutputAttrName, PCGFirstEntryKey) == TEXT("bravo"));
}

// =============================================================================
// Input data type axis
// =============================================================================

TEST_CASE_METHOD(FPCGExtractAttributeTest, "PCG::ExtractAttribute::AtIndex::Input::ParamData", "[PCG][ExtractAttribute][AtIndex]")
{
	ConfigureAtIndex(/*Index=*/0);
	AddInput(MakeParam<double>({42.0}));

	ExecuteElement();

	CHECK(NumErrors == 0);
	const UPCGData* OutData = RequireSingleOutput();
	const UPCGParamData* OutParam = Cast<const UPCGParamData>(OutData);
	REQUIRE(OutParam);
	CHECK(OutParam->Metadata->GetLocalItemCount() == 1);
	CHECK(ReadAttr<double>(OutData, OutputAttrName, PCGFirstEntryKey) == 42.0);
}

TEST_CASE_METHOD(FPCGExtractAttributeTest, "PCG::ExtractAttribute::AtIndex::Input::PointData", "[PCG][ExtractAttribute][AtIndex]")
{
	ConfigureAtIndex(/*Index=*/2);
	AddInput(MakePointData<double>({1.0, 2.0, 3.0, 4.0}));

	ExecuteElement();

	CHECK(NumErrors == 0);
	const UPCGData* OutData = RequireSingleOutput();
	// AtIndex always emits a UPCGParamData regardless of input type.
	const UPCGParamData* OutParam = Cast<const UPCGParamData>(OutData);
	REQUIRE(OutParam);
	CHECK(OutParam->Metadata->GetLocalItemCount() == 1);
	CHECK(ReadAttr<double>(OutData, OutputAttrName, PCGFirstEntryKey) == 3.0);
}

TEST_CASE_METHOD(FPCGExtractAttributeTest, "PCG::ExtractAttribute::AtIndex::Output::IsParamData", "[PCG][ExtractAttribute][AtIndex]")
{
	// Even when the input is a UPCGBasePointData, the output is a single-entry UPCGParamData.
	ConfigureAtIndex(/*Index=*/0);
	AddInput(MakePointData<double>({99.0, 100.0}));

	ExecuteElement();

	CHECK(NumErrors == 0);
	const UPCGData* OutData = RequireSingleOutput();
	CHECK(Cast<const UPCGParamData>(OutData) != nullptr);
}