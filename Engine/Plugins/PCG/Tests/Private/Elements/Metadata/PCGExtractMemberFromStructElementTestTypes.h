// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

#include "PCGExtractMemberFromStructElementTestTypes.generated.h"

/** Inner leaf struct used to compose the array-in-chain test layouts. */
USTRUCT()
struct FPCGExtractAttrTestLeaf
{
	GENERATED_BODY()

	UPROPERTY()
	int32 Value = 0;
};

/**
 * Test struct exercising the property-chain extractor for the
 * PCGExtractMemberFromStruct element. Members:
 *  - LeafIntArray:    TArray<int32> at the outer level — leaf array (chain length 1)
 *  - ArrayOfStructs:  TArray<FPCGExtractAttrTestLeaf> — array property in the middle
 *                     of a 2-element chain when traversing ArrayOfStructs.Value
 *  - NestedStruct:    FPCGExtractAttrTestLeaf — non-array struct in the middle of
 *                     a 2-element chain when traversing NestedStruct.Value
 */
USTRUCT()
struct FPCGExtractAttrTestOuter
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<int32> LeafIntArray;

	UPROPERTY()
	TArray<FPCGExtractAttrTestLeaf> ArrayOfStructs;

	UPROPERTY()
	FPCGExtractAttrTestLeaf NestedStruct;
};

/** Inner leaf with a mix of BlueprintReadWrite / non-BP members for bExtractAll filter tests. */
USTRUCT(BlueprintType)
struct FPCGExtractAttrTestBPLeaf
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, Category = "PCGTest")
	int32 BPValue = 0;

	UPROPERTY()
	int32 NonBPValue = 0;
};

/**
 * Test struct exercising the bExtractAll CPF_BlueprintVisible filter:
 *  - BPInt:           BlueprintReadWrite int32 — extracted
 *  - BPNestedStruct:  BlueprintReadWrite FPCGExtractAttrTestBPLeaf — extracted as leaf struct
 *                     (also used as the input source for "extract all into nested struct" cases)
 *  - NonBPInt:        plain UPROPERTY() int32 — filtered out
 */
USTRUCT(BlueprintType)
struct FPCGExtractAttrTestBPOuter
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, Category = "PCGTest")
	int32 BPInt = 0;

	UPROPERTY(BlueprintReadWrite, Category = "PCGTest")
	FPCGExtractAttrTestBPLeaf BPNestedStruct;

	UPROPERTY()
	int32 NonBPInt = 0;
};
