// Copyright Epic Games, Inc. All Rights Reserved.

// =============================================================================
// PCG Metadata Array Operation Element Tests
//
// Tests for UPCGMetadataArrayOperationSettings (PCGMetadataArrayOperation.cpp),
// the node that exposes 14 array-manipulation operations on metadata attributes.
//
// Specifically exercises the index math and byte-copy paths for non-trivially-
// copyable element types (FString, FPCGPoint), the Data-domain (single-entry)
// code path, and the cross-type broadcast / constructible conversions allowed
// by IsValidType.
//
// Test summary:
//
// Single-input ops (per-op edge-case axis on int32):
//  - Length:                         Op output equals input array size (empty / single / many)
//  - ConvertToArray:                 Single double value wraps into a 1-element TArray<double>
//  - MakeArray::Param:               Packs N int32 entries on a UPCGParamData into one array
//  - MakeArray::PointData:           Packs N int32 entries on a UPCGBasePointData into one array
//  - Flatten::Param:                 Unrolls TArray<int32> per-entry into N entries (empty / single / multi)
//  - Flatten::PointData:             Unrolls TArray<int32> per-point into N output points
//  - Pop::EdgeCases:                 drop-last / single-elem / empty-error
//
// Two-input ops (per-op edge-case axis on int32):
//  - Add::EdgeCases:                 empty array gets element / populated array appends at end
//  - AddUnique:                      value-already-present unchanged / new-value appended / empty input
//  - Get:                            first / middle / last / negative / OOB+ / OOB-
//  - Append:                         empty+empty / empty+full / full+empty / full+full
//  - Find:                           empty / found-middle / found-last / not-present
//  - Contains:                       empty / present / absent
//  - RemoveAtIndex::EdgeCases:       first / middle / last / negative / OOB+ / OOB-
//  - RemoveAtIndex::EmptyError:      Errors when called on an empty array
//
// Three-input ops (per-op edge-case axis on int32):
//  - Insert::EdgeCases:              first / middle / append-at-end / negative / OOB+ / OOB-
//  - Insert::EmptyArray:             Inserts at index 0 of an empty array
//  - ReplaceAtIndex::EdgeCases:      first / middle / last / negative / OOB+ / OOB-
//  - ReplaceAtIndex::EmptyError:     Errors when called on an empty array
//
// Cross-type axis (each runs int32 / double / FVector / FString / FPCGPoint at a middle index):
//  - CrossType::Insert:              Insert middle across element types
//  - CrossType::ReplaceAtIndex:      Replace middle across element types
//  - CrossType::RemoveAtIndex:       Remove middle across element types
//  - CrossType::Add:                 Append element across element types
//  - CrossType::Pop:                 Drop-last across element types
//  - CrossType::Append:              Concat two arrays of the same element type
//
// Type-promotion axis (broadcastable / constructible across pin types):
//  - TypePromotion::Append:          TArray<int32> appended to TArray<double>
//  - TypePromotion::Add:             int32 added into TArray<double>
//  - TypePromotion::AddUnique:       new value + already-present-after-promotion
//  - TypePromotion::Insert:          float inserted into TArray<double>
//  - TypePromotion::ReplaceAtIndex:  int32 replaced into TArray<double>
//  - TypePromotion::Find:            int32 needle located in TArray<double> haystack
//  - TypePromotion::Contains:        int32 needle queried against TArray<double>
//
// Data-domain axis (UPCGParamData single-entry Data domain; per
// SupportsSingleEntryDomains every op below is allowed except Flatten):
//  - DataDomain::Length:             Length on the single-entry Data domain
//  - DataDomain::Pop:                Pop on Data domain
//  - DataDomain::Add:                Add on Data domain
//  - DataDomain::AddUnique:          AddUnique new-value + already-present
//  - DataDomain::Get:                Get on Data domain
//  - DataDomain::Find:               Find on Data domain
//  - DataDomain::Contains:           Contains on Data domain
//  - DataDomain::RemoveAtIndex:      RemoveAtIndex on Data domain
//  - DataDomain::Append:             Append on Data domain
//  - DataDomain::Insert:             Insert on Data domain
//  - DataDomain::ReplaceAtIndex:     ReplaceAtIndex on Data domain
//  - DataDomain::ConvertToArray:     ConvertToArray on Data domain
//  - DataDomain::Flatten:            Flatten errors out (rejected by SupportsSingleEntryDomains)
// =============================================================================

#include "CoreMinimal.h"
#include "TestHarness.h"

#include <catch2/generators/catch_generators.hpp>

#include "PCGCommon.h"
#include "PCGContext.h"
#include "PCGData.h"
#include "PCGParamData.h"
#include "PCGTestsCommon.h"
#include "Data/PCGBasePointData.h"
#include "Elements/Metadata/PCGMetadataArrayOperation.h"
#include "Elements/Metadata/PCGMetadataOpElementBase.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/PCGMetadataAttributeTraits.h"
#include "Metadata/PCGMetadataContainerTypes.h"

#include "Metadata/PCGMetadataAttributeTestsCommonHelper.h"

namespace PCGMetadataArrayOpTests
{
	using namespace PCGAttributeTestsCommonHelper;

	// Small int32 tester to round out the type axis. Mirrors FloatTester / DoubleTester.
	inline TypedAttributeTester<int32> Int32Tester()
	{
		auto GenerateRandom = [](FRandomStream& Rng, int32& OutValue)
		{
			OutValue = Rng.RandRange(-1000000, 1000000);
		};

		auto Verify = [](const int32& LHS, const int32& RHS) { return LHS == RHS; };

		const int32 DefaultValue = 1234567;

		FPCGMetadataAttributeDesc ExpectedDesc
		{
			.Name = AttributeName,
			.ValueType = EPCGMetadataTypes::Integer32
		};

		return TypedAttributeTester<int32>(GenerateRandom, Verify, DefaultValue, MoveTemp(ExpectedDesc));
	}

	// Element-wise equality used by ExpectOutputArrayMatches. Default falls back to operator==,
	// which is fine for int32 / double / FVector / FString. FPCGPoint has no operator== so we
	// mirror what FPCGPointTester::Verify does: compare density and translation.
	template <typename T>
	bool AreEqual(const T& LHS, const T& RHS)
	{
		return LHS == RHS;
	}

	template <>
	inline bool AreEqual<FPCGPoint>(const FPCGPoint& LHS, const FPCGPoint& RHS)
	{
		return LHS.Density == RHS.Density
			&& LHS.Transform.GetLocation() == RHS.Transform.GetLocation();
	}
}

class FPCGMetadataArrayOpTest : public PCGTests::FPCGSingleElementBaseTest<UPCGMetadataArrayOperationSettings>
{
public:
	inline static const FName ArrayAttrName = TEXT("InArray");
	inline static const FName ValueAttrName = TEXT("InValue");
	inline static const FName IndexAttrName = TEXT("InIndex");
	inline static const FName OutputAttrName = TEXT("OutResult");

	// ------------------------------------------------------------------
	// Input factories
	// ------------------------------------------------------------------

	template <typename T>
	UPCGParamData* MakeArrayParam(const TArray<T>& Values, FName InAttrName = ArrayAttrName)
	{
		UPCGParamData* ParamData = NewObject<UPCGParamData>();
		PCGMetadataEntryKey Key = ParamData->Metadata->AddEntry();

		FPCGMetadataAttributeBase* Attr = ParamData->Metadata->CreateAttribute<TArray<T>>(InAttrName, TArray<T>{}, /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false);
		check(Attr);
		Attr->SetValue<TArray<T>>(Key, Values);

		return ParamData;
	}

	template <typename T>
	UPCGParamData* MakeArrayParamMulti(const TArray<TArray<T>>& PerEntry, FName InAttrName = ArrayAttrName)
	{
		UPCGParamData* ParamData = NewObject<UPCGParamData>();

		FPCGMetadataAttributeBase* Attr = ParamData->Metadata->CreateAttribute<TArray<T>>(InAttrName, TArray<T>{}, /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false);
		check(Attr);

		for (const TArray<T>& Values : PerEntry)
		{
			PCGMetadataEntryKey Key = ParamData->Metadata->AddEntry();
			Attr->SetValue<TArray<T>>(Key, Values);
		}

		return ParamData;
	}

	// Multi-entry single-value attribute. Used by MakeArray (which builds an array from a
	// list of single values across entries).
	template <typename T>
	UPCGParamData* MakeMultiValueParam(const TArray<T>& PerEntry, FName InAttrName = ArrayAttrName)
	{
		UPCGParamData* ParamData = NewObject<UPCGParamData>();

		FPCGMetadataAttributeBase* Attr = ParamData->Metadata->CreateAttribute<T>(InAttrName, PCG::Private::MetadataTraits<T>::ZeroValue(), /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false);
		check(Attr);

		for (const T& Value : PerEntry)
		{
			PCGMetadataEntryKey Key = ParamData->Metadata->AddEntry();
			Attr->SetValue<T>(Key, Value);
		}

		return ParamData;
	}

	template <typename T>
	UPCGBasePointData* MakeMultiValuePointData(const TArray<T>& PerPoint, FName InAttrName = ArrayAttrName)
	{
		UPCGBasePointData* PointData = CreatePointData();
		PointData->SetNumPoints(PerPoint.Num());
		PointData->AllocateProperties(EPCGPointNativeProperties::MetadataEntry);

		FPCGMetadataAttributeBase* Attr = PointData->MutableMetadata()->CreateAttribute<T>(InAttrName, PCG::Private::MetadataTraits<T>::ZeroValue(), /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false);
		check(Attr);

		TPCGValueRange<PCGMetadataEntryKey> EntryRange = PointData->GetMetadataEntryValueRange();
		for (int32 i = 0; i < PerPoint.Num(); ++i)
		{
			PointData->MutableMetadata()->InitializeOnSet(EntryRange[i]);
			Attr->SetValue<T>(EntryRange[i], PerPoint[i]);
		}

		return PointData;
	}

	template <typename T>
	UPCGParamData* MakeValueParam(const T& Value, FName InAttrName = ValueAttrName)
	{
		UPCGParamData* ParamData = NewObject<UPCGParamData>();
		PCGMetadataEntryKey Key = ParamData->Metadata->AddEntry();

		FPCGMetadataAttributeBase* Attr = ParamData->Metadata->CreateAttribute<T>(InAttrName, PCG::Private::MetadataTraits<T>::ZeroValue(), /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false);
		check(Attr);
		Attr->SetValue<T>(Key, Value);

		return ParamData;
	}

	// Index for the third pin of 3-input ops (Insert / ReplaceAtIndex). Uses IndexAttrName
	// so it matches InputSource3.
	UPCGParamData* MakeIndexParamForPin2(int32 Index)
	{
		return MakeValueParam<int32>(Index, IndexAttrName);
	}

	// Index for the second pin of 2-input ops (Get / RemoveAtIndex). Uses ValueAttrName
	// so it matches InputSource2.
	UPCGParamData* MakeIndexParamForPin1(int32 Index)
	{
		return MakeValueParam<int32>(Index, ValueAttrName);
	}

	// PointData with one TArray<T> attribute per point. Used to exercise the BasePointData
	// branches in CreateOutputData (PCGMetadataArrayOperation.cpp:1498 / 1558).
	template <typename T>
	UPCGBasePointData* MakeArrayPointData(const TArray<TArray<T>>& PerPoint, FName InAttrName = ArrayAttrName)
	{
		UPCGBasePointData* PointData = CreatePointData();
		PointData->SetNumPoints(PerPoint.Num());
		PointData->AllocateProperties(EPCGPointNativeProperties::MetadataEntry);

		FPCGMetadataAttributeBase* Attr = PointData->MutableMetadata()->CreateAttribute<TArray<T>>(InAttrName, TArray<T>{}, /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false);
		check(Attr);

		TPCGValueRange<PCGMetadataEntryKey> EntryRange = PointData->GetMetadataEntryValueRange();
		for (int32 i = 0; i < PerPoint.Num(); ++i)
		{
			PointData->MutableMetadata()->InitializeOnSet(EntryRange[i]);
			Attr->SetValue<TArray<T>>(EntryRange[i], PerPoint[i]);
		}

		return PointData;
	}

	// ------------------------------------------------------------------
	// Configuration / execution
	// ------------------------------------------------------------------

	void Configure(EPCGMetadataArrayOperation Op)
	{
		TypedSettings->Operation = Op;
		TypedSettings->InputSource1 = FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(ArrayAttrName);
		TypedSettings->InputSource2 = FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(ValueAttrName);
		TypedSettings->InputSource3 = FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(IndexAttrName);
		TypedSettings->OutputTarget.SetAttributeName(OutputAttrName);
	}

	// Hardcoded mirror of UPCGMetadataArrayOperationSettings::GetInputPinName (which is
	// protected). Driven by the live operand count from the (public) GetNumOperands.
	// The "Value" / "Index" literals match PCGMetadataArrayOperationConstants::ValuePinLabel /
	// IndexPinLabel in the (private) header.
	FName ResolveInputPinName(int32 PinIndex) const
	{
		const int32 NumOperands = TypedSettings->GetNumOperands();
		if (PinIndex == 0)
		{
			return NumOperands == 1 ? PCGPinConstants::DefaultInputLabel : PCGMetadataSettingsBaseConstants::DoubleInputFirstLabel;
		}
		if (PinIndex == 1)
		{
			return NumOperands == 3 ? FName(TEXT("Value")) : PCGMetadataSettingsBaseConstants::DoubleInputSecondLabel;
		}
		if (PinIndex == 2)
		{
			return NumOperands == 3 ? FName(TEXT("Index")) : PCGMetadataSettingsBaseConstants::DoubleInputThirdLabel;
		}
		return NAME_None;
	}

	void AddInput(int32 PinIndex, UPCGData* Data)
	{
		FPCGTaggedData& Tagged = InputData.TaggedData.Emplace_GetRef();
		Tagged.Data = Data;
		Tagged.Pin = ResolveInputPinName(PinIndex);
	}

	// ------------------------------------------------------------------
	// Output assertions
	// ------------------------------------------------------------------

	template <typename T>
	void ExpectOutputArrayAt(int32 OutputIndex, const TArray<T>& Expected)
	{
		REQUIRE(OutputIndex < Context->OutputData.TaggedData.Num());
		const UPCGData* OutData = Context->OutputData.TaggedData[OutputIndex].Data;
		REQUIRE(OutData);

		// Direct attribute read. Going through PCGAttributeAccessorHelpers::CreateConstAccessor +
		// GetRange constructs a fresh FArrayProperty whose Inner pointer doesn't match the
		// attribute's, which trips the FScriptArrayHelper "Inner == InInner" check and floods
		// the log until the runtime stack overflows. Bypassing the accessor for the typed
		// read avoids the issue.
		const UPCGMetadata* OutMetadata = OutData->ConstMetadata();
		REQUIRE(OutMetadata);
		const FPCGMetadataAttributeBase* AttrBase = OutMetadata->GetConstAttribute(OutputAttrName);
		REQUIRE(AttrBase);

		// Resolve the first metadata entry key for this data — for ParamData it's typically 0,
		// but for PointData each point owns its own entry key obtained through the keys helper.
		const FPCGAttributePropertySelector OutSelector = FPCGAttributePropertySelector::CreateAttributeSelector(OutputAttrName);
		const TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(OutData, OutSelector);
		REQUIRE(Keys.IsValid());
		REQUIRE(Keys->GetNum() >= 1);

		// Use the public templated GetKey<PCGMetadataEntryKey> which dispatches to the
		// protected GetMetadataEntryKeys path internally.
		const PCGMetadataEntryKey* EntryKeyPtr = nullptr;
		Keys->GetKey<PCGMetadataEntryKey>(EntryKeyPtr);
		const PCGMetadataEntryKey FirstEntryKey = EntryKeyPtr ? *EntryKeyPtr : PCGInvalidEntryKey;
		
		TConstArrayView<T> OutValue = AttrBase->GetValueFromItemKey<TConstArrayView<T>>(FirstEntryKey);

		REQUIRE(OutValue.Num() == Expected.Num());
		for (int32 i = 0; i < Expected.Num(); ++i)
		{
			CHECK(PCGMetadataArrayOpTests::AreEqual<T>(OutValue[i], Expected[i]));
		}
	}

	template <typename T>
	void ExpectOutputArray(const TArray<T>& Expected)
	{
		ExpectOutputArrayAt<T>(0, Expected);
	}

	template <typename T>
	void ExpectOutputValueAt(int32 OutputIndex, const T& Expected)
	{
		REQUIRE(OutputIndex < Context->OutputData.TaggedData.Num());
		const UPCGData* OutData = Context->OutputData.TaggedData[OutputIndex].Data;
		REQUIRE(OutData);

		const FPCGAttributePropertySelector OutSelector = FPCGAttributePropertySelector::CreateAttributeSelector(OutputAttrName);
		const TUniquePtr<const IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(OutData, OutSelector);
		const TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(OutData, OutSelector);
		REQUIRE(Accessor.IsValid());
		REQUIRE(Keys.IsValid());

		T OutValue{};
		REQUIRE(Accessor->Get(OutValue, /*Index=*/0, *Keys));
		CHECK(PCGMetadataArrayOpTests::AreEqual<T>(OutValue, Expected));
	}

	template <typename T>
	void ExpectOutputValue(const T& Expected)
	{
		ExpectOutputValueAt<T>(0, Expected);
	}

	void ExpectErrors(int32 AtLeast = 1)
	{
		CHECK(NumErrors >= AtLeast);
	}

	// Helper used by the cross-type happy-path SECTIONs. Builds a deterministic input array
	// from Tester.GenerateRandom, runs the op, and compares the output against Expected
	// element-by-element using AreEqual<T>.
	template <typename Tester>
	static TArray<typename Tester::Type> SeedArray(const Tester& T, int32 Count, int32 Seed = 1234)
	{
		using ValueType = typename Tester::Type;

		FRandomStream Rng(Seed);
		TArray<ValueType> Result;
		Result.Reserve(Count);
		for (int32 i = 0; i < Count; ++i)
		{
			ValueType Value{};
			T.GenerateRandom(Rng, Value);
			Result.Add(MoveTemp(Value));
		}
		return Result;
	}
};

// =============================================================================
// Single-input ops: Length, ConvertToArray, MakeArray, Flatten, Pop
// =============================================================================

TEST_CASE_METHOD(FPCGMetadataArrayOpTest, "PCG::ArrayOp::Length", "[PCG][MetadataArrayOp][Length]")
{
	auto [Input, ExpectedLength] = GENERATE(
		table<TArray<int32>, int32>({
			{TArray<int32>{}, 0},
			{TArray<int32>{42}, 1},
			{TArray<int32>{1, 2, 3, 4, 5}, 5}
		}));

	DYNAMIC_SECTION("Length of array of size " << Input.Num())
	{
		Configure(EPCGMetadataArrayOperation::Length);
		AddInput(0, MakeArrayParam<int32>(Input));

		ExecuteElement();

		CHECK(NumErrors == 0);
		ExpectOutputValue<int32>(ExpectedLength);
	}
}

TEST_CASE_METHOD(FPCGMetadataArrayOpTest, "PCG::ArrayOp::ConvertToArray", "[PCG][MetadataArrayOp][ConvertToArray]")
{
	Configure(EPCGMetadataArrayOperation::ConvertToArray);
	AddInput(0, MakeValueParam<double>(7.5, ArrayAttrName));

	ExecuteElement();

	CHECK(NumErrors == 0);
	ExpectOutputArray<double>({7.5});
}

TEST_CASE_METHOD(FPCGMetadataArrayOpTest, "PCG::ArrayOp::MakeArray::Param", "[PCG][MetadataArrayOp][MakeArray]")
{
	// MakeArray takes a list of single values (one per entry) and packs them into a single
	// array attribute output.
	const TArray<int32> Values = {1, 2, 3, 4, 5};

	Configure(EPCGMetadataArrayOperation::MakeArray);
	AddInput(0, MakeMultiValueParam<int32>(Values));

	ExecuteElement();

	CHECK(NumErrors == 0);
	ExpectOutputArray<int32>(Values);
}

TEST_CASE_METHOD(FPCGMetadataArrayOpTest, "PCG::ArrayOp::MakeArray::PointData", "[PCG][MetadataArrayOp][MakeArray]")
{
	const TArray<int32> Values = {10, 20, 30, 40};

	Configure(EPCGMetadataArrayOperation::MakeArray);
	AddInput(0, MakeMultiValuePointData<int32>(Values));

	ExecuteElement();

	CHECK(NumErrors == 0);
	ExpectOutputArray<int32>(Values);
}

TEST_CASE_METHOD(FPCGMetadataArrayOpTest, "PCG::ArrayOp::Flatten::Param", "[PCG][MetadataArrayOp][Flatten]")
{
	auto Per = GENERATE(
		TArray<TArray<int32>>{{}},
		TArray<TArray<int32>>{{42}},
		TArray<TArray<int32>>{{1, 2, 3}, {4, 5}, {6}});

	DYNAMIC_SECTION("Flatten " << Per.Num() << " arrays")
	{
		TArray<int32> ExpectedFlat;
		for (const TArray<int32>& Inner : Per)
		{
			ExpectedFlat.Append(Inner);
		}

		Configure(EPCGMetadataArrayOperation::Flatten);
		AddInput(0, MakeArrayParamMulti<int32>(Per));

		ExecuteElement();

		CHECK(NumErrors == 0);

		REQUIRE(Context->OutputData.TaggedData.Num() == 1);
		const UPCGData* OutData = Context->OutputData.TaggedData[0].Data;
		REQUIRE(OutData);

		// Flatten produces a single attribute with one entry per source element. Read each
		// entry back and concatenate.
		const FPCGAttributePropertySelector OutSelector = FPCGAttributePropertySelector::CreateAttributeSelector(OutputAttrName);
		const TUniquePtr<const IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(OutData, OutSelector);
		const TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(OutData, OutSelector);
		REQUIRE(Accessor.IsValid());
		REQUIRE(Keys.IsValid());
		REQUIRE(Keys->GetNum() == ExpectedFlat.Num());

		if (ExpectedFlat.Num() > 0)
		{
			TArray<int32> OutValues;
			OutValues.SetNumUninitialized(ExpectedFlat.Num());
			REQUIRE(Accessor->GetRange<int32>(OutValues, /*Index=*/0, *Keys));

			for (int32 i = 0; i < ExpectedFlat.Num(); ++i)
			{
				CHECK(OutValues[i] == ExpectedFlat[i]);
			}
		}
	}
}

TEST_CASE_METHOD(FPCGMetadataArrayOpTest, "PCG::ArrayOp::Flatten::PointData", "[PCG][MetadataArrayOp][Flatten]")
{
	const TArray<TArray<int32>> Per = {{1, 2}, {3}, {4, 5, 6}};
	const int32 ExpectedNum = 6;

	Configure(EPCGMetadataArrayOperation::Flatten);
	AddInput(0, MakeArrayPointData<int32>(Per));

	ExecuteElement();

	CHECK(NumErrors == 0);

	REQUIRE(Context->OutputData.TaggedData.Num() == 1);
	const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(Context->OutputData.TaggedData[0].Data);
	REQUIRE(OutPoints);
	CHECK(OutPoints->GetNumPoints() == ExpectedNum);
}

TEST_CASE_METHOD(FPCGMetadataArrayOpTest, "PCG::ArrayOp::Pop::EdgeCases", "[PCG][MetadataArrayOp][Pop]")
{
	SECTION("non-empty array drops the last element")
	{
		Configure(EPCGMetadataArrayOperation::Pop);
		AddInput(0, MakeArrayParam<int32>({10, 20, 30}));

		ExecuteElement();

		CHECK(NumErrors == 0);
		ExpectOutputArray<int32>({10, 20});
	}

	SECTION("single-element array becomes empty")
	{
		Configure(EPCGMetadataArrayOperation::Pop);
		AddInput(0, MakeArrayParam<int32>({99}));

		ExecuteElement();

		CHECK(NumErrors == 0);
		ExpectOutputArray<int32>({});
	}

	SECTION("empty array is an error")
	{
		FSuppressErrorsScope Suppress(*this);

		Configure(EPCGMetadataArrayOperation::Pop);
		AddInput(0, MakeArrayParam<int32>({}));

		ExecuteElement();

		ExpectErrors();
	}
}

// =============================================================================
// Two-input ops: Add, AddUnique, Get, Append, Find, Contains, RemoveAtIndex
// =============================================================================

TEST_CASE_METHOD(FPCGMetadataArrayOpTest, "PCG::ArrayOp::Add::EdgeCases", "[PCG][MetadataArrayOp][Add]")
{
	SECTION("empty array")
	{
		Configure(EPCGMetadataArrayOperation::Add);
		AddInput(0, MakeArrayParam<int32>({}));
		AddInput(1, MakeValueParam<int32>(7));

		ExecuteElement();

		CHECK(NumErrors == 0);
		ExpectOutputArray<int32>({7});
	}

	SECTION("populated array appends at end")
	{
		Configure(EPCGMetadataArrayOperation::Add);
		AddInput(0, MakeArrayParam<int32>({1, 2, 3, 4, 5}));
		AddInput(1, MakeValueParam<int32>(99));

		ExecuteElement();

		CHECK(NumErrors == 0);
		ExpectOutputArray<int32>({1, 2, 3, 4, 5, 99});
	}
}

TEST_CASE_METHOD(FPCGMetadataArrayOpTest, "PCG::ArrayOp::AddUnique", "[PCG][MetadataArrayOp][AddUnique]")
{
	// Regression sentinel for bug #1 (inverted condition). With the original
	// `!= INDEX_NONE` bug, "new value: appended" would have failed (no append) and
	// "value already present" would have appended a duplicate.
	SECTION("value already present: array is unchanged")
	{
		Configure(EPCGMetadataArrayOperation::AddUnique);
		AddInput(0, MakeArrayParam<int32>({1, 2, 3, 4, 5}));
		AddInput(1, MakeValueParam<int32>(3));

		ExecuteElement();

		CHECK(NumErrors == 0);
		ExpectOutputArray<int32>({1, 2, 3, 4, 5});
	}

	SECTION("new value: appended at end")
	{
		Configure(EPCGMetadataArrayOperation::AddUnique);
		AddInput(0, MakeArrayParam<int32>({1, 2, 3, 4, 5}));
		AddInput(1, MakeValueParam<int32>(99));

		ExecuteElement();

		CHECK(NumErrors == 0);
		ExpectOutputArray<int32>({1, 2, 3, 4, 5, 99});
	}

	SECTION("empty input: value added")
	{
		Configure(EPCGMetadataArrayOperation::AddUnique);
		AddInput(0, MakeArrayParam<int32>({}));
		AddInput(1, MakeValueParam<int32>(7));

		ExecuteElement();

		CHECK(NumErrors == 0);
		ExpectOutputArray<int32>({7});
	}
}

TEST_CASE_METHOD(FPCGMetadataArrayOpTest, "PCG::ArrayOp::Get", "[PCG][MetadataArrayOp][Get]")
{
	const TArray<int32> Source = {10, 20, 30, 40, 50};

	auto [Index, Expected, bExpectError] = GENERATE(
		table<int32, int32, bool>({
			{0, 10, false},   // first
			{2, 30, false},   // middle
			{4, 50, false},   // last
			{-1, 50, false},  // negative -> last
			{-3, 30, false},  // negative -> middle
			{5, 0, true},     // OOB positive
			{-6, 0, true}     // OOB negative
		}));

	DYNAMIC_SECTION("Get index " << Index)
	{
		Configure(EPCGMetadataArrayOperation::Get);
		AddInput(0, MakeArrayParam<int32>(Source));
		AddInput(1, MakeIndexParamForPin1(Index));

		if (bExpectError)
		{
			FSuppressErrorsScope Suppress(*this);
			ExecuteElement();
			ExpectErrors();
		}
		else
		{
			ExecuteElement();
			CHECK(NumErrors == 0);
			ExpectOutputValue<int32>(Expected);
		}
	}
}

TEST_CASE_METHOD(FPCGMetadataArrayOpTest, "PCG::ArrayOp::Append", "[PCG][MetadataArrayOp][Append]")
{
	auto [A, B, Expected] = GENERATE(
		table<TArray<int32>, TArray<int32>, TArray<int32>>({
			{TArray<int32>{}, TArray<int32>{}, TArray<int32>{}},
			{TArray<int32>{1, 2}, TArray<int32>{}, TArray<int32>{1, 2}},
			{TArray<int32>{}, TArray<int32>{3, 4}, TArray<int32>{3, 4}},
			{TArray<int32>{1, 2}, TArray<int32>{3, 4}, TArray<int32>{1, 2, 3, 4}}
		}));

	DYNAMIC_SECTION("Append [" << A.Num() << "] + [" << B.Num() << "]")
	{
		Configure(EPCGMetadataArrayOperation::Append);
		AddInput(0, MakeArrayParam<int32>(A));
		AddInput(1, MakeArrayParam<int32>(B, ValueAttrName));

		ExecuteElement();

		CHECK(NumErrors == 0);
		ExpectOutputArray<int32>(Expected);
	}
}

TEST_CASE_METHOD(FPCGMetadataArrayOpTest, "PCG::ArrayOp::Find", "[PCG][MetadataArrayOp][Find]")
{
	auto [Source, Needle, Expected] = GENERATE(
		table<TArray<int32>, int32, int32>({
			{TArray<int32>{}, 5, INDEX_NONE},
			{TArray<int32>{10, 20, 30}, 20, 1},          // middle
			{TArray<int32>{10, 20, 30}, 30, 2},          // last
			{TArray<int32>{10, 20, 30}, 99, INDEX_NONE}  // not present
		}));

	DYNAMIC_SECTION("Find in array of size " << Source.Num())
	{
		Configure(EPCGMetadataArrayOperation::Find);
		AddInput(0, MakeArrayParam<int32>(Source));
		AddInput(1, MakeValueParam<int32>(Needle));

		ExecuteElement();

		CHECK(NumErrors == 0);
		ExpectOutputValue<int32>(Expected);
	}
}

TEST_CASE_METHOD(FPCGMetadataArrayOpTest, "PCG::ArrayOp::Contains", "[PCG][MetadataArrayOp][Contains]")
{
	auto [Source, Needle, Expected] = GENERATE(
		table<TArray<int32>, int32, bool>({
			{TArray<int32>{}, 5, false},
			{TArray<int32>{10, 20, 30}, 20, true},
			{TArray<int32>{10, 20, 30}, 99, false}
		}));

	DYNAMIC_SECTION("Contains in array of size " << Source.Num())
	{
		Configure(EPCGMetadataArrayOperation::Contains);
		AddInput(0, MakeArrayParam<int32>(Source));
		AddInput(1, MakeValueParam<int32>(Needle));

		ExecuteElement();

		CHECK(NumErrors == 0);
		ExpectOutputValue<bool>(Expected);
	}
}

TEST_CASE_METHOD(FPCGMetadataArrayOpTest, "PCG::ArrayOp::RemoveAtIndex::EdgeCases", "[PCG][MetadataArrayOp][RemoveAtIndex]")
{
	const TArray<int32> Source = {10, 20, 30, 40, 50};

	// Regression sentinel for bug #2. Old code wrote out-of-bounds and left dest[Index]
	// uninitialized for any non-tail removal.
	auto [Index, Expected, bExpectError] = GENERATE(
		table<int32, TArray<int32>, bool>({
			{0,  TArray<int32>{20, 30, 40, 50}, false},  // first
			{2,  TArray<int32>{10, 20, 40, 50}, false},  // middle (sentinel)
			{4,  TArray<int32>{10, 20, 30, 40}, false},  // last
			{-1, TArray<int32>{10, 20, 30, 40}, false},  // negative -> last
			{-3, TArray<int32>{10, 20, 40, 50}, false},  // negative -> middle (sentinel)
			{5,  TArray<int32>{},               true},   // OOB positive
			{-6, TArray<int32>{},               true}    // OOB negative
		}));

	DYNAMIC_SECTION("RemoveAtIndex " << Index)
	{
		Configure(EPCGMetadataArrayOperation::RemoveAtIndex);
		AddInput(0, MakeArrayParam<int32>(Source));
		AddInput(1, MakeIndexParamForPin1(Index));

		if (bExpectError)
		{
			FSuppressErrorsScope Suppress(*this);
			ExecuteElement();
			ExpectErrors();
		}
		else
		{
			ExecuteElement();
			CHECK(NumErrors == 0);
			ExpectOutputArray<int32>(Expected);
		}
	}
}

TEST_CASE_METHOD(FPCGMetadataArrayOpTest, "PCG::ArrayOp::RemoveAtIndex::EmptyError", "[PCG][MetadataArrayOp][RemoveAtIndex]")
{
	FSuppressErrorsScope Suppress(*this);

	Configure(EPCGMetadataArrayOperation::RemoveAtIndex);
	AddInput(0, MakeArrayParam<int32>({}));
	AddInput(1, MakeIndexParamForPin1(0));

	ExecuteElement();

	ExpectErrors();
}

// =============================================================================
// Three-input ops: Insert, ReplaceAtIndex
// =============================================================================

TEST_CASE_METHOD(FPCGMetadataArrayOpTest, "PCG::ArrayOp::Insert::EdgeCases", "[PCG][MetadataArrayOp][Insert]")
{
	const TArray<int32> Source = {10, 20, 30, 40, 50};
	constexpr int32 NewValue = 99;

	// Regression sentinel for bug #3 (off-by-one). Old code dropped the last source element
	// and left a slot uninitialized for middle / last insertions.
	auto [Index, Expected, bExpectError] = GENERATE(
		table<int32, TArray<int32>, bool>({
			{0,  TArray<int32>{99, 10, 20, 30, 40, 50}, false},  // first
			{2,  TArray<int32>{10, 20, 99, 30, 40, 50}, false},  // middle (sentinel)
			{5,  TArray<int32>{10, 20, 30, 40, 50, 99}, false},  // append at end (Index == ArrayNum)
			{-1, TArray<int32>{10, 20, 30, 40, 50, 99}, false},  // negative-1 inserts at end
			{-3, TArray<int32>{10, 20, 30, 99, 40, 50}, false},  // negative middle
			{6,  TArray<int32>{},                       true},   // OOB positive
			{-7, TArray<int32>{},                       true}    // OOB negative
		}));

	DYNAMIC_SECTION("Insert at " << Index)
	{
		Configure(EPCGMetadataArrayOperation::Insert);
		AddInput(0, MakeArrayParam<int32>(Source));
		AddInput(1, MakeValueParam<int32>(NewValue));
		AddInput(2, MakeIndexParamForPin2(Index));

		if (bExpectError)
		{
			FSuppressErrorsScope Suppress(*this);
			ExecuteElement();
			ExpectErrors();
		}
		else
		{
			ExecuteElement();
			CHECK(NumErrors == 0);
			ExpectOutputArray<int32>(Expected);
		}
	}
}

TEST_CASE_METHOD(FPCGMetadataArrayOpTest, "PCG::ArrayOp::Insert::EmptyArray", "[PCG][MetadataArrayOp][Insert]")
{
	Configure(EPCGMetadataArrayOperation::Insert);
	AddInput(0, MakeArrayParam<int32>({}));
	AddInput(1, MakeValueParam<int32>(7));
	AddInput(2, MakeIndexParamForPin2(0));

	ExecuteElement();

	CHECK(NumErrors == 0);
	ExpectOutputArray<int32>({7});
}

TEST_CASE_METHOD(FPCGMetadataArrayOpTest, "PCG::ArrayOp::ReplaceAtIndex::EdgeCases", "[PCG][MetadataArrayOp][ReplaceAtIndex]")
{
	const TArray<int32> Source = {10, 20, 30, 40, 50};
	constexpr int32 NewValue = 99;

	// Regression sentinel for bug #4. Old code dropped the last element on middle replacements.
	auto [Index, Expected, bExpectError] = GENERATE(
		table<int32, TArray<int32>, bool>({
			{0,  TArray<int32>{99, 20, 30, 40, 50}, false},  // first
			{2,  TArray<int32>{10, 20, 99, 40, 50}, false},  // middle (sentinel)
			{4,  TArray<int32>{10, 20, 30, 40, 99}, false},  // last
			{-1, TArray<int32>{10, 20, 30, 40, 99}, false},  // negative -> last
			{-3, TArray<int32>{10, 20, 99, 40, 50}, false},  // negative -> middle (sentinel)
			{5,  TArray<int32>{},                   true},   // OOB positive
			{-6, TArray<int32>{},                   true}    // OOB negative
		}));

	DYNAMIC_SECTION("ReplaceAtIndex " << Index)
	{
		Configure(EPCGMetadataArrayOperation::ReplaceAtIndex);
		AddInput(0, MakeArrayParam<int32>(Source));
		AddInput(1, MakeValueParam<int32>(NewValue));
		AddInput(2, MakeIndexParamForPin2(Index));

		if (bExpectError)
		{
			FSuppressErrorsScope Suppress(*this);
			ExecuteElement();
			ExpectErrors();
		}
		else
		{
			ExecuteElement();
			CHECK(NumErrors == 0);
			ExpectOutputArray<int32>(Expected);
		}
	}
}

TEST_CASE_METHOD(FPCGMetadataArrayOpTest, "PCG::ArrayOp::ReplaceAtIndex::EmptyError", "[PCG][MetadataArrayOp][ReplaceAtIndex]")
{
	FSuppressErrorsScope Suppress(*this);

	Configure(EPCGMetadataArrayOperation::ReplaceAtIndex);
	AddInput(0, MakeArrayParam<int32>({}));
	AddInput(1, MakeValueParam<int32>(7));
	AddInput(2, MakeIndexParamForPin2(0));

	ExecuteElement();

	ExpectErrors();
}

// =============================================================================
// Cross-type axis: byte-copy ops on int32 / double / FVector / FString / FPCGPoint.
//
// Single happy-path case per type per op. The point of this axis is to verify that the
// CopyArray paths preserve the bits faithfully for non-trivially-copyable types — the
// regressions in bugs #2-#4 would corrupt the FString heap pointer and the FPCGPoint
// inner struct on most middle-index runs, surfacing as garbage / leaks.
// =============================================================================

namespace PCGMetadataArrayOpTests
{
	template <typename Tester>
	void RunInsertMiddle(FPCGMetadataArrayOpTest& Fixture, const Tester& T)
	{
		using ValueType = typename Tester::Type;

		TArray<ValueType> Source = FPCGMetadataArrayOpTest::SeedArray(T, /*Count=*/5);
		const ValueType ToInsert = T.DefaultValue;
		constexpr int32 Index = 2;

		Fixture.Configure(EPCGMetadataArrayOperation::Insert);
		Fixture.AddInput(0, Fixture.MakeArrayParam<ValueType>(Source));
		Fixture.AddInput(1, Fixture.MakeValueParam<ValueType>(ToInsert));
		Fixture.AddInput(2, Fixture.MakeIndexParamForPin2(Index));

		Fixture.ExecuteElement();

		CHECK(Fixture.NumErrors == 0);

		TArray<ValueType> Expected = Source;
		Expected.Insert(ToInsert, Index);
		Fixture.ExpectOutputArray<ValueType>(Expected);
	}

	template <typename Tester>
	void RunReplaceMiddle(FPCGMetadataArrayOpTest& Fixture, const Tester& T)
	{
		using ValueType = typename Tester::Type;

		TArray<ValueType> Source = FPCGMetadataArrayOpTest::SeedArray(T, /*Count=*/5);
		const ValueType Replacement = T.DefaultValue;
		constexpr int32 Index = 2;

		Fixture.Configure(EPCGMetadataArrayOperation::ReplaceAtIndex);
		Fixture.AddInput(0, Fixture.MakeArrayParam<ValueType>(Source));
		Fixture.AddInput(1, Fixture.MakeValueParam<ValueType>(Replacement));
		Fixture.AddInput(2, Fixture.MakeIndexParamForPin2(Index));

		Fixture.ExecuteElement();

		CHECK(Fixture.NumErrors == 0);

		TArray<ValueType> Expected = Source;
		Expected[Index] = Replacement;
		Fixture.ExpectOutputArray<ValueType>(Expected);
	}

	template <typename Tester>
	void RunRemoveMiddle(FPCGMetadataArrayOpTest& Fixture, const Tester& T)
	{
		using ValueType = typename Tester::Type;

		TArray<ValueType> Source = FPCGMetadataArrayOpTest::SeedArray(T, /*Count=*/5);
		constexpr int32 Index = 2;

		Fixture.Configure(EPCGMetadataArrayOperation::RemoveAtIndex);
		Fixture.AddInput(0, Fixture.MakeArrayParam<ValueType>(Source));
		Fixture.AddInput(1, Fixture.MakeIndexParamForPin1(Index));

		Fixture.ExecuteElement();

		CHECK(Fixture.NumErrors == 0);

		TArray<ValueType> Expected = Source;
		Expected.RemoveAt(Index);
		Fixture.ExpectOutputArray<ValueType>(Expected);
	}

	template <typename Tester>
	void RunAdd(FPCGMetadataArrayOpTest& Fixture, const Tester& T)
	{
		using ValueType = typename Tester::Type;

		TArray<ValueType> Source = FPCGMetadataArrayOpTest::SeedArray(T, /*Count=*/3);
		const ValueType ToAdd = T.DefaultValue;

		Fixture.Configure(EPCGMetadataArrayOperation::Add);
		Fixture.AddInput(0, Fixture.MakeArrayParam<ValueType>(Source));
		Fixture.AddInput(1, Fixture.MakeValueParam<ValueType>(ToAdd));

		Fixture.ExecuteElement();

		CHECK(Fixture.NumErrors == 0);

		TArray<ValueType> Expected = Source;
		Expected.Add(ToAdd);
		Fixture.ExpectOutputArray<ValueType>(Expected);
	}

	template <typename Tester>
	void RunPop(FPCGMetadataArrayOpTest& Fixture, const Tester& T)
	{
		using ValueType = typename Tester::Type;

		TArray<ValueType> Source = FPCGMetadataArrayOpTest::SeedArray(T, /*Count=*/4);

		Fixture.Configure(EPCGMetadataArrayOperation::Pop);
		Fixture.AddInput(0, Fixture.MakeArrayParam<ValueType>(Source));

		Fixture.ExecuteElement();

		CHECK(Fixture.NumErrors == 0);

		TArray<ValueType> Expected = Source;
		Expected.Pop();
		Fixture.ExpectOutputArray<ValueType>(Expected);
	}

	template <typename Tester>
	void RunAppend(FPCGMetadataArrayOpTest& Fixture, const Tester& T)
	{
		using ValueType = typename Tester::Type;

		TArray<ValueType> A = FPCGMetadataArrayOpTest::SeedArray(T, /*Count=*/3, /*Seed=*/1);
		TArray<ValueType> B = FPCGMetadataArrayOpTest::SeedArray(T, /*Count=*/2, /*Seed=*/2);

		Fixture.Configure(EPCGMetadataArrayOperation::Append);
		Fixture.AddInput(0, Fixture.MakeArrayParam<ValueType>(A));
		Fixture.AddInput(1, Fixture.MakeArrayParam<ValueType>(B, FPCGMetadataArrayOpTest::ValueAttrName));

		Fixture.ExecuteElement();

		CHECK(Fixture.NumErrors == 0);

		TArray<ValueType> Expected = A;
		Expected.Append(B);
		Fixture.ExpectOutputArray<ValueType>(Expected);
	}
}

TEST_CASE_METHOD(FPCGMetadataArrayOpTest, "PCG::ArrayOp::CrossType::Insert", "[PCG][MetadataArrayOp][Insert][CrossType]")
{
	SECTION("int32")
	{
		PCGMetadataArrayOpTests::RunInsertMiddle(*this, PCGMetadataArrayOpTests::Int32Tester());
	}

	SECTION("double")
	{
		PCGMetadataArrayOpTests::RunInsertMiddle(*this, PCGAttributeTestsCommonHelper::DoubleTester());
	}

	SECTION("FVector")
	{
		PCGMetadataArrayOpTests::RunInsertMiddle(*this, PCGAttributeTestsCommonHelper::VectorTester_Basic());
	}

	SECTION("FString")
	{
		PCGMetadataArrayOpTests::RunInsertMiddle(*this, PCGAttributeTestsCommonHelper::StringTester());
	}

	SECTION("FPCGPoint")
	{
		PCGMetadataArrayOpTests::RunInsertMiddle(*this, PCGAttributeTestsCommonHelper::FPCGPointTester());
	}
}

TEST_CASE_METHOD(FPCGMetadataArrayOpTest, "PCG::ArrayOp::CrossType::ReplaceAtIndex", "[PCG][MetadataArrayOp][ReplaceAtIndex][CrossType]")
{
	SECTION("int32")
	{
		PCGMetadataArrayOpTests::RunReplaceMiddle(*this, PCGMetadataArrayOpTests::Int32Tester());
	}

	SECTION("double")
	{
		PCGMetadataArrayOpTests::RunReplaceMiddle(*this, PCGAttributeTestsCommonHelper::DoubleTester());
	}

	SECTION("FVector")
	{
		PCGMetadataArrayOpTests::RunReplaceMiddle(*this, PCGAttributeTestsCommonHelper::VectorTester_Basic());
	}

	SECTION("FString")
	{
		PCGMetadataArrayOpTests::RunReplaceMiddle(*this, PCGAttributeTestsCommonHelper::StringTester());
	}

	SECTION("FPCGPoint")
	{
		PCGMetadataArrayOpTests::RunReplaceMiddle(*this, PCGAttributeTestsCommonHelper::FPCGPointTester());
	}
}

TEST_CASE_METHOD(FPCGMetadataArrayOpTest, "PCG::ArrayOp::CrossType::RemoveAtIndex", "[PCG][MetadataArrayOp][RemoveAtIndex][CrossType]")
{
	SECTION("int32")
	{
		PCGMetadataArrayOpTests::RunRemoveMiddle(*this, PCGMetadataArrayOpTests::Int32Tester());
	}

	SECTION("double")
	{
		PCGMetadataArrayOpTests::RunRemoveMiddle(*this, PCGAttributeTestsCommonHelper::DoubleTester());
	}

	SECTION("FVector")
	{
		PCGMetadataArrayOpTests::RunRemoveMiddle(*this, PCGAttributeTestsCommonHelper::VectorTester_Basic());
	}

	SECTION("FString")
	{
		PCGMetadataArrayOpTests::RunRemoveMiddle(*this, PCGAttributeTestsCommonHelper::StringTester());
	}

	SECTION("FPCGPoint")
	{
		PCGMetadataArrayOpTests::RunRemoveMiddle(*this, PCGAttributeTestsCommonHelper::FPCGPointTester());
	}
}

TEST_CASE_METHOD(FPCGMetadataArrayOpTest, "PCG::ArrayOp::CrossType::Add", "[PCG][MetadataArrayOp][Add][CrossType]")
{
	SECTION("int32")
	{
		PCGMetadataArrayOpTests::RunAdd(*this, PCGMetadataArrayOpTests::Int32Tester());
	}

	SECTION("double")
	{
		PCGMetadataArrayOpTests::RunAdd(*this, PCGAttributeTestsCommonHelper::DoubleTester());
	}

	SECTION("FVector")
	{
		PCGMetadataArrayOpTests::RunAdd(*this, PCGAttributeTestsCommonHelper::VectorTester_Basic());
	}

	SECTION("FString")
	{
		PCGMetadataArrayOpTests::RunAdd(*this, PCGAttributeTestsCommonHelper::StringTester());
	}

	SECTION("FPCGPoint")
	{
		PCGMetadataArrayOpTests::RunAdd(*this, PCGAttributeTestsCommonHelper::FPCGPointTester());
	}
}

TEST_CASE_METHOD(FPCGMetadataArrayOpTest, "PCG::ArrayOp::CrossType::Pop", "[PCG][MetadataArrayOp][Pop][CrossType]")
{
	SECTION("int32")
	{
		PCGMetadataArrayOpTests::RunPop(*this, PCGMetadataArrayOpTests::Int32Tester());
	}

	SECTION("double")
	{
		PCGMetadataArrayOpTests::RunPop(*this, PCGAttributeTestsCommonHelper::DoubleTester());
	}

	SECTION("FVector")
	{
		PCGMetadataArrayOpTests::RunPop(*this, PCGAttributeTestsCommonHelper::VectorTester_Basic());
	}

	SECTION("FString")
	{
		PCGMetadataArrayOpTests::RunPop(*this, PCGAttributeTestsCommonHelper::StringTester());
	}

	SECTION("FPCGPoint")
	{
		PCGMetadataArrayOpTests::RunPop(*this, PCGAttributeTestsCommonHelper::FPCGPointTester());
	}
}

TEST_CASE_METHOD(FPCGMetadataArrayOpTest, "PCG::ArrayOp::CrossType::Append", "[PCG][MetadataArrayOp][Append][CrossType]")
{
	SECTION("int32")
	{
		PCGMetadataArrayOpTests::RunAppend(*this, PCGMetadataArrayOpTests::Int32Tester());
	}

	SECTION("double")
	{
		PCGMetadataArrayOpTests::RunAppend(*this, PCGAttributeTestsCommonHelper::DoubleTester());
	}

	SECTION("FVector")
	{
		PCGMetadataArrayOpTests::RunAppend(*this, PCGAttributeTestsCommonHelper::VectorTester_Basic());
	}

	SECTION("FString")
	{
		PCGMetadataArrayOpTests::RunAppend(*this, PCGAttributeTestsCommonHelper::StringTester());
	}

	SECTION("FPCGPoint")
	{
		PCGMetadataArrayOpTests::RunAppend(*this, PCGAttributeTestsCommonHelper::FPCGPointTester());
	}
}

// =============================================================================
// Type-promotion axis: ops where pin 1's element type differs from pin 0's, but
// is broadcastable / constructible to it. Output uses pin 0's element type.
//
// Source-side gate is UPCGMetadataArrayOperationSettings::IsValidType which calls
// PCG::Private::IsBroadcastableOrConstructible(...). int32 -> double is broadcastable
// (lossless integer-to-floating-point), float -> double is too. These tests verify
// the value actually flows through and gets converted (rather than being silently
// reinterpreted or rejected).
// =============================================================================

TEST_CASE_METHOD(FPCGMetadataArrayOpTest, "PCG::ArrayOp::TypePromotion::Append", "[PCG][MetadataArrayOp][Append][TypePromotion]")
{
	// Pin 0: TArray<double>, Pin 1: TArray<int32>. int32 -> double broadcasts.
	Configure(EPCGMetadataArrayOperation::Append);
	AddInput(0, MakeArrayParam<double>({1.0, 2.0}));
	AddInput(1, MakeArrayParam<int32>({3, 4}, ValueAttrName));

	ExecuteElement();

	CHECK(NumErrors == 0);
	ExpectOutputArray<double>({1.0, 2.0, 3.0, 4.0});
}

TEST_CASE_METHOD(FPCGMetadataArrayOpTest, "PCG::ArrayOp::TypePromotion::Add", "[PCG][MetadataArrayOp][Add][TypePromotion]")
{
	// Pin 0: TArray<double>, Pin 1: int32 single value.
	Configure(EPCGMetadataArrayOperation::Add);
	AddInput(0, MakeArrayParam<double>({1.0, 2.0}));
	AddInput(1, MakeValueParam<int32>(3));

	ExecuteElement();

	CHECK(NumErrors == 0);
	ExpectOutputArray<double>({1.0, 2.0, 3.0});
}

TEST_CASE_METHOD(FPCGMetadataArrayOpTest, "PCG::ArrayOp::TypePromotion::AddUnique", "[PCG][MetadataArrayOp][AddUnique][TypePromotion]")
{
	SECTION("new value (broadcast int -> double)")
	{
		Configure(EPCGMetadataArrayOperation::AddUnique);
		AddInput(0, MakeArrayParam<double>({1.0, 2.0, 3.0}));
		AddInput(1, MakeValueParam<int32>(99));

		ExecuteElement();

		CHECK(NumErrors == 0);
		ExpectOutputArray<double>({1.0, 2.0, 3.0, 99.0});
	}

	SECTION("already present after promotion (int 2 == double 2.0)")
	{
		Configure(EPCGMetadataArrayOperation::AddUnique);
		AddInput(0, MakeArrayParam<double>({1.0, 2.0, 3.0}));
		AddInput(1, MakeValueParam<int32>(2));

		ExecuteElement();

		CHECK(NumErrors == 0);
		ExpectOutputArray<double>({1.0, 2.0, 3.0});
	}
}

TEST_CASE_METHOD(FPCGMetadataArrayOpTest, "PCG::ArrayOp::TypePromotion::Insert", "[PCG][MetadataArrayOp][Insert][TypePromotion]")
{
	// Pin 0: TArray<double>, Pin 1: float value, Pin 2: int32 index.
	Configure(EPCGMetadataArrayOperation::Insert);
	AddInput(0, MakeArrayParam<double>({1.0, 2.0, 3.0}));
	AddInput(1, MakeValueParam<float>(99.5f));
	AddInput(2, MakeIndexParamForPin2(1));

	ExecuteElement();

	CHECK(NumErrors == 0);
	ExpectOutputArray<double>({1.0, 99.5, 2.0, 3.0});
}

TEST_CASE_METHOD(FPCGMetadataArrayOpTest, "PCG::ArrayOp::TypePromotion::ReplaceAtIndex", "[PCG][MetadataArrayOp][ReplaceAtIndex][TypePromotion]")
{
	Configure(EPCGMetadataArrayOperation::ReplaceAtIndex);
	AddInput(0, MakeArrayParam<double>({1.0, 2.0, 3.0}));
	AddInput(1, MakeValueParam<int32>(99));
	AddInput(2, MakeIndexParamForPin2(1));

	ExecuteElement();

	CHECK(NumErrors == 0);
	ExpectOutputArray<double>({1.0, 99.0, 3.0});
}

TEST_CASE_METHOD(FPCGMetadataArrayOpTest, "PCG::ArrayOp::TypePromotion::Find", "[PCG][MetadataArrayOp][Find][TypePromotion]")
{
	// Look up an int32 needle inside a double array — match if int promoted to double matches.
	Configure(EPCGMetadataArrayOperation::Find);
	AddInput(0, MakeArrayParam<double>({10.0, 20.0, 30.0}));
	AddInput(1, MakeValueParam<int32>(20));

	ExecuteElement();

	CHECK(NumErrors == 0);
	ExpectOutputValue<int32>(1);
}

TEST_CASE_METHOD(FPCGMetadataArrayOpTest, "PCG::ArrayOp::TypePromotion::Contains", "[PCG][MetadataArrayOp][Contains][TypePromotion]")
{
	Configure(EPCGMetadataArrayOperation::Contains);
	AddInput(0, MakeArrayParam<double>({10.0, 20.0, 30.0}));
	AddInput(1, MakeValueParam<int32>(20));

	ExecuteElement();

	CHECK(NumErrors == 0);
	ExpectOutputValue<bool>(true);
}

// =============================================================================
// Data-domain axis: a UPCGParamData has both an Elements (default) domain and a
// single-entry Data domain. Most array ops are valid on the Data domain because
// SupportsSingleEntryDomains returns true when NumberOfElementsToProcess <= 1.
// Flatten is explicitly rejected.
// =============================================================================

namespace PCGMetadataArrayOpTests
{
	// Build a UPCGParamData with a single-entry Data domain attribute.
	template <typename T>
	UPCGParamData* MakeArrayParamOnDataDomain(const TArray<T>& Values, FName AttrName)
	{
		UPCGParamData* ParamData = NewObject<UPCGParamData>();
		FPCGMetadataDomain* DataDomain = ParamData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Data);
		check(DataDomain);

		const PCGMetadataEntryKey Key = DataDomain->AddEntry();
		FPCGMetadataAttributeBase* Attr = DataDomain->CreateAttribute<TArray<T>>(AttrName, TArray<T>{}, /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false);
		check(Attr);
		Attr->SetValue<TArray<T>>(Key, Values);

		return ParamData;
	}

	template <typename T>
	UPCGParamData* MakeValueParamOnDataDomain(const T& Value, FName AttrName)
	{
		UPCGParamData* ParamData = NewObject<UPCGParamData>();
		FPCGMetadataDomain* DataDomain = ParamData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Data);
		check(DataDomain);

		const PCGMetadataEntryKey Key = DataDomain->AddEntry();
		FPCGMetadataAttributeBase* Attr = DataDomain->CreateAttribute<T>(AttrName, PCG::Private::MetadataTraits<T>::ZeroValue(), /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false);
		check(Attr);
		Attr->SetValue<T>(Key, Value);

		return ParamData;
	}
}

class FPCGMetadataArrayOpDataDomainTest : public FPCGMetadataArrayOpTest
{
public:
	// Like Configure() but routes every selector through the Data domain.
	void ConfigureOnDataDomain(EPCGMetadataArrayOperation Op)
	{
		TypedSettings->Operation = Op;
		TypedSettings->InputSource1 = FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(ArrayAttrName, PCGDataConstants::DataDomainName);
		TypedSettings->InputSource2 = FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(ValueAttrName, PCGDataConstants::DataDomainName);
		TypedSettings->InputSource3 = FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(IndexAttrName, PCGDataConstants::DataDomainName);
		TypedSettings->OutputTarget.SetDomainName(PCGDataConstants::DataDomainName);
		TypedSettings->OutputTarget.SetAttributeName(OutputAttrName);
	}

	template <typename T>
	void ExpectOutputArrayOnDataDomain(const TArray<T>& Expected)
	{
		REQUIRE(Context->OutputData.TaggedData.Num() >= 1);
		const UPCGData* OutData = Context->OutputData.TaggedData[0].Data;
		REQUIRE(OutData);

		const UPCGMetadata* OutMetadata = OutData->ConstMetadata();
		REQUIRE(OutMetadata);
		const FPCGMetadataDomain* DataDomain = OutMetadata->GetConstMetadataDomain(PCGMetadataDomainID::Data);
		REQUIRE(DataDomain);

		const FPCGMetadataAttributeBase* AttrBase = DataDomain->GetConstAttribute(OutputAttrName);
		REQUIRE(AttrBase);

		// Single-entry domain — first key is the data entry.
		const TConstArrayView<T> OutValue = AttrBase->GetValueFromItemKey<TConstArrayView<T>>(/*EntryKey=*/0);

		REQUIRE(OutValue.Num() == Expected.Num());
		for (int32 i = 0; i < Expected.Num(); ++i)
		{
			CHECK(PCGMetadataArrayOpTests::AreEqual<T>(OutValue[i], Expected[i]));
		}
	}

	template <typename T>
	void ExpectOutputValueOnDataDomain(const T& Expected)
	{
		REQUIRE(Context->OutputData.TaggedData.Num() >= 1);
		const UPCGData* OutData = Context->OutputData.TaggedData[0].Data;
		REQUIRE(OutData);

		const UPCGMetadata* OutMetadata = OutData->ConstMetadata();
		REQUIRE(OutMetadata);
		const FPCGMetadataDomain* DataDomain = OutMetadata->GetConstMetadataDomain(PCGMetadataDomainID::Data);
		REQUIRE(DataDomain);

		const FPCGMetadataAttributeBase* AttrBase = DataDomain->GetConstAttribute(OutputAttrName);
		REQUIRE(AttrBase);

		const T OutValue = AttrBase->GetValueFromItemKey<T>(/*EntryKey=*/0);
		CHECK(PCGMetadataArrayOpTests::AreEqual<T>(OutValue, Expected));
	}
};

TEST_CASE_METHOD(FPCGMetadataArrayOpDataDomainTest, "PCG::ArrayOp::DataDomain::Length", "[PCG][MetadataArrayOp][DataDomain]")
{
	ConfigureOnDataDomain(EPCGMetadataArrayOperation::Length);
	AddInput(0, PCGMetadataArrayOpTests::MakeArrayParamOnDataDomain<int32>({10, 20, 30, 40}, ArrayAttrName));

	ExecuteElement();

	CHECK(NumErrors == 0);
	ExpectOutputValueOnDataDomain<int32>(4);
}

TEST_CASE_METHOD(FPCGMetadataArrayOpDataDomainTest, "PCG::ArrayOp::DataDomain::Pop", "[PCG][MetadataArrayOp][DataDomain]")
{
	ConfigureOnDataDomain(EPCGMetadataArrayOperation::Pop);
	AddInput(0, PCGMetadataArrayOpTests::MakeArrayParamOnDataDomain<int32>({10, 20, 30}, ArrayAttrName));

	ExecuteElement();

	CHECK(NumErrors == 0);
	ExpectOutputArrayOnDataDomain<int32>({10, 20});
}

TEST_CASE_METHOD(FPCGMetadataArrayOpDataDomainTest, "PCG::ArrayOp::DataDomain::Add", "[PCG][MetadataArrayOp][DataDomain]")
{
	ConfigureOnDataDomain(EPCGMetadataArrayOperation::Add);
	AddInput(0, PCGMetadataArrayOpTests::MakeArrayParamOnDataDomain<int32>({1, 2, 3}, ArrayAttrName));
	AddInput(1, PCGMetadataArrayOpTests::MakeValueParamOnDataDomain<int32>(99, ValueAttrName));

	ExecuteElement();

	CHECK(NumErrors == 0);
	ExpectOutputArrayOnDataDomain<int32>({1, 2, 3, 99});
}

TEST_CASE_METHOD(FPCGMetadataArrayOpDataDomainTest, "PCG::ArrayOp::DataDomain::AddUnique", "[PCG][MetadataArrayOp][DataDomain]")
{
	SECTION("new value")
	{
		ConfigureOnDataDomain(EPCGMetadataArrayOperation::AddUnique);
		AddInput(0, PCGMetadataArrayOpTests::MakeArrayParamOnDataDomain<int32>({1, 2, 3}, ArrayAttrName));
		AddInput(1, PCGMetadataArrayOpTests::MakeValueParamOnDataDomain<int32>(99, ValueAttrName));

		ExecuteElement();

		CHECK(NumErrors == 0);
		ExpectOutputArrayOnDataDomain<int32>({1, 2, 3, 99});
	}

	SECTION("already present")
	{
		ConfigureOnDataDomain(EPCGMetadataArrayOperation::AddUnique);
		AddInput(0, PCGMetadataArrayOpTests::MakeArrayParamOnDataDomain<int32>({1, 2, 3}, ArrayAttrName));
		AddInput(1, PCGMetadataArrayOpTests::MakeValueParamOnDataDomain<int32>(2, ValueAttrName));

		ExecuteElement();

		CHECK(NumErrors == 0);
		ExpectOutputArrayOnDataDomain<int32>({1, 2, 3});
	}
}

TEST_CASE_METHOD(FPCGMetadataArrayOpDataDomainTest, "PCG::ArrayOp::DataDomain::Get", "[PCG][MetadataArrayOp][DataDomain]")
{
	ConfigureOnDataDomain(EPCGMetadataArrayOperation::Get);
	AddInput(0, PCGMetadataArrayOpTests::MakeArrayParamOnDataDomain<int32>({10, 20, 30}, ArrayAttrName));
	AddInput(1, PCGMetadataArrayOpTests::MakeValueParamOnDataDomain<int32>(1, ValueAttrName));

	ExecuteElement();

	CHECK(NumErrors == 0);
	ExpectOutputValueOnDataDomain<int32>(20);
}

TEST_CASE_METHOD(FPCGMetadataArrayOpDataDomainTest, "PCG::ArrayOp::DataDomain::Find", "[PCG][MetadataArrayOp][DataDomain]")
{
	ConfigureOnDataDomain(EPCGMetadataArrayOperation::Find);
	AddInput(0, PCGMetadataArrayOpTests::MakeArrayParamOnDataDomain<int32>({10, 20, 30}, ArrayAttrName));
	AddInput(1, PCGMetadataArrayOpTests::MakeValueParamOnDataDomain<int32>(20, ValueAttrName));

	ExecuteElement();

	CHECK(NumErrors == 0);
	ExpectOutputValueOnDataDomain<int32>(1);
}

TEST_CASE_METHOD(FPCGMetadataArrayOpDataDomainTest, "PCG::ArrayOp::DataDomain::Contains", "[PCG][MetadataArrayOp][DataDomain]")
{
	ConfigureOnDataDomain(EPCGMetadataArrayOperation::Contains);
	AddInput(0, PCGMetadataArrayOpTests::MakeArrayParamOnDataDomain<int32>({10, 20, 30}, ArrayAttrName));
	AddInput(1, PCGMetadataArrayOpTests::MakeValueParamOnDataDomain<int32>(20, ValueAttrName));

	ExecuteElement();

	CHECK(NumErrors == 0);
	ExpectOutputValueOnDataDomain<bool>(true);
}

TEST_CASE_METHOD(FPCGMetadataArrayOpDataDomainTest, "PCG::ArrayOp::DataDomain::RemoveAtIndex", "[PCG][MetadataArrayOp][DataDomain]")
{
	ConfigureOnDataDomain(EPCGMetadataArrayOperation::RemoveAtIndex);
	AddInput(0, PCGMetadataArrayOpTests::MakeArrayParamOnDataDomain<int32>({10, 20, 30, 40}, ArrayAttrName));
	AddInput(1, PCGMetadataArrayOpTests::MakeValueParamOnDataDomain<int32>(1, ValueAttrName));

	ExecuteElement();

	CHECK(NumErrors == 0);
	ExpectOutputArrayOnDataDomain<int32>({10, 30, 40});
}

TEST_CASE_METHOD(FPCGMetadataArrayOpDataDomainTest, "PCG::ArrayOp::DataDomain::Append", "[PCG][MetadataArrayOp][DataDomain]")
{
	ConfigureOnDataDomain(EPCGMetadataArrayOperation::Append);
	AddInput(0, PCGMetadataArrayOpTests::MakeArrayParamOnDataDomain<int32>({1, 2}, ArrayAttrName));
	AddInput(1, PCGMetadataArrayOpTests::MakeArrayParamOnDataDomain<int32>({3, 4}, ValueAttrName));

	ExecuteElement();

	CHECK(NumErrors == 0);
	ExpectOutputArrayOnDataDomain<int32>({1, 2, 3, 4});
}

TEST_CASE_METHOD(FPCGMetadataArrayOpDataDomainTest, "PCG::ArrayOp::DataDomain::Insert", "[PCG][MetadataArrayOp][DataDomain]")
{
	ConfigureOnDataDomain(EPCGMetadataArrayOperation::Insert);
	AddInput(0, PCGMetadataArrayOpTests::MakeArrayParamOnDataDomain<int32>({10, 20, 30}, ArrayAttrName));
	AddInput(1, PCGMetadataArrayOpTests::MakeValueParamOnDataDomain<int32>(99, ValueAttrName));
	AddInput(2, PCGMetadataArrayOpTests::MakeValueParamOnDataDomain<int32>(1, IndexAttrName));

	ExecuteElement();

	CHECK(NumErrors == 0);
	ExpectOutputArrayOnDataDomain<int32>({10, 99, 20, 30});
}

TEST_CASE_METHOD(FPCGMetadataArrayOpDataDomainTest, "PCG::ArrayOp::DataDomain::ReplaceAtIndex", "[PCG][MetadataArrayOp][DataDomain]")
{
	ConfigureOnDataDomain(EPCGMetadataArrayOperation::ReplaceAtIndex);
	AddInput(0, PCGMetadataArrayOpTests::MakeArrayParamOnDataDomain<int32>({10, 20, 30}, ArrayAttrName));
	AddInput(1, PCGMetadataArrayOpTests::MakeValueParamOnDataDomain<int32>(99, ValueAttrName));
	AddInput(2, PCGMetadataArrayOpTests::MakeValueParamOnDataDomain<int32>(1, IndexAttrName));

	ExecuteElement();

	CHECK(NumErrors == 0);
	ExpectOutputArrayOnDataDomain<int32>({10, 99, 30});
}

TEST_CASE_METHOD(FPCGMetadataArrayOpDataDomainTest, "PCG::ArrayOp::DataDomain::ConvertToArray", "[PCG][MetadataArrayOp][DataDomain]")
{
	ConfigureOnDataDomain(EPCGMetadataArrayOperation::ConvertToArray);
	AddInput(0, PCGMetadataArrayOpTests::MakeValueParamOnDataDomain<double>(7.5, ArrayAttrName));

	ExecuteElement();

	CHECK(NumErrors == 0);
	ExpectOutputArrayOnDataDomain<double>({7.5});
}

TEST_CASE_METHOD(FPCGMetadataArrayOpDataDomainTest, "PCG::ArrayOp::DataDomain::Flatten", "[PCG][MetadataArrayOp][DataDomain]")
{
	// Flatten on a single-entry domain is explicitly rejected by SupportsSingleEntryDomains.
	FSuppressErrorsScope Suppress(*this);

	ConfigureOnDataDomain(EPCGMetadataArrayOperation::Flatten);
	AddInput(0, PCGMetadataArrayOpTests::MakeArrayParamOnDataDomain<int32>({1, 2, 3}, ArrayAttrName));

	ExecuteElement();

	ExpectErrors();
}
