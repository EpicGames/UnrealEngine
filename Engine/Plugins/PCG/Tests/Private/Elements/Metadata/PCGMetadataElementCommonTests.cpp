// Copyright Epic Games, Inc. All Rights Reserved.

// =============================================================================
// PCG ApplyOnAccessor / ApplyOnMultiAccessorsRange Tests
//
// Tests for the iteration helpers in PCGMetadataElementCommon that read values
// from one or more accessors and deliver them to callbacks in chunked ranges.
//
// Specifically exercises the aligned raw-buffer path in ApplyOnMultiAccessorsRange
// with types that require alignment > 1 (FVector, FTransform). And non trivial constructible
// types like Strings.
//
// Test summary:
//  - ApplyOnAccessorRange:             Iterates all entries via range callback
//  - ApplyOnAccessor:                  Iterates all entries via per-element callback
//  - ApplyOnAccessorRangeEarlyOut:     Bool callback returning false stops iteration
//  - ApplyOnAccessorRangeEmpty:        Returns false when keys are empty
//  - ApplyOnAccessorRangeSmallChunk:   Multiple chunks with small ChunkSize
//  - ApplyOnMultiAccessorsRangeFloat:  Multi-accessor with single float type
//  - ApplyOnMultiAccessorsRangeAligned: Multi-accessor with FVector + FTransform (alignment > 1)
//  - ApplyOnMultiAccessorsSmallChunk:  Multi-accessor with small ChunkSize forces multiple iterations
//  - ApplyOnMultiAccessorsEarlyOut:    Multi-accessor bool callback early termination
//  - ApplyOnMultiAccessorsEmpty:       Multi-accessor returns false on empty inputs
//  - ApplyOnMultiAccessorsPerKey:      Multi-accessor with one key per accessor
//  - ApplyOnMultiAccessors:            Per-element multi-accessor wrapper
//  - ApplyOnAccessorRangeString:       Single-accessor with FString (non-trivially-copyable)
//  - ApplyOnMultiAccessorsRangeString: Multi-accessor with FString + float (construction/destruction)
// =============================================================================

#include "PCGTestsCommon.h"

#include "Data/PCGPointData.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Metadata/Accessors/PCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include <catch2/catch_test_macros.hpp>

#include "TestHarness.h"

namespace PCGApplyOnAccessorTests
{
	constexpr int32 NumEntries = 8;

	struct AccessorData
	{
		static FString MakeTestString(int32 Index)
		{
			return FString::Printf(TEXT("Entry_%d"), Index);
		}

		explicit AccessorData(UPCGBasePointData* InPointData)
			: PointData(InPointData)
		{
			const FName FloatName = TEXT("FloatAttribute");
			const FName VectorName = TEXT("VectorAttribute");
			const FName TransformName = TEXT("TransformAttribute");
			const FName StringName = TEXT("StringAttribute");

			FloatAttribute = PointData->MutableMetadata()->CreateAttribute<float>(FloatName, 0.0f, true, false);
			VectorAttribute = PointData->MutableMetadata()->CreateAttribute<FVector>(VectorName, FVector::ZeroVector, false, false);
			TransformAttribute = PointData->MutableMetadata()->CreateAttribute<FTransform>(TransformName, FTransform::Identity, false, false);
			StringAttribute = PointData->MutableMetadata()->CreateAttribute<FString>(StringName, FString{}, false, false);
			
			PointData->SetNumPoints(NumEntries);
			PointData->AllocateProperties(EPCGPointNativeProperties::MetadataEntry);
			
			TPCGValueRange<PCGMetadataEntryKey> EntryKeyRange = PointData->GetMetadataEntryValueRange();

			for (int32 i = 0; i < NumEntries; ++i)
			{
				PointData->MutableMetadata()->InitializeOnSet(EntryKeyRange[i]);
				FloatAttribute->SetValue(EntryKeyRange[i], static_cast<float>(i) + 0.5f);
				VectorAttribute->SetValue(EntryKeyRange[i], FVector(static_cast<double>(i), static_cast<double>(i) * 2.0, static_cast<double>(i) * 3.0));
				TransformAttribute->SetValue(EntryKeyRange[i], FTransform(FVector(static_cast<double>(i) * 10.0)));
				StringAttribute->SetValue(EntryKeyRange[i], MakeTestString(i));
			}

			FloatAccessor = PCGAttributeAccessorHelpers::CreateAccessor(FloatAttribute, PointData->MutableMetadata());
			VectorAccessor = PCGAttributeAccessorHelpers::CreateAccessor(VectorAttribute, PointData->MutableMetadata());
			TransformAccessor = PCGAttributeAccessorHelpers::CreateAccessor(TransformAttribute, PointData->MutableMetadata());
			StringAccessor = PCGAttributeAccessorHelpers::CreateAccessor(StringAttribute, PointData->MutableMetadata());

			FloatKeys = PCGAttributeAccessorHelpers::CreateKeys(PointData, FPCGAttributePropertySelector::CreateAttributeSelector(FloatName));
			VectorKeys = PCGAttributeAccessorHelpers::CreateKeys(PointData, FPCGAttributePropertySelector::CreateAttributeSelector(VectorName));
			TransformKeys = PCGAttributeAccessorHelpers::CreateKeys(PointData, FPCGAttributePropertySelector::CreateAttributeSelector(TransformName));
			StringKeys = PCGAttributeAccessorHelpers::CreateKeys(PointData, FPCGAttributePropertySelector::CreateAttributeSelector(StringName));
		}

		TObjectPtr<UPCGBasePointData> PointData;
		FPCGMetadataAttribute<float>* FloatAttribute = nullptr;
		FPCGMetadataAttribute<FVector>* VectorAttribute = nullptr;
		FPCGMetadataAttribute<FTransform>* TransformAttribute = nullptr;
		FPCGMetadataAttribute<FString>* StringAttribute = nullptr;

		TUniquePtr<IPCGAttributeAccessor> FloatAccessor;
		TUniquePtr<IPCGAttributeAccessor> VectorAccessor;
		TUniquePtr<IPCGAttributeAccessor> TransformAccessor;
		TUniquePtr<IPCGAttributeAccessor> StringAccessor;

		TUniquePtr<IPCGAttributeAccessorKeys> FloatKeys;
		TUniquePtr<IPCGAttributeAccessorKeys> VectorKeys;
		TUniquePtr<IPCGAttributeAccessorKeys> TransformKeys;
		TUniquePtr<IPCGAttributeAccessorKeys> StringKeys;
	};
}

// Tests that ApplyOnAccessorRange iterates all entries and the callback receives correct values.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::ApplyOnAccessorRange", "[PCG][Accessor][ApplyOn]")
{
	using namespace PCGApplyOnAccessorTests;

	AccessorData Data(CreatePointData());

	REQUIRE_EQUAL(Data.FloatKeys->GetNum(), NumEntries);

	TArray<float> Gathered;
	Gathered.Reserve(NumEntries);

	bool bResult = PCGMetadataElementCommon::ApplyOnAccessorRange<float>(*Data.FloatKeys, *Data.FloatAccessor,
		[&Gathered](const TArrayView<float>& View, int32 Start, int32 Range)
		{
			for (int32 i = 0; i < Range; ++i)
			{
				Gathered.Add(View[i]);
			}
		});

	REQUIRE(bResult);
	REQUIRE_EQUAL(Gathered.Num(), NumEntries);

	for (int32 i = 0; i < NumEntries; ++i)
	{
		REQUIRE_EQUAL(Gathered[i], static_cast<float>(i) + 0.5f);
	}
}

// Tests that ApplyOnAccessor iterates per-element with correct values and indices.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::ApplyOnAccessor", "[PCG][Accessor][ApplyOn]")
{
	using namespace PCGApplyOnAccessorTests;

	AccessorData Data(CreatePointData());

	TArray<float> GatheredValues;
	TArray<int32> GatheredIndices;
	GatheredValues.Reserve(NumEntries);
	GatheredIndices.Reserve(NumEntries);

	bool bResult = PCGMetadataElementCommon::ApplyOnAccessor<float>(*Data.FloatKeys, *Data.FloatAccessor,
		[&GatheredValues, &GatheredIndices](const float& Value, int32 Index)
		{
			GatheredValues.Add(Value);
			GatheredIndices.Add(Index);
		});

	REQUIRE(bResult);
	REQUIRE_EQUAL(GatheredValues.Num(), NumEntries);

	for (int32 i = 0; i < NumEntries; ++i)
	{
		REQUIRE_EQUAL(GatheredValues[i], static_cast<float>(i) + 0.5f);
		REQUIRE_EQUAL(GatheredIndices[i], i);
	}
}

// Tests that a bool-returning callback can stop iteration early.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::ApplyOnAccessorRangeEarlyOut", "[PCG][Accessor][ApplyOn]")
{
	using namespace PCGApplyOnAccessorTests;

	AccessorData Data(CreatePointData());

	int32 CallbackCount = 0;
	constexpr int32 SmallChunk = 2;

	bool bResult = PCGMetadataElementCommon::ApplyOnAccessorRange<float>(*Data.FloatKeys, *Data.FloatAccessor,
		[&CallbackCount](const TArrayView<float>& View, int32 Start, int32 Range) -> bool
		{
			++CallbackCount;
			return false; // Stop after first chunk
		},
		EPCGAttributeAccessorFlags::StrictType,
		SmallChunk);

	REQUIRE(bResult);
	REQUIRE_EQUAL(CallbackCount, 1);
}

// Tests that ApplyOnAccessorRange returns false when keys are empty.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::ApplyOnAccessorRangeEmpty", "[PCG][Accessor][ApplyOn]")
{
	using namespace PCGApplyOnAccessorTests;

	AccessorData Data(CreatePointData());

	// Create keys with no entries
	FPCGAttributeAccessorKeysEntries EmptyKeys(TArrayView<PCGMetadataEntryKey>{});

	REQUIRE_EQUAL(EmptyKeys.GetNum(), 0);

	bool bCallbackCalled = false;
	bool bResult = PCGMetadataElementCommon::ApplyOnAccessorRange<float>(EmptyKeys, *Data.FloatAccessor,
		[&bCallbackCalled](const TArrayView<float>& View, int32 Start, int32 Range)
		{
			bCallbackCalled = true;
		});

	REQUIRE_FALSE(bResult);
	REQUIRE_FALSE(bCallbackCalled);
}

// Tests that a small ChunkSize causes multiple iterations that together cover all entries.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::ApplyOnAccessorRangeSmallChunk", "[PCG][Accessor][ApplyOn]")
{
	using namespace PCGApplyOnAccessorTests;

	AccessorData Data(CreatePointData());

	constexpr int32 SmallChunk = 3;
	int32 ChunkCallCount = 0;
	TArray<float> Gathered;
	Gathered.Reserve(NumEntries);

	bool bResult = PCGMetadataElementCommon::ApplyOnAccessorRange<float>(*Data.FloatKeys, *Data.FloatAccessor,
		[&ChunkCallCount, &Gathered](const TArrayView<float>& View, int32 Start, int32 Range)
		{
			++ChunkCallCount;
			for (int32 i = 0; i < Range; ++i)
			{
				Gathered.Add(View[i]);
			}
		},
		EPCGAttributeAccessorFlags::StrictType,
		SmallChunk);

	REQUIRE(bResult);
	// 8 entries / chunk size 3 = ceil(8/3) = 3 chunks
	REQUIRE_EQUAL(ChunkCallCount, (NumEntries + SmallChunk - 1) / SmallChunk);
	REQUIRE_EQUAL(Gathered.Num(), NumEntries);

	for (int32 i = 0; i < NumEntries; ++i)
	{
		REQUIRE_EQUAL(Gathered[i], static_cast<float>(i) + 0.5f);
	}
}

// Tests ApplyOnMultiAccessorsRange with a single float type to verify basic multi-accessor functionality.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::ApplyOnMultiAccessorsRangeFloat", "[PCG][Accessor][ApplyOn]")
{
	using namespace PCGApplyOnAccessorTests;

	AccessorData Data(CreatePointData());

	TArray<float> GatheredFloats;
	GatheredFloats.Reserve(NumEntries);

	const IPCGAttributeAccessor* Accessors[] = { Data.FloatAccessor.Get() };

	bool bResult = PCGMetadataElementCommon::ApplyOnMultiAccessorsRange<float>(*Data.FloatKeys, Accessors,
		[&GatheredFloats](const TArrayView<float>& FloatView, int32 Start, int32 Range)
		{
			for (int32 i = 0; i < Range; ++i)
			{
				GatheredFloats.Add(FloatView[i]);
			}
		});

	REQUIRE(bResult);
	REQUIRE_EQUAL(GatheredFloats.Num(), NumEntries);

	for (int32 i = 0; i < NumEntries; ++i)
	{
		REQUIRE_EQUAL(GatheredFloats[i], static_cast<float>(i) + 0.5f);
	}
}

// Tests ApplyOnMultiAccessorsRange with FVector and FTransform types, exercising the aligned buffer path.
// This is the key test for the alignment fix: FTransform has alignment 16, which previously caused UB with uint8 buffers.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::ApplyOnMultiAccessorsRangeAligned", "[PCG][Accessor][ApplyOn]")
{
	using namespace PCGApplyOnAccessorTests;

	AccessorData Data(CreatePointData());

	TArray<FVector> GatheredVectors;
	TArray<FTransform> GatheredTransforms;
	GatheredVectors.Reserve(NumEntries);
	GatheredTransforms.Reserve(NumEntries);

	const IPCGAttributeAccessor* Accessors[] = { Data.VectorAccessor.Get(), Data.TransformAccessor.Get() };

	bool bResult = PCGMetadataElementCommon::ApplyOnMultiAccessorsRange<FVector, FTransform>(*Data.FloatKeys, Accessors,
		[&GatheredVectors, &GatheredTransforms](const TArrayView<FVector>& VectorView, const TArrayView<FTransform>& TransformView, int32 Start, int32 Range)
		{
			for (int32 i = 0; i < Range; ++i)
			{
				GatheredVectors.Add(VectorView[i]);
				GatheredTransforms.Add(TransformView[i]);
			}
		});

	REQUIRE(bResult);
	REQUIRE_EQUAL(GatheredVectors.Num(), NumEntries);
	REQUIRE_EQUAL(GatheredTransforms.Num(), NumEntries);

	for (int32 i = 0; i < NumEntries; ++i)
	{
		const double D = static_cast<double>(i);
		REQUIRE_EQUAL(GatheredVectors[i], FVector(D, D * 2.0, D * 3.0));
		REQUIRE_EQUAL(GatheredTransforms[i].GetTranslation(), FVector(D * 10.0));
	}
}

// Tests ApplyOnMultiAccessorsRange with a small ChunkSize to force multiple iterations through the aligned buffer path.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::ApplyOnMultiAccessorsSmallChunk", "[PCG][Accessor][ApplyOn]")
{
	using namespace PCGApplyOnAccessorTests;

	AccessorData Data(CreatePointData());

	constexpr int32 SmallChunk = 3;
	int32 ChunkCallCount = 0;
	TArray<float> GatheredFloats;
	TArray<FVector> GatheredVectors;
	GatheredFloats.Reserve(NumEntries);
	GatheredVectors.Reserve(NumEntries);

	const IPCGAttributeAccessor* Accessors[] = { Data.FloatAccessor.Get(), Data.VectorAccessor.Get() };

	bool bResult = PCGMetadataElementCommon::ApplyOnMultiAccessorsRange<float, FVector>(*Data.FloatKeys, Accessors,
		[&ChunkCallCount, &GatheredFloats, &GatheredVectors](const TArrayView<float>& FloatView, const TArrayView<FVector>& VectorView, int32 Start, int32 Range)
		{
			++ChunkCallCount;
			for (int32 i = 0; i < Range; ++i)
			{
				GatheredFloats.Add(FloatView[i]);
				GatheredVectors.Add(VectorView[i]);
			}
		},
		EPCGAttributeAccessorFlags::StrictType,
		SmallChunk);

	REQUIRE(bResult);
	REQUIRE_EQUAL(ChunkCallCount, (NumEntries + SmallChunk - 1) / SmallChunk);
	REQUIRE_EQUAL(GatheredFloats.Num(), NumEntries);
	REQUIRE_EQUAL(GatheredVectors.Num(), NumEntries);

	for (int32 i = 0; i < NumEntries; ++i)
	{
		REQUIRE_EQUAL(GatheredFloats[i], static_cast<float>(i) + 0.5f);
		const double D = static_cast<double>(i);
		REQUIRE_EQUAL(GatheredVectors[i], FVector(D, D * 2.0, D * 3.0));
	}
}

// Tests that a bool-returning callback can stop iteration early in ApplyOnMultiAccessorsRange.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::ApplyOnMultiAccessorsEarlyOut", "[PCG][Accessor][ApplyOn]")
{
	using namespace PCGApplyOnAccessorTests;

	AccessorData Data(CreatePointData());

	constexpr int32 SmallChunk = 2;
	int32 CallbackCount = 0;

	const IPCGAttributeAccessor* Accessors[] = { Data.FloatAccessor.Get(), Data.VectorAccessor.Get() };

	bool bResult = PCGMetadataElementCommon::ApplyOnMultiAccessorsRange<float, FVector>(*Data.FloatKeys, Accessors,
		[&CallbackCount](const TArrayView<float>& FloatView, const TArrayView<FVector>& VectorView, int32 Start, int32 Range) -> bool
		{
			++CallbackCount;
			return false; // Stop after first chunk
		},
		EPCGAttributeAccessorFlags::StrictType,
		SmallChunk);

	REQUIRE(bResult);
	REQUIRE_EQUAL(CallbackCount, 1);
}

// Tests that ApplyOnMultiAccessorsRange returns false on empty accessors/keys.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::ApplyOnMultiAccessorsEmpty", "[PCG][Accessor][ApplyOn]")
{
	using namespace PCGApplyOnAccessorTests;

	AccessorData Data(CreatePointData());

	SECTION("Empty accessors array")
	{
		TConstArrayView<IPCGAttributeAccessor const*> EmptyAccessors;
		bool bResult = PCGMetadataElementCommon::ApplyOnMultiAccessorsRange<float>(*Data.FloatKeys, EmptyAccessors,
			[](const TArrayView<float>& View, int32 Start, int32 Range) {});
		REQUIRE_FALSE(bResult);
	}

	SECTION("Empty keys array")
	{
		const IPCGAttributeAccessor* Accessors[] = { Data.FloatAccessor.Get() };
		TConstArrayView<IPCGAttributeAccessorKeys const*> EmptyKeys;
		bool bResult = PCGMetadataElementCommon::ApplyOnMultiAccessorsRange<float>(EmptyKeys, Accessors,
			[](const TArrayView<float>& View, int32 Start, int32 Range) {});
		REQUIRE_FALSE(bResult);
	}
}

// Tests ApplyOnMultiAccessorsRange with one key per accessor (multi-key mode).
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::ApplyOnMultiAccessorsPerKey", "[PCG][Accessor][ApplyOn]")
{
	using namespace PCGApplyOnAccessorTests;

	AccessorData Data(CreatePointData());
	
	const IPCGAttributeAccessorKeys* MultiKeys[] = { Data.FloatKeys.Get(), Data.VectorKeys.Get() };
	const IPCGAttributeAccessor* Accessors[] = { Data.FloatAccessor.Get(), Data.VectorAccessor.Get() };

	TArray<float> GatheredFloats;
	TArray<FVector> GatheredVectors;
	GatheredFloats.Reserve(NumEntries);
	GatheredVectors.Reserve(NumEntries);

	bool bResult = PCGMetadataElementCommon::ApplyOnMultiAccessorsRange<float, FVector>(MultiKeys, Accessors,
		[&GatheredFloats, &GatheredVectors](const TArrayView<float>& FloatView, const TArrayView<FVector>& VectorView, int32 Start, int32 Range)
		{
			for (int32 i = 0; i < Range; ++i)
			{
				GatheredFloats.Add(FloatView[i]);
				GatheredVectors.Add(VectorView[i]);
			}
		});

	REQUIRE(bResult);
	REQUIRE_EQUAL(GatheredFloats.Num(), NumEntries);

	for (int32 i = 0; i < NumEntries; ++i)
	{
		REQUIRE_EQUAL(GatheredFloats[i], static_cast<float>(i) + 0.5f);
		const double D = static_cast<double>(i);
		REQUIRE_EQUAL(GatheredVectors[i], FVector(D, D * 2.0, D * 3.0));
	}
}

// Tests ApplyOnMultiAccessors per-element wrapper with multiple types including FTransform (alignment-sensitive).
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::ApplyOnMultiAccessors", "[PCG][Accessor][ApplyOn]")
{
	using namespace PCGApplyOnAccessorTests;

	AccessorData Data(CreatePointData());

	TArray<float> GatheredFloats;
	TArray<FTransform> GatheredTransforms;
	GatheredFloats.Reserve(NumEntries);
	GatheredTransforms.Reserve(NumEntries);

	const IPCGAttributeAccessor* Accessors[] = { Data.FloatAccessor.Get(), Data.TransformAccessor.Get() };

	bool bResult = PCGMetadataElementCommon::ApplyOnMultiAccessors<float, FTransform>(*Data.FloatKeys, Accessors,
		[&GatheredFloats, &GatheredTransforms](const float& FloatVal, const FTransform& TransformVal, int32 Index)
		{
			GatheredFloats.Add(FloatVal);
			GatheredTransforms.Add(TransformVal);
		});

	REQUIRE(bResult);
	REQUIRE_EQUAL(GatheredFloats.Num(), NumEntries);
	REQUIRE_EQUAL(GatheredTransforms.Num(), NumEntries);

	for (int32 i = 0; i < NumEntries; ++i)
	{
		REQUIRE_EQUAL(GatheredFloats[i], static_cast<float>(i) + 0.5f);
		REQUIRE_EQUAL(GatheredTransforms[i].GetTranslation(), FVector(static_cast<double>(i) * 10.0));
	}
}

// Tests ApplyOnAccessorRange with FString, which is non-trivially-copyable and requires construction/destruction in the raw buffer.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::ApplyOnAccessorRangeString", "[PCG][Accessor][ApplyOn]")
{
	using namespace PCGApplyOnAccessorTests;

	AccessorData Data(CreatePointData());

	TArray<FString> Gathered;
	Gathered.Reserve(NumEntries);

	bool bResult = PCGMetadataElementCommon::ApplyOnAccessorRange<FString>(*Data.StringKeys, *Data.StringAccessor,
		[&Gathered](const TArrayView<FString>& View, int32 Start, int32 Range)
		{
			for (int32 i = 0; i < Range; ++i)
			{
				Gathered.Add(View[i]);
			}
		});

	REQUIRE(bResult);
	REQUIRE_EQUAL(Gathered.Num(), NumEntries);

	for (int32 i = 0; i < NumEntries; ++i)
	{
		REQUIRE_EQUAL(Gathered[i], AccessorData::MakeTestString(i));
	}
}

// Tests ApplyOnMultiAccessorsRange with FString + float, exercising the non-trivially-copyable buffer path
// alongside a trivially-copyable type. This verifies that placement new and explicit destructor calls
// work correctly when mixed types share the same chunked iteration.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::ApplyOnMultiAccessorsRangeString", "[PCG][Accessor][ApplyOn]")
{
	using namespace PCGApplyOnAccessorTests;

	AccessorData Data(CreatePointData());

	constexpr int32 SmallChunk = 3;
	TArray<FString> GatheredStrings;
	TArray<float> GatheredFloats;
	GatheredStrings.Reserve(NumEntries);
	GatheredFloats.Reserve(NumEntries);

	const IPCGAttributeAccessor* Accessors[] = { Data.StringAccessor.Get(), Data.FloatAccessor.Get() };

	bool bResult = PCGMetadataElementCommon::ApplyOnMultiAccessorsRange<FString, float>(*Data.StringKeys, Accessors,
		[&GatheredStrings, &GatheredFloats](const TArrayView<FString>& StringView, const TArrayView<float>& FloatView, int32 Start, int32 Range)
		{
			for (int32 i = 0; i < Range; ++i)
			{
				GatheredStrings.Add(StringView[i]);
				GatheredFloats.Add(FloatView[i]);
			}
		},
		EPCGAttributeAccessorFlags::StrictType,
		SmallChunk);

	REQUIRE(bResult);
	REQUIRE_EQUAL(GatheredStrings.Num(), NumEntries);
	REQUIRE_EQUAL(GatheredFloats.Num(), NumEntries);

	for (int32 i = 0; i < NumEntries; ++i)
	{
		REQUIRE_EQUAL(GatheredStrings[i], AccessorData::MakeTestString(i));
		REQUIRE_EQUAL(GatheredFloats[i], static_cast<float>(i) + 0.5f);
	}
}
