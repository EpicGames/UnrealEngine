// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGTestsCommon.h"

#include "Data/PCGBasePointData.h"
#include "Data/PCGPointArrayData.h"
#include "Elements/PCGAlignPoints.h"

#include <catch2/catch_test_macros.hpp>

#include "TestHarness.h"

namespace
{
	const FVector SrcBoundsMin(-50.f, -40.f, -30.f);
	const FVector SrcBoundsMax( 50.f,  40.f,  30.f);
	const FVector TgtBoundsMin(-30.f, -20.f, -10.f);
	const FVector TgtBoundsMax( 60.f,  70.f,  80.f);
	const FVector SrcPos( 100.f, 200.f,  50.f);
	const FVector TgtPos( -50.f, 100.f, 200.f);
	constexpr float PosTolerance = 1.0e-2f;
} // anonymous namespace

// ---------------------------------------------------------------------------
// Test geometry (chosen so expected values can be hand-verified):
//
//   Source local bounds:  Min=(-50,-40,-30)  Max=(50,40,30)   LocalCenter=(0,0,0)
//   Target local bounds:  Min=(-30,-20,-10)  Max=(60,70,80)   LocalCenter=(15,25,35)
//   Source position:  (100, 200, 50)
//   Target position:  (-50, 100, 200)
//
// World-space AABBs for identity transforms:
//   Src world: Min=(50,160,20)   Max=(150,240,80)  Center=(100,200,50)
//   Tgt world: Min=(-80,80,190)  Max=(10,170,280)  Center=(-35,125,235)
// ---------------------------------------------------------------------------

class FPCGAlignPointsBaseTest : public PCGTests::FPCGSingleElementBaseTest<UPCGAlignPointsSettings>
{
public:
	FPCGAlignPointsBaseTest() : FPCGSingleElementBaseTest() {}

protected:
	using ERef     = EPCGAlignPointsAxisReferential;
	using ESpatial = EPCGAlignPointsSpatialReferential;

	UPCGPointArrayData* MakePointData(const FTransform& Transform, const FVector& BoundsMin, const FVector& BoundsMax)
	{
		UPCGPointArrayData* PD = NewObject<UPCGPointArrayData>();
		PD->SetNumPoints(1);
		FPCGPointValueRanges Ranges(PD);
		Ranges.TransformRange[0] = Transform;
		Ranges.BoundsMinRange[0] = BoundsMin;
		Ranges.BoundsMaxRange[0] = BoundsMax;
		return PD;
	}

	UPCGPointArrayData* MakePointData(const FVector& Position, const FVector& BoundsMin, const FVector& BoundsMax)
	{
		return MakePointData(FTransform(Position), BoundsMin, BoundsMax);
	}

	FPCGTaggedData MakeSourceEntry(UPCGPointArrayData* Data)
	{
		FPCGTaggedData Entry;
		Entry.Data = Data;
		Entry.Pin = PCGAlignPointsConstants::SourcePointsLabel;
		return Entry;
	}

	FPCGTaggedData MakeTargetEntry(UPCGPointArrayData* Data)
	{
		FPCGTaggedData Entry;
		Entry.Data = Data;
		Entry.Pin = PCGAlignPointsConstants::TargetPointsLabel;
		return Entry;
	}

	void SetAxis(bool bX, ERef SrcX, ERef TgtX,
	             bool bY, ERef SrcY, ERef TgtY,
	             bool bZ, ERef SrcZ, ERef TgtZ)
	{
		TypedSettings->XAxis = { bX, SrcX, TgtX };
		TypedSettings->YAxis = { bY, SrcY, TgtY };
		TypedSettings->ZAxis = { bZ, SrcZ, TgtZ };
	}

	TArray<FTransform> RunAlignPointsTransforms(const TArray<FPCGTaggedData>& SourceEntries, const TArray<FPCGTaggedData>& TargetEntries)
	{
		for (const FPCGTaggedData& Entry : SourceEntries)
		{
			InputData.TaggedData.Add(Entry);
		}

		for (const FPCGTaggedData& Entry : TargetEntries)
		{
			InputData.TaggedData.Add(Entry);
		}

		ExecuteElement();

		TArray<FTransform> OutTransforms;
		for (const FPCGTaggedData& Output : Context->OutputData.TaggedData)
		{
			const UPCGBasePointData* PointData = Cast<UPCGBasePointData>(Output.Data);
			if (!PointData)
			{
				continue;
			}

			TConstPCGValueRange<FTransform> TransformRange = PointData->GetConstTransformValueRange();
			for (int32 i = 0; i < PointData->GetNumPoints(); ++i)
			{
				OutTransforms.Add(TransformRange[i]);
			}
		}
		return OutTransforms;
	}

	TArray<FVector> RunAlignPoints(const TArray<FPCGTaggedData>& SourceEntries, const TArray<FPCGTaggedData>& TargetEntries)
	{
		TArray<FVector> OutPositions;
		for (const FTransform& T : RunAlignPointsTransforms(SourceEntries, TargetEntries))
		{
			OutPositions.Add(T.GetLocation());
		}

		return OutPositions;
	}

	FVector Run1v1(const FTransform& Src, const FTransform& Tgt)
	{
		const TArray<FVector> Out = RunAlignPoints(
			{ MakeSourceEntry(MakePointData(Src, SrcBoundsMin, SrcBoundsMax)) },
			{ MakeTargetEntry(MakePointData(Tgt, TgtBoundsMin, TgtBoundsMax)) });
		return (Out.Num() == 1) ? Out[0] : FVector(UE_BIG_NUMBER);
	}

	FTransform Run1v1T(const FTransform& Src, const FTransform& Tgt)
	{
		const TArray<FTransform> Out = RunAlignPointsTransforms(
			{ MakeSourceEntry(MakePointData(Src, SrcBoundsMin, SrcBoundsMax)) },
			{ MakeTargetEntry(MakePointData(Tgt, TgtBoundsMin, TgtBoundsMax)) });
		return (Out.Num() == 1) ? Out[0] : FTransform::Identity;
	}
};

// =========================================================================
// World referential — identity transforms, all expected values hand-computed
//
// Src world AABB: Min=(50,160,20)  Max=(150,240,80)  Center=(100,200,50)
// Tgt world AABB: Min=(-80,80,190) Max=(10,170,280)  Center=(-35,125,235)
// =========================================================================

TEST_CASE_METHOD(FPCGAlignPointsBaseTest, "PCG::AlignPoints::World", "[PCG][AlignPoints]")
{
	TypedSettings->SpatialReferential = ESpatial::World;
	const FTransform SrcIdentity(FQuat::Identity, SrcPos, FVector::OneVector);
	const FTransform TgtIdentity(FQuat::Identity, TgtPos, FVector::OneVector);

	SECTION("NoOp")
	{
		// No axes enabled: source must pass through unchanged.
		SetAxis(false, ERef::Center, ERef::Center,
		        false, ERef::Center, ERef::Center,
		        false, ERef::Center, ERef::Center);
		const FVector Out = Run1v1(SrcIdentity, TgtIdentity);
		CHECK(Out.Equals(SrcPos, PosTolerance));
	}

	SECTION("AllCenter")
	{
		// Delta = TgtWorldCenter - SrcWorldCenter = (-135,-75,185)
		// OutPos = (100,200,50) + (-135,-75,185) = (-35,125,235)
		SetAxis(true, ERef::Center, ERef::Center,
		        true, ERef::Center, ERef::Center,
		        true, ERef::Center, ERef::Center);
		const FVector Out = Run1v1(SrcIdentity, TgtIdentity);
		CHECK(Out.Equals(FVector(-35.f, 125.f, 235.f), PosTolerance));
	}

	SECTION("AllMin")
	{
		// Delta = TgtWorldMin - SrcWorldMin = (-80-50, 80-160, 190-20) = (-130,-80,170)
		// OutPos = (100,200,50) + (-130,-80,170) = (-30,120,220)
		SetAxis(true, ERef::Min, ERef::Min,
		        true, ERef::Min, ERef::Min,
		        true, ERef::Min, ERef::Min);
		const FVector Out = Run1v1(SrcIdentity, TgtIdentity);
		CHECK(Out.Equals(FVector(-30.f, 120.f, 220.f), PosTolerance));
	}

	SECTION("AllMax")
	{
		// Delta = TgtWorldMax - SrcWorldMax = (10-150, 170-240, 280-80) = (-140,-70,200)
		// OutPos = (100,200,50) + (-140,-70,200) = (-40,130,250)
		SetAxis(true, ERef::Max, ERef::Max,
		        true, ERef::Max, ERef::Max,
		        true, ERef::Max, ERef::Max);
		const FVector Out = Run1v1(SrcIdentity, TgtIdentity);
		CHECK(Out.Equals(FVector(-40.f, 130.f, 250.f), PosTolerance));
	}

	SECTION("XOnly")
	{
		// Delta.X = TgtWorldCenter.X - SrcWorldCenter.X = -35 - 100 = -135
		// OutPos = (-35, 200, 50)
		SetAxis(true,  ERef::Center, ERef::Center,
		        false, ERef::Center, ERef::Center,
		        false, ERef::Center, ERef::Center);
		const FVector Out = Run1v1(SrcIdentity, TgtIdentity);
		CHECK(Out.Equals(FVector(-35.f, 200.f, 50.f), PosTolerance));
	}

	SECTION("ZOnly")
	{
		// Delta.Z = TgtWorldMin.Z - SrcWorldMin.Z = 190 - 20 = 170
		// OutPos = (100, 200, 220)
		SetAxis(false, ERef::Center, ERef::Center,
		        false, ERef::Center, ERef::Center,
		        true,  ERef::Min, ERef::Min);
		const FVector Out = Run1v1(SrcIdentity, TgtIdentity);
		CHECK(Out.Equals(FVector(100.f, 200.f, 220.f), PosTolerance));
	}

	SECTION("YMaxToMin")
	{
		// SrcWorldMax.Y = 240, TgtWorldMin.Y = 80
		// Delta.Y = 80 - 240 = -160 → OutPos = (100, 40, 50)
		SetAxis(false, ERef::Center, ERef::Center,
		        true,  ERef::Max, ERef::Min,
		        false, ERef::Center, ERef::Center);
		const FVector Out = Run1v1(SrcIdentity, TgtIdentity);
		CHECK(Out.Equals(FVector(100.f, 40.f, 50.f), PosTolerance));
	}

	SECTION("Mixed")
	{
		// X: center-to-min  → Delta.X = TgtWorldMin.X - SrcWorldCenter.X = -80 - 100 = -180
		// Y: min-to-max     → Delta.Y = TgtWorldMax.Y - SrcWorldMin.Y   = 170 - 160 = 10
		// Z: max-to-center  → Delta.Z = TgtWorldCenter.Z - SrcWorldMax.Z = 235 - 80  = 155
		// OutPos = (100-180, 200+10, 50+155) = (-80, 210, 205)
		SetAxis(true, ERef::Center, ERef::Min,
		        true, ERef::Min,    ERef::Max,
		        true, ERef::Max,    ERef::Center);
		const FVector Out = Run1v1(SrcIdentity, TgtIdentity);
		CHECK(Out.Equals(FVector(-80.f, 210.f, 205.f), PosTolerance));
	}
}

// =========================================================================
// Target referential — target has 90° yaw, source is identity
//
// In UE: yaw=90 maps +X→+Y, +Y→-X.
// After computing source bounds in target-local space, center-to-center gives
// OutPos = (-75, 115, 235).
// =========================================================================

TEST_CASE_METHOD(FPCGAlignPointsBaseTest, "PCG::AlignPoints::Target", "[PCG][AlignPoints]")
{
	TypedSettings->SpatialReferential = ESpatial::Target;
	const FTransform SrcIdentity(FQuat::Identity, SrcPos, FVector::OneVector);

	SECTION("AllCenter_TgtYaw90")
	{
		const FTransform TgtYaw90(FRotator(0.f, 90.f, 0.f), TgtPos, FVector::OneVector);
		SetAxis(true, ERef::Center, ERef::Center,
		        true, ERef::Center, ERef::Center,
		        true, ERef::Center, ERef::Center);
		const FVector Out = Run1v1(SrcIdentity, TgtYaw90);
		CHECK(Out.Equals(FVector(-75.f, 115.f, 235.f), PosTolerance));
	}
}

// =========================================================================
// Source referential — source has 90° yaw, target is identity
//
// Center-to-center DeltaInRef=(-75,135,185), WorldDelta=yaw90*(-75,135,185)=(-135,-75,185)
// OutPos = (-35, 125, 235).
//
// Min-to-min: TargetBounds in source-local Min=(-120,90,140),
// DeltaInRef=(-70,130,170), WorldDelta=yaw90*(-70,130,170)=(-130,-70,170)
// OutPos = (-30, 130, 220).
// =========================================================================

TEST_CASE_METHOD(FPCGAlignPointsBaseTest, "PCG::AlignPoints::Source", "[PCG][AlignPoints]")
{
	TypedSettings->SpatialReferential = ESpatial::Source;
	const FTransform TgtIdentity(FQuat::Identity, TgtPos, FVector::OneVector);
	const FTransform SrcYaw90(FRotator(0.f, 90.f, 0.f), SrcPos, FVector::OneVector);

	SECTION("AllCenter_SrcYaw90")
	{
		SetAxis(true, ERef::Center, ERef::Center,
		        true, ERef::Center, ERef::Center,
		        true, ERef::Center, ERef::Center);
		const FVector Out = Run1v1(SrcYaw90, TgtIdentity);
		CHECK(Out.Equals(FVector(-35.f, 125.f, 235.f), PosTolerance));
	}

	SECTION("AllMin_SrcYaw90")
	{
		SetAxis(true, ERef::Min, ERef::Min,
		        true, ERef::Min, ERef::Min,
		        true, ERef::Min, ERef::Min);
		const FVector Out = Run1v1(SrcYaw90, TgtIdentity);
		CHECK(Out.Equals(FVector(-30.f, 130.f, 220.f), PosTolerance));
	}
}

// =========================================================================
// Pairing: N:1 and N:N
// World referential, all axes, center-to-center.
// =========================================================================

TEST_CASE_METHOD(FPCGAlignPointsBaseTest, "PCG::AlignPoints::Pairing", "[PCG][AlignPoints]")
{
	TypedSettings->SpatialReferential = ESpatial::World;
	SetAxis(true, ERef::Center, ERef::Center,
	        true, ERef::Center, ERef::Center,
	        true, ERef::Center, ERef::Center);

	const FTransform SrcIdentity(FQuat::Identity, SrcPos, FVector::OneVector);
	const FTransform TgtIdentity(FQuat::Identity, TgtPos, FVector::OneVector);

	SECTION("N_to_1")
	{
		// Two source points both aligned to the single target center (-35,125,235).
		// Src0=(100,200,50), Src1=(0,0,0), both with same bounds.
		UPCGPointArrayData* Src2Pts = NewObject<UPCGPointArrayData>();
		Src2Pts->SetNumPoints(2);
		{
			FPCGPointValueRanges R(Src2Pts);
			R.TransformRange[0] = SrcIdentity;
			R.BoundsMinRange[0] = SrcBoundsMin;
			R.BoundsMaxRange[0] = SrcBoundsMax;
			R.TransformRange[1] = FTransform::Identity;
			R.BoundsMinRange[1] = SrcBoundsMin;
			R.BoundsMaxRange[1] = SrcBoundsMax;
		}

		FPCGTaggedData SrcEntry;
		SrcEntry.Data = Src2Pts;
		SrcEntry.Pin  = PCGAlignPointsConstants::SourcePointsLabel;

		const TArray<FVector> Out = RunAlignPoints(
			{ SrcEntry },
			{ MakeTargetEntry(MakePointData(TgtIdentity, TgtBoundsMin, TgtBoundsMax)) });

		REQUIRE_EQUAL(Out.Num(), 2);
		CHECK(Out[0].Equals(FVector(-35.f, 125.f, 235.f), PosTolerance));
		CHECK(Out[1].Equals(FVector(-35.f, 125.f, 235.f), PosTolerance));
	}

	SECTION("N_to_N")
	{
		// Entry 0: Src=(100,200,50)  Tgt=(-50,100,200)  → OutPos=(-35,125,235)
		// Entry 1: Src=(0,0,0)       Tgt=(300,300,300)
		//   Tgt1 world center: (315,325,335) = (300+15, 300+25, 300+35)
		//   OutPos1 = (315,325,335)
		const FVector Src1Pos(0.f, 0.f, 0.f);
		const FVector Tgt1Pos(300.f, 300.f, 300.f);

		const TArray<FVector> Out = RunAlignPoints(
			{ MakeSourceEntry(MakePointData(SrcPos,   SrcBoundsMin, SrcBoundsMax)),
			  MakeSourceEntry(MakePointData(Src1Pos,  SrcBoundsMin, SrcBoundsMax)) },
			{ MakeTargetEntry(MakePointData(TgtPos,   TgtBoundsMin, TgtBoundsMax)),
			  MakeTargetEntry(MakePointData(Tgt1Pos,  TgtBoundsMin, TgtBoundsMax)) });

		REQUIRE_EQUAL(Out.Num(), 2);
		CHECK(Out[0].Equals(FVector(-35.f, 125.f, 235.f), PosTolerance));
		CHECK(Out[1].Equals(FVector(315.f, 325.f, 335.f), PosTolerance));
	}
}

// =========================================================================
// LocalToTarget referential
//
// Behavior: before computing bounds, the source's rotation is replaced by
// the target's rotation. The RelativeTransform then has Identity rotation,
// so source bounds are translated (not rotated) into target-local space.
// The output transform inherits the target's rotation (not the source's).
// =========================================================================

TEST_CASE_METHOD(FPCGAlignPointsBaseTest, "PCG::AlignPoints::LocalToTarget", "[PCG][AlignPoints]")
{
	TypedSettings->SpatialReferential = ESpatial::LocalToTarget;
	SetAxis(true, ERef::Center, ERef::Center,
	        true, ERef::Center, ERef::Center,
	        true, ERef::Center, ERef::Center);

	const FTransform SrcIdentity(FQuat::Identity, SrcPos, FVector::OneVector);
	const FTransform TgtIdentity(FQuat::Identity, TgtPos, FVector::OneVector);
	const FQuat Yaw90 = FQuat(FRotator(0.f, 90.f, 0.f));

	SECTION("BothIdentity")
	{
		// SourcePointTransform rotation → Identity (= TgtRot).
		// RelativeTransform = (Identity, (150,100,-150), 1).
		// Source bounds center in target-local = (150,100,-150). Target center = (15,25,35).
		// DeltaInRef = (-135,-75,185). WorldDelta = (-135,-75,185).
		// OutPos = (-35,125,235). OutRot = Identity.
		const FTransform Out = Run1v1T(SrcIdentity, TgtIdentity);
		CHECK(Out.GetLocation().Equals(FVector(-35.f, 125.f, 235.f), PosTolerance));
		CHECK(Out.GetRotation().Equals(FQuat::Identity, PosTolerance));
	}

	SECTION("SrcYaw90_TgtIdentity")
	{
		// Source rotation overridden to Identity (= TgtRot).
		// Same relative translation as BothIdentity → OutPos = (-35,125,235). OutRot = Identity.
		const FTransform SrcYaw90(Yaw90, SrcPos, FVector::OneVector);
		const FTransform Out = Run1v1T(SrcYaw90, TgtIdentity);
		CHECK(Out.GetLocation().Equals(FVector(-35.f, 125.f, 235.f), PosTolerance));
		CHECK(Out.GetRotation().Equals(FQuat::Identity, PosTolerance));
	}

	SECTION("SrcIdentity_TgtYaw90")
	{
		// Source rotation overridden to yaw90. RelativeTransform rotation = Identity.
		// RelTranslation in tgt-local = yaw=-90 * (SrcPos - TgtPos) = yaw=-90*(150,100,-150) = (100,-150,-150).
		// Source bounds center in tgt-local = (100,-150,-150). Target center = (15,25,35).
		// DeltaInRef = (-85,175,185). WorldDelta = yaw90*(-85,175,185) = (-175,-85,185).
		// OutPos = (-75,115,235). OutRot = yaw90.
		const FTransform TgtYaw90(Yaw90, TgtPos, FVector::OneVector);
		const FTransform Out = Run1v1T(SrcIdentity, TgtYaw90);
		CHECK(Out.GetLocation().Equals(FVector(-75.f, 115.f, 235.f), PosTolerance));
		CHECK(Out.GetRotation().Equals(Yaw90, PosTolerance));
	}

	SECTION("BothYaw90")
	{
		// Source rotation already = TgtRot → behaves like Target/AllCenter/TgtYaw90.
		// OutPos = (-75,115,235). OutRot = yaw90.
		const FTransform SrcYaw90(Yaw90, SrcPos, FVector::OneVector);
		const FTransform TgtYaw90(Yaw90, TgtPos, FVector::OneVector);
		const FTransform Out = Run1v1T(SrcYaw90, TgtYaw90);
		CHECK(Out.GetLocation().Equals(FVector(-75.f, 115.f, 235.f), PosTolerance));
		CHECK(Out.GetRotation().Equals(Yaw90, PosTolerance));
	}

	SECTION("AllMin_SrcYaw90_TgtIdentity")
	{
		// Source bounds min in tgt-local (identity relative rotation, translated by (150,100,-150)):
		//   Min = SrcBoundsMin + (150,100,-150) = (100,60,-180). TargetBoundsMin = (-30,-20,-10).
		// DeltaInRef = (-130,-80,170). WorldDelta = (-130,-80,170).
		// OutPos = (-30,120,220).
		const FTransform SrcYaw90(Yaw90, SrcPos, FVector::OneVector);
		SetAxis(true, ERef::Min, ERef::Min,
		        true, ERef::Min, ERef::Min,
		        true, ERef::Min, ERef::Min);
		const FTransform Out = Run1v1T(SrcYaw90, TgtIdentity);
		CHECK(Out.GetLocation().Equals(FVector(-30.f, 120.f, 220.f), PosTolerance));
	}
}
