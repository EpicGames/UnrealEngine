// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * PCG Attribute Filter Tests (Catch2)
 *
 * Test overview:
 * - PointFilterDensity:                    Filter points by density using a constant threshold (Lesser operator).
 * - PointFilterDensityRange:               Filter points by density using a min/max range.
 * - AttributeFilterInt:                    Filter param data entries by integer attribute using a constant threshold.
 * - AttributeFilterIntRange:               Filter param data entries by integer attribute using a min/max range.
 * - SkipTestBug:                           Regression test for UE-201595 - ensures SkipTests are properly reset during spatial query chunking.
 * - Points_GenerateEmptyOutput:            When bGenerateOutputDataEvenIfEmpty is true, verify empty outputs are generated for points.
 * - PointsRange_GenerateEmptyOutput:       When bGenerateOutputDataEvenIfEmpty is true, verify empty outputs for point range filtering.
 * - Attributes_GenerateEmptyOutput:        When bGenerateOutputDataEvenIfEmpty is true, verify empty outputs for attribute filtering.
 * - AttributesRange_GenerateEmptyOutput:   When bGenerateOutputDataEvenIfEmpty is true, verify empty outputs for attribute range filtering.
 * - Points_NoGenerateEmptyOutput:          When bGenerateOutputDataEvenIfEmpty is false, verify no empty outputs for points.
 * - PointsRange_NoGenerateEmptyOutput:     When bGenerateOutputDataEvenIfEmpty is false, verify no empty outputs for point range filtering.
 * - Attributes_NoGenerateEmptyOutput:      When bGenerateOutputDataEvenIfEmpty is false, verify no empty outputs for attributes.
 * - AttributesRange_NoGenerateEmptyOutput: When bGenerateOutputDataEvenIfEmpty is false, verify no empty outputs for attribute range filtering.
 * - NtoN_Points_MatchingCount:             N:N filtering with points where filter count matches input count.
 * - NtoN_Points_MismatchedCount:           N:N filtering with points where filter count does NOT match input count (expects error).
 * - NtoN_Attributes_MatchingCount:         N:N filtering with attributes where filter count matches input count.
 * - NtoN_Attributes_MismatchedCount:       N:N filtering with attributes where filter count does NOT match input count (expects error).
 * - NtoN_Range_MatchingCount:              N:N range filtering where filter count matches input count.
 * - NtoN_Range_MismatchedCount:            N:N range filtering where filter count does NOT match input count (expects error).
 */

#include "PCGTestsCommon.h"

#include "PCGContext.h"
#include "PCGGraph.h"
#include "PCGParamData.h"
#include "Data/PCGPointArrayData.h"
#include "Data/PCGPointData.h"
#include "Elements/PCGAttributeFilter.h"
#include "Elements/PCGGather.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/PCGMetadataAttributeTpl.h"

#include "Algo/Count.h"
#include "Math/RandomStream.h"

#include <catch2/catch_test_macros.hpp>

#include "TestHarness.h"

/**
 * Base fixture for attribute filter tests. Inherits from the single-element base test
 * and sets up a graph with the filter node connected to a gather node so that
 * output pins are connected (required for non-editor builds).
 */
template <typename SettingsType>
class FPCGAttributeFilterBaseTest : public PCGTests::FPCGSingleElementBaseTest<SettingsType>
{
public:
	static inline const FName InsideFilterLabel = PCGPinConstants::DefaultInFilterLabel;
	static inline const FName OutsideFilterLabel = PCGPinConstants::DefaultOutFilterLabel;
	static inline const FName FilterLabel = PCGAttributeFilterConstants::FilterLabel;
	static inline const FName MinFilterLabel = PCGAttributeFilterConstants::FilterMinLabel;
	static inline const FName MaxFilterLabel = PCGAttributeFilterConstants::FilterMaxLabel;
	
	static inline const FName IntAttributeName = "Int";
	static inline const FName DoubleAttributeName = "Double";

	FPCGAttributeFilterBaseTest()
	{
		// Create a graph and add the filter settings node so that output pins are connected
		Graph = NewObject<UPCGGraph>();

		FilterNode = Graph->AddNodeInstance(this->TypedSettings);

		UPCGGatherSettings* GatherSettings = nullptr;
		GatherNode = Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

		// Connect both output pins of the filter to the gather node so IsOutputPinConnected returns true
		Graph->AddEdge(FilterNode, InsideFilterLabel, GatherNode, PCGPinConstants::DefaultInputLabel);
		Graph->AddEdge(FilterNode, OutsideFilterLabel, GatherNode, PCGPinConstants::DefaultInputLabel);
	}
	
	void Execute()
	{
		this->ExecuteElement(/*ExecutionSource=*/nullptr, /*Node=*/FilterNode);
	}

	UPCGBasePointData* GeneratePointDataWithRandomDensity(int32 InNumPoints, int32 InRandomSeed)
	{
		UPCGBasePointData* PointData = this->CreatePointData();

		PointData->SetNumPoints(InNumPoints);
		PointData->AllocateProperties(EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::Density | EPCGPointNativeProperties::Seed);

		FRandomStream RandomSource(InRandomSeed);

		FPCGPointValueRanges ValueRanges(PointData, /*bAllocate=*/false);
		for (int i = 0; i < InNumPoints; ++i)
		{
			ValueRanges.TransformRange[i] = FTransform();
			ValueRanges.DensityRange[i] = RandomSource.FRand();
			ValueRanges.SeedRange[i] = i;
		}

		return PointData;
	}

	UPCGBasePointData* GeneratePointDataWithSingleDensity(const float Density)
	{
		UPCGBasePointData* PointData = this->CreatePointData();

		PointData->SetNumPoints(1);
		PointData->AllocateProperties(EPCGPointNativeProperties::Density);

		FPCGPointValueRanges ValueRanges(PointData, /*bAllocate=*/false);
		ValueRanges.DensityRange[0] = Density;

		return PointData;
	}

	UPCGParamData* GenerateAttributeSetWithRandomInt(int32 InNumEntries, int32 InRandomSeed)
	{
		UPCGParamData* ParamData = NewObject<UPCGParamData>();
		check(ParamData && ParamData->Metadata);
		FPCGMetadataAttribute<int32>* Attribute = ParamData->Metadata->CreateAttribute<int32>(IntAttributeName, 0, true, false);
		check(Attribute)

		FRandomStream RandomSource(InRandomSeed);

		for (int i = 0; i < InNumEntries; ++i)
		{
			PCGMetadataEntryKey Key = ParamData->Metadata->AddEntry();
			Attribute->SetValue(Key, RandomSource.GetUnsignedInt());
		}

		return ParamData;
	}

	UPCGParamData* GenerateAttributeSetWithSingleValue(const double Value)
	{
		UPCGParamData* ParamData = NewObject<UPCGParamData>();
		check(ParamData && ParamData->Metadata);
		FPCGMetadataAttribute<double>* Attribute = ParamData->Metadata->CreateAttribute<double>(DoubleAttributeName, 0, true, false);
		check(Attribute)

		const PCGMetadataEntryKey Key = ParamData->Metadata->AddEntry();
		Attribute->SetValue(Key, Value);

		return ParamData;
	}

	UPCGParamData* GenerateAttributeSetWithSingleInt(const int32 Value)
	{
		UPCGParamData* ParamData = NewObject<UPCGParamData>();
		check(ParamData && ParamData->Metadata);
		FPCGMetadataAttribute<int32>* Attribute = ParamData->Metadata->CreateAttribute<int32>(IntAttributeName, 0, true, false);
		check(Attribute)

		const PCGMetadataEntryKey Key = ParamData->Metadata->AddEntry();
		Attribute->SetValue(Key, Value);

		return ParamData;
	}

	static int GetNumEmptyData(const TArray<FPCGTaggedData>& InDataArray)
	{
		return Algo::CountIf(InDataArray, [](const FPCGTaggedData& Data)
		{
			if (const UPCGBasePointData* PointData = Cast<UPCGBasePointData>(Data.Data))
			{
				return PointData->IsEmpty();
			}
			else if (const UPCGParamData* ParamData = Cast<UPCGParamData>(Data.Data))
			{
				check(ParamData->Metadata)
				return ParamData->Metadata->GetAttributeCount() == 0 || ParamData->Metadata->GetItemCountForChild() == 0;
			}

			return false;
		});
	}

protected:
	TObjectPtr<UPCGGraph> Graph;
	TObjectPtr<UPCGNode> FilterNode;
	TObjectPtr<UPCGNode> GatherNode;
};

using FPCGAttributeFilterTest = FPCGAttributeFilterBaseTest<UPCGAttributeFilteringSettings>;
using FPCGAttributeFilterRangeTest = FPCGAttributeFilterBaseTest<UPCGAttributeFilteringRangeSettings>;

/**
 * Filter points by density with a constant threshold using the Lesser operator.
 * Generates 50 random-density points and verifies that inside-filter points have density < 0.5
 * and outside-filter points have density >= 0.5.
 */
TEST_CASE_METHOD(FPCGAttributeFilterTest, "PCG::AttributeFilter::Points::Density", "[PCG][AttributeFilter]")
{
	static constexpr int32 NumPoints = 50;
	static constexpr float DensityThreshold = 0.5f;

	TypedSettings->Operator = EPCGAttributeFilterOperator::Lesser;
	TypedSettings->TargetAttribute.SetPointProperty(EPCGPointProperties::Density);
	TypedSettings->bUseConstantThreshold = true;
	TypedSettings->AttributeTypes.FloatValue = DensityThreshold;
	TypedSettings->AttributeTypes.Type = EPCGMetadataTypes::Float;

	FPCGTaggedData& TaggedData = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
	TaggedData.Pin = PCGPinConstants::DefaultInputLabel;
	TaggedData.Data = GeneratePointDataWithRandomDensity(NumPoints, TypedSettings->Seed);

	Execute();

	TArray<FPCGTaggedData> InFilterOutput = Context->OutputData.GetInputsByPin(InsideFilterLabel);
	TArray<FPCGTaggedData> OutFilterOutput = Context->OutputData.GetInputsByPin(OutsideFilterLabel);

	REQUIRE(InFilterOutput.Num() == 1);
	REQUIRE(OutFilterOutput.Num() == 1);

	const UPCGBasePointData* InFilterPointData = Cast<UPCGBasePointData>(InFilterOutput[0].Data);
	const UPCGBasePointData* OutFilterPointData = Cast<UPCGBasePointData>(OutFilterOutput[0].Data);

	REQUIRE(InFilterPointData != nullptr);
	REQUIRE(OutFilterPointData != nullptr);

	const TConstPCGValueRange<float> InFilterDensityRange = InFilterPointData->GetConstDensityValueRange();
	
	for (const float InFilterDensity : InFilterDensityRange)
	{
		CHECK(InFilterDensity < DensityThreshold);
	}

	const TConstPCGValueRange<float> OutFilterDensityRange = OutFilterPointData->GetConstDensityValueRange();
	for (const float OutFilterDensity : OutFilterDensityRange)
	{
		CHECK(OutFilterDensity >= DensityThreshold);
	}
}

/**
 * Filter points by density using a min/max range ([0.3, 0.8]).
 * Verifies inside-filter points have density within the range and outside-filter points are outside it.
 */
TEST_CASE_METHOD(FPCGAttributeFilterRangeTest, "PCG::AttributeFilter::Points::DensityRange", "[PCG][AttributeFilter]")
{
	static constexpr int32 NumPoints = 50;
	static constexpr float DensityMinThreshold = 0.3f;
	static constexpr float DensityMaxThreshold = 0.8f;

	TypedSettings->TargetAttribute.SetPointProperty(EPCGPointProperties::Density);

	TypedSettings->MinThreshold.bUseConstantThreshold = true;
	TypedSettings->MinThreshold.AttributeTypes.FloatValue = DensityMinThreshold;
	TypedSettings->MinThreshold.AttributeTypes.Type = EPCGMetadataTypes::Float;

	TypedSettings->MaxThreshold.bUseConstantThreshold = true;
	TypedSettings->MaxThreshold.AttributeTypes.FloatValue = DensityMaxThreshold;
	TypedSettings->MaxThreshold.AttributeTypes.Type = EPCGMetadataTypes::Float;

	FPCGTaggedData& TaggedData = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
	TaggedData.Pin = PCGPinConstants::DefaultInputLabel;
	TaggedData.Data = GeneratePointDataWithRandomDensity(NumPoints, TypedSettings->Seed);

	Execute();

	TArray<FPCGTaggedData> InFilterOutput = Context->OutputData.GetInputsByPin(InsideFilterLabel);
	TArray<FPCGTaggedData> OutFilterOutput = Context->OutputData.GetInputsByPin(OutsideFilterLabel);

	REQUIRE(InFilterOutput.Num() == 1);
	REQUIRE(OutFilterOutput.Num() == 1);

	const UPCGBasePointData* InFilterPointData = Cast<UPCGBasePointData>(InFilterOutput[0].Data);
	const UPCGBasePointData* OutFilterPointData = Cast<UPCGBasePointData>(OutFilterOutput[0].Data);

	REQUIRE(InFilterPointData != nullptr);
	REQUIRE(OutFilterPointData != nullptr);

	const TConstPCGValueRange<float> InFilterDensityRange = InFilterPointData->GetConstDensityValueRange();
	for (const float InFilterDensity : InFilterDensityRange)
	{
		CHECK(((InFilterDensity >= DensityMinThreshold) && (InFilterDensity <= DensityMaxThreshold)));
	}

	const TConstPCGValueRange<float> OutFilterDensityRange = OutFilterPointData->GetConstDensityValueRange();
	for (const float OutFilterDensity : OutFilterDensityRange)
	{
		CHECK(((OutFilterDensity < DensityMinThreshold) || (OutFilterDensity > DensityMaxThreshold)));
	}
}

/**
 * Filter param data entries by an integer attribute using a constant threshold (Lesser operator).
 * Generates 50 random-int entries and verifies inside-filter entries have value < threshold
 * and outside-filter entries have value >= threshold.
 */
TEST_CASE_METHOD(FPCGAttributeFilterTest, "PCG::AttributeFilter::Params::Int", "[PCG][AttributeFilter]")
{
	static constexpr int32 NumEntries = 50;
	static constexpr int64 IntThreshold = 999999;

	TypedSettings->Operator = EPCGAttributeFilterOperator::Lesser;
	TypedSettings->TargetAttribute.SetAttributeName(IntAttributeName);
	TypedSettings->bUseConstantThreshold = true;
	TypedSettings->AttributeTypes.IntValue = IntThreshold;
	TypedSettings->AttributeTypes.Type = EPCGMetadataTypes::Integer64;

	FPCGTaggedData& TaggedData = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
	TaggedData.Pin = PCGPinConstants::DefaultInputLabel;
	TaggedData.Data = GenerateAttributeSetWithRandomInt(NumEntries, TypedSettings->Seed);

	Execute();

	TArray<FPCGTaggedData> InFilterOutput = Context->OutputData.GetInputsByPin(InsideFilterLabel);
	TArray<FPCGTaggedData> OutFilterOutput = Context->OutputData.GetInputsByPin(OutsideFilterLabel);

	REQUIRE(InFilterOutput.Num() == 1);
	REQUIRE(OutFilterOutput.Num() == 1);

	const UPCGParamData* InFilterParamData = Cast<UPCGParamData>(InFilterOutput[0].Data);
	const UPCGParamData* OutFilterParamData = Cast<UPCGParamData>(OutFilterOutput[0].Data);

	REQUIRE(InFilterParamData != nullptr);
	REQUIRE(OutFilterParamData != nullptr);

	const FPCGMetadataAttribute<int32>* InFilterAttribute = InFilterParamData->Metadata->GetConstTypedAttribute<int32>(IntAttributeName);
	const FPCGMetadataAttribute<int32>* OutFilterAttribute = OutFilterParamData->Metadata->GetConstTypedAttribute<int32>(IntAttributeName);

	REQUIRE(InFilterAttribute != nullptr);
	REQUIRE(OutFilterAttribute != nullptr);

	for (int32 Key = 0; Key < InFilterParamData->Metadata->GetLocalItemCount(); ++Key)
	{
		int32 Value = InFilterAttribute->GetValueFromItemKey(Key);
		CHECK(Value < (int32)IntThreshold);
	}

	for (int32 Key = 0; Key < OutFilterParamData->Metadata->GetLocalItemCount(); ++Key)
	{
		int32 Value = OutFilterAttribute->GetValueFromItemKey(Key);
		CHECK(Value >= (int32)IntThreshold);
	}
}

/**
 * Filter param data entries by an integer attribute using a min/max range.
 * Generates 50 random-int entries and verifies inside-filter entries are within [0, 1999999999]
 * and outside-filter entries are outside that range.
 */
TEST_CASE_METHOD(FPCGAttributeFilterRangeTest, "PCG::AttributeFilter::Params::IntRange", "[PCG][AttributeFilter]")
{
	static constexpr int32 NumEntries = 50;
	static constexpr int64 IntMinThreshold = 0;
	static constexpr int64 IntMaxThreshold = 1999999999;

	TypedSettings->TargetAttribute.SetAttributeName(IntAttributeName);

	TypedSettings->MinThreshold.bUseConstantThreshold = true;
	TypedSettings->MinThreshold.AttributeTypes.IntValue = IntMinThreshold;
	TypedSettings->MinThreshold.AttributeTypes.Type = EPCGMetadataTypes::Integer64;

	TypedSettings->MaxThreshold.bUseConstantThreshold = true;
	TypedSettings->MaxThreshold.AttributeTypes.IntValue = IntMaxThreshold;
	TypedSettings->MaxThreshold.AttributeTypes.Type = EPCGMetadataTypes::Integer64;

	FPCGTaggedData& TaggedData = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
	TaggedData.Pin = PCGPinConstants::DefaultInputLabel;
	TaggedData.Data = GenerateAttributeSetWithRandomInt(NumEntries, TypedSettings->Seed);

	Execute();

	TArray<FPCGTaggedData> InFilterOutput = Context->OutputData.GetInputsByPin(InsideFilterLabel);
	TArray<FPCGTaggedData> OutFilterOutput = Context->OutputData.GetInputsByPin(OutsideFilterLabel);

	REQUIRE(InFilterOutput.Num() == 1);
	REQUIRE(OutFilterOutput.Num() == 1);

	const UPCGParamData* InFilterParamData = Cast<UPCGParamData>(InFilterOutput[0].Data);
	const UPCGParamData* OutFilterParamData = Cast<UPCGParamData>(OutFilterOutput[0].Data);

	REQUIRE(InFilterParamData != nullptr);
	REQUIRE(OutFilterParamData != nullptr);

	const FPCGMetadataAttribute<int32>* InFilterAttribute = InFilterParamData->Metadata->GetConstTypedAttribute<int32>(IntAttributeName);
	const FPCGMetadataAttribute<int32>* OutFilterAttribute = OutFilterParamData->Metadata->GetConstTypedAttribute<int32>(IntAttributeName);

	REQUIRE(InFilterAttribute != nullptr);
	REQUIRE(OutFilterAttribute != nullptr);

	for (int32 Key = 0; Key < InFilterParamData->Metadata->GetLocalItemCount(); ++Key)
	{
		int32 Value = InFilterAttribute->GetValueFromItemKey(Key);
		CHECK(((Value >= (int32)IntMinThreshold) && (Value <= (int32)IntMaxThreshold)));
	}

	for (int32 Key = 0; Key < OutFilterParamData->Metadata->GetLocalItemCount(); ++Key)
	{
		int32 Value = OutFilterAttribute->GetValueFromItemKey(Key);
		CHECK(((Value < (int32)IntMinThreshold) || (Value > (int32)IntMaxThreshold)));
	}
}

/**
 * Regression test for UE-201595: SkipTests were not properly reset during spatial query chunking,
 * causing points to be incorrectly accepted. Uses 2048 points (to exceed the 256 default chunk size)
 * with the first half having no spatial overlap and the second half matching but with different densities.
 * Verifies the inside and outside filter each receive exactly half the points.
 */
TEST_CASE_METHOD(FPCGAttributeFilterTest, "PCG::AttributeFilter::SkipTestBug", "[PCG][AttributeFilter]")
{
	TypedSettings->TargetAttribute.SetPointProperty(EPCGPointProperties::Density);
	TypedSettings->ThresholdAttribute.SetPointProperty(EPCGPointProperties::Density);
	TypedSettings->Operator = EPCGAttributeFilterOperator::Equal;
	TypedSettings->bUseSpatialQuery = true;

	UPCGBasePointData* InputPointData = this->CreatePointData();
	UPCGBasePointData* ThresholdPointData = this->CreatePointData();

	constexpr int32 NumPoints = 2048;
	constexpr int32 HalfNumPoints = NumPoints / 2;
	InputPointData->SetNumPoints(NumPoints);
	TPCGValueRange<FTransform> InputTransformRange = InputPointData->GetTransformValueRange();

	ThresholdPointData->SetNumPoints(NumPoints);
	TPCGValueRange<FTransform> ThresholdTransformRange = ThresholdPointData->GetTransformValueRange();
	TPCGValueRange<float> ThresholdDensityRange = ThresholdPointData->GetDensityValueRange();

	for (int32 i = 0; i < NumPoints; ++i)
	{
		// First half are very different so the sampling should fail, second half are the same for the sampling to succeed but the filtering to fail.
		if (i >= HalfNumPoints)
		{
			InputTransformRange[i].SetLocation(FVector(10 * i, 10 * i, 10 * i));
			ThresholdTransformRange[i] = InputTransformRange[i];
			ThresholdDensityRange[i] = 0.5f;
		}
		else
		{
			InputTransformRange[i].SetLocation(FVector(i, i, i));
			ThresholdTransformRange[i].SetLocation(FVector(-10 * i - 1000, -10 * i - 1000, -10 * i - 1000));
		}
	}

	FPCGTaggedData& TaggedData = InputData.TaggedData.Emplace_GetRef();
	TaggedData.Pin = PCGPinConstants::DefaultInputLabel;
	TaggedData.Data = InputPointData;

	FPCGTaggedData& SecondTaggedData = InputData.TaggedData.Emplace_GetRef();
	SecondTaggedData.Pin = FilterLabel;
	SecondTaggedData.Data = ThresholdPointData;

	Execute();

	TArray<FPCGTaggedData> InFilterOutput = Context->OutputData.GetInputsByPin(InsideFilterLabel);
	TArray<FPCGTaggedData> OutFilterOutput = Context->OutputData.GetInputsByPin(OutsideFilterLabel);

	REQUIRE(InFilterOutput.Num() == 1);
	REQUIRE(OutFilterOutput.Num() == 1);

	const UPCGBasePointData* InFilterPointData = Cast<UPCGBasePointData>(InFilterOutput[0].Data);
	const UPCGBasePointData* OutFilterPointData = Cast<UPCGBasePointData>(OutFilterOutput[0].Data);

	REQUIRE(InFilterPointData != nullptr);
	REQUIRE(OutFilterPointData != nullptr);

	CHECK(InFilterPointData->GetNumPoints() == HalfNumPoints);
	CHECK(OutFilterPointData->GetNumPoints() == HalfNumPoints);
}

/**
 * With bGenerateOutputDataEvenIfEmpty=true and point data, verify that empty outputs are always generated.
 * Tests two thresholds: 0.0 (all pass) and 1.0 (none pass), both should produce 4 outputs on each pin.
 */
TEST_CASE_METHOD(FPCGAttributeFilterTest, "PCG::AttributeFilter::Points::GenerateEmptyOutput", "[PCG][AttributeFilter]")
{
	TypedSettings->Operator = EPCGAttributeFilterOperator::Greater;
	TypedSettings->TargetAttribute.SetPointProperty(EPCGPointProperties::Density);
	TypedSettings->bUseConstantThreshold = true;
	TypedSettings->AttributeTypes.Type = EPCGMetadataTypes::Float;

	for (int32 i = 0; i < 4; ++i)
	{
		FPCGTaggedData& TaggedData = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		TaggedData.Pin = PCGPinConstants::DefaultInputLabel;
		TaggedData.Data = GeneratePointDataWithSingleDensity((1.0 + i) / 4.0);
	}

	TypedSettings->bGenerateOutputDataEvenIfEmpty = true;

	SECTION("All points pass filter (threshold 0.0)")
	{
		TypedSettings->AttributeTypes.FloatValue = 0.f;
		Execute();

		TArray<FPCGTaggedData> InFilterOutput = Context->OutputData.GetInputsByPin(InsideFilterLabel);
		TArray<FPCGTaggedData> OutFilterOutput = Context->OutputData.GetInputsByPin(OutsideFilterLabel);

		CHECK(InFilterOutput.Num() == 4);
		CHECK(OutFilterOutput.Num() == 4);
		CHECK(GetNumEmptyData(InFilterOutput) == 0);
		CHECK(GetNumEmptyData(OutFilterOutput) == 4);
	}

	SECTION("No points pass filter (threshold 1.0)")
	{
		TypedSettings->AttributeTypes.FloatValue = 1.f;
		Execute();

		TArray<FPCGTaggedData> InFilterOutput = Context->OutputData.GetInputsByPin(InsideFilterLabel);
		TArray<FPCGTaggedData> OutFilterOutput = Context->OutputData.GetInputsByPin(OutsideFilterLabel);

		CHECK(InFilterOutput.Num() == 4);
		CHECK(OutFilterOutput.Num() == 4);
		CHECK(GetNumEmptyData(InFilterOutput) == 4);
		CHECK(GetNumEmptyData(OutFilterOutput) == 0);
	}
}

/**
 * With bGenerateOutputDataEvenIfEmpty=true and point range filtering, verify empty outputs are generated.
 * Tests [0, 1] inclusive (all pass) and [0, 0] inclusive (none pass).
 */
TEST_CASE_METHOD(FPCGAttributeFilterRangeTest, "PCG::AttributeFilter::PointsRange::GenerateEmptyOutput", "[PCG][AttributeFilter]")
{
	TypedSettings->TargetAttribute.SetPointProperty(EPCGPointProperties::Density);
	TypedSettings->MinThreshold.bUseConstantThreshold = true;
	TypedSettings->MaxThreshold.bUseConstantThreshold = true;
	TypedSettings->MinThreshold.AttributeTypes.Type = EPCGMetadataTypes::Float;
	TypedSettings->MaxThreshold.AttributeTypes.Type = EPCGMetadataTypes::Float;
	TypedSettings->MinThreshold.bInclusive = true;
	TypedSettings->MaxThreshold.bInclusive = true;

	for (int32 i = 0; i < 4; ++i)
	{
		FPCGTaggedData& TaggedData = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		TaggedData.Pin = PCGPinConstants::DefaultInputLabel;
		TaggedData.Data = GeneratePointDataWithSingleDensity((1.0 + i) / 4.0);
	}

	TypedSettings->bGenerateOutputDataEvenIfEmpty = true;

	SECTION("All points in range [0, 1]")
	{
		TypedSettings->MinThreshold.AttributeTypes.FloatValue = 0.0f;
		TypedSettings->MaxThreshold.AttributeTypes.FloatValue = 1.0f;
		Execute();

		TArray<FPCGTaggedData> InFilterOutput = Context->OutputData.GetInputsByPin(InsideFilterLabel);
		TArray<FPCGTaggedData> OutFilterOutput = Context->OutputData.GetInputsByPin(OutsideFilterLabel);

		CHECK(InFilterOutput.Num() == 4);
		CHECK(OutFilterOutput.Num() == 4);
		CHECK(GetNumEmptyData(InFilterOutput) == 0);
		CHECK(GetNumEmptyData(OutFilterOutput) == 4);
	}

	SECTION("No points in range [0, 0]")
	{
		TypedSettings->MinThreshold.AttributeTypes.FloatValue = 0.0f;
		TypedSettings->MaxThreshold.AttributeTypes.FloatValue = 0.0f;
		Execute();

		TArray<FPCGTaggedData> InFilterOutput = Context->OutputData.GetInputsByPin(InsideFilterLabel);
		TArray<FPCGTaggedData> OutFilterOutput = Context->OutputData.GetInputsByPin(OutsideFilterLabel);

		CHECK(InFilterOutput.Num() == 4);
		CHECK(OutFilterOutput.Num() == 4);
		CHECK(GetNumEmptyData(InFilterOutput) == 4);
		CHECK(GetNumEmptyData(OutFilterOutput) == 0);
	}
}

/**
 * With bGenerateOutputDataEvenIfEmpty=true and attribute (double) filtering, verify empty outputs are generated.
 * Tests threshold 0.0 (all pass) and 1.0 (none pass).
 */
TEST_CASE_METHOD(FPCGAttributeFilterTest, "PCG::AttributeFilter::Params::GenerateEmptyOutput", "[PCG][AttributeFilter]")
{
	TypedSettings->Operator = EPCGAttributeFilterOperator::Greater;
	TypedSettings->TargetAttribute.SetAttributeName(DoubleAttributeName);
	TypedSettings->AttributeTypes.Type = EPCGMetadataTypes::Double;
	TypedSettings->bUseConstantThreshold = true;

	for (int32 i = 0; i < 4; ++i)
	{
		FPCGTaggedData& TaggedData = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		TaggedData.Pin = PCGPinConstants::DefaultInputLabel;
		TaggedData.Data = GenerateAttributeSetWithSingleValue((1.0 + i) / 4.0);
	}

	TypedSettings->bGenerateOutputDataEvenIfEmpty = true;

	SECTION("All attributes pass filter (threshold 0.0)")
	{
		TypedSettings->AttributeTypes.DoubleValue = 0.f;
		Execute();

		TArray<FPCGTaggedData> InFilterOutput = Context->OutputData.GetInputsByPin(InsideFilterLabel);
		TArray<FPCGTaggedData> OutFilterOutput = Context->OutputData.GetInputsByPin(OutsideFilterLabel);

		CHECK(InFilterOutput.Num() == 4);
		CHECK(OutFilterOutput.Num() == 4);
		CHECK(GetNumEmptyData(InFilterOutput) == 0);
		CHECK(GetNumEmptyData(OutFilterOutput) == 4);
	}

	SECTION("No attributes pass filter (threshold 1.0)")
	{
		TypedSettings->AttributeTypes.DoubleValue = 1.f;
		Execute();

		TArray<FPCGTaggedData> InFilterOutput = Context->OutputData.GetInputsByPin(InsideFilterLabel);
		TArray<FPCGTaggedData> OutFilterOutput = Context->OutputData.GetInputsByPin(OutsideFilterLabel);

		CHECK(InFilterOutput.Num() == 4);
		CHECK(OutFilterOutput.Num() == 4);
		CHECK(GetNumEmptyData(InFilterOutput) == 4);
		CHECK(GetNumEmptyData(OutFilterOutput) == 0);
	}
}

/**
 * With bGenerateOutputDataEvenIfEmpty=true and attribute range filtering, verify empty outputs are generated.
 * Tests [0, 1] inclusive (all pass) and [0, 0] inclusive (none pass).
 */
TEST_CASE_METHOD(FPCGAttributeFilterRangeTest, "PCG::AttributeFilter::ParamsRange::GenerateEmptyOutput", "[PCG][AttributeFilter]")
{
	TypedSettings->TargetAttribute.SetAttributeName(DoubleAttributeName);
	TypedSettings->MinThreshold.bUseConstantThreshold = true;
	TypedSettings->MaxThreshold.bUseConstantThreshold = true;
	TypedSettings->MinThreshold.AttributeTypes.Type = EPCGMetadataTypes::Double;
	TypedSettings->MaxThreshold.AttributeTypes.Type = EPCGMetadataTypes::Double;
	TypedSettings->MinThreshold.bInclusive = true;
	TypedSettings->MaxThreshold.bInclusive = true;

	for (int32 i = 0; i < 4; ++i)
	{
		FPCGTaggedData& TaggedData = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		TaggedData.Pin = PCGPinConstants::DefaultInputLabel;
		TaggedData.Data = GenerateAttributeSetWithSingleValue((1.0 + i) / 4.0);
	}

	TypedSettings->bGenerateOutputDataEvenIfEmpty = true;

	SECTION("All attributes in range [0, 1]")
	{
		TypedSettings->MinThreshold.AttributeTypes.DoubleValue = 0.0f;
		TypedSettings->MaxThreshold.AttributeTypes.DoubleValue = 1.0f;
		Execute();

		TArray<FPCGTaggedData> InFilterOutput = Context->OutputData.GetInputsByPin(InsideFilterLabel);
		TArray<FPCGTaggedData> OutFilterOutput = Context->OutputData.GetInputsByPin(OutsideFilterLabel);

		CHECK(InFilterOutput.Num() == 4);
		CHECK(OutFilterOutput.Num() == 4);
		CHECK(GetNumEmptyData(InFilterOutput) == 0);
		CHECK(GetNumEmptyData(OutFilterOutput) == 4);
	}

	SECTION("No attributes in range [0, 0]")
	{
		TypedSettings->MinThreshold.AttributeTypes.DoubleValue = 0.0f;
		TypedSettings->MaxThreshold.AttributeTypes.DoubleValue = 0.0f;
		Execute();

		TArray<FPCGTaggedData> InFilterOutput = Context->OutputData.GetInputsByPin(InsideFilterLabel);
		TArray<FPCGTaggedData> OutFilterOutput = Context->OutputData.GetInputsByPin(OutsideFilterLabel);

		CHECK(InFilterOutput.Num() == 4);
		CHECK(OutFilterOutput.Num() == 4);
		CHECK(GetNumEmptyData(InFilterOutput) == 4);
		CHECK(GetNumEmptyData(OutFilterOutput) == 0);
	}
}

/**
 * With bGenerateOutputDataEvenIfEmpty=false and point data, verify no empty outputs are generated.
 * Tests three thresholds: 0.0 (all pass -> 4 in, 0 out), 0.333 (3 in, 1 out), 1.0 (0 in, 4 out).
 */
TEST_CASE_METHOD(FPCGAttributeFilterTest, "PCG::AttributeFilter::Points::NoGenerateEmptyOutput", "[PCG][AttributeFilter]")
{
	TypedSettings->Operator = EPCGAttributeFilterOperator::Greater;
	TypedSettings->TargetAttribute.SetPointProperty(EPCGPointProperties::Density);
	TypedSettings->bUseConstantThreshold = true;
	TypedSettings->AttributeTypes.Type = EPCGMetadataTypes::Float;

	for (int32 i = 0; i < 4; ++i)
	{
		FPCGTaggedData& TaggedData = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		TaggedData.Pin = PCGPinConstants::DefaultInputLabel;
		TaggedData.Data = GeneratePointDataWithSingleDensity((1.0 + i) / 4.0);
	}

	TypedSettings->bGenerateOutputDataEvenIfEmpty = false;

	SECTION("All points pass filter (threshold 0.0)")
	{
		TypedSettings->AttributeTypes.FloatValue = 0.f;
		Execute();

		TArray<FPCGTaggedData> InFilterOutput = Context->OutputData.GetInputsByPin(InsideFilterLabel);
		TArray<FPCGTaggedData> OutFilterOutput = Context->OutputData.GetInputsByPin(OutsideFilterLabel);

		CHECK(InFilterOutput.Num() == 4);
		CHECK(OutFilterOutput.Num() == 0);
		CHECK(GetNumEmptyData(InFilterOutput) == 0);
	}

	SECTION("Some points pass filter (threshold 0.333)")
	{
		TypedSettings->AttributeTypes.FloatValue = 0.333f;
		Execute();

		TArray<FPCGTaggedData> InFilterOutput = Context->OutputData.GetInputsByPin(InsideFilterLabel);
		TArray<FPCGTaggedData> OutFilterOutput = Context->OutputData.GetInputsByPin(OutsideFilterLabel);

		CHECK(InFilterOutput.Num() == 3);
		CHECK(OutFilterOutput.Num() == 1);
		CHECK(GetNumEmptyData(InFilterOutput) == 0);
		CHECK(GetNumEmptyData(OutFilterOutput) == 0);
	}

	SECTION("No points pass filter (threshold 1.0)")
	{
		TypedSettings->AttributeTypes.FloatValue = 1.f;
		Execute();

		TArray<FPCGTaggedData> InFilterOutput = Context->OutputData.GetInputsByPin(InsideFilterLabel);
		TArray<FPCGTaggedData> OutFilterOutput = Context->OutputData.GetInputsByPin(OutsideFilterLabel);

		CHECK(InFilterOutput.Num() == 0);
		CHECK(OutFilterOutput.Num() == 4);
		CHECK(GetNumEmptyData(OutFilterOutput) == 0);
	}
}

/**
 * With bGenerateOutputDataEvenIfEmpty=false and point range filtering, verify no empty outputs are generated.
 * Tests [0,1] (all), [0.2,0.6] (some), [1,1] inclusive (1 point), [1,1] exclusive (none).
 */
TEST_CASE_METHOD(FPCGAttributeFilterRangeTest, "PCG::AttributeFilter::PointsRange::NoGenerateEmptyOutput", "[PCG][AttributeFilter]")
{
	TypedSettings->TargetAttribute.SetPointProperty(EPCGPointProperties::Density);
	TypedSettings->MinThreshold.bUseConstantThreshold = true;
	TypedSettings->MaxThreshold.bUseConstantThreshold = true;
	TypedSettings->MinThreshold.AttributeTypes.Type = EPCGMetadataTypes::Float;
	TypedSettings->MaxThreshold.AttributeTypes.Type = EPCGMetadataTypes::Float;
	TypedSettings->MinThreshold.bInclusive = true;
	TypedSettings->MaxThreshold.bInclusive = true;

	for (int32 i = 0; i < 4; ++i)
	{
		FPCGTaggedData& TaggedData = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		TaggedData.Pin = PCGPinConstants::DefaultInputLabel;
		TaggedData.Data = GeneratePointDataWithSingleDensity((1.0 + i) / 4.0);
	}

	TypedSettings->bGenerateOutputDataEvenIfEmpty = false;

	SECTION("All in range [0, 1] inclusive")
	{
		TypedSettings->MinThreshold.AttributeTypes.FloatValue = 0.0f;
		TypedSettings->MaxThreshold.AttributeTypes.FloatValue = 1.0f;
		Execute();

		TArray<FPCGTaggedData> InFilterOutput = Context->OutputData.GetInputsByPin(InsideFilterLabel);
		TArray<FPCGTaggedData> OutFilterOutput = Context->OutputData.GetInputsByPin(OutsideFilterLabel);

		CHECK(InFilterOutput.Num() == 4);
		CHECK(OutFilterOutput.Num() == 0);
	}

	SECTION("Some in range [0.2, 0.6] inclusive")
	{
		TypedSettings->MinThreshold.AttributeTypes.FloatValue = 0.2f;
		TypedSettings->MaxThreshold.AttributeTypes.FloatValue = 0.6f;
		Execute();

		TArray<FPCGTaggedData> InFilterOutput = Context->OutputData.GetInputsByPin(InsideFilterLabel);
		TArray<FPCGTaggedData> OutFilterOutput = Context->OutputData.GetInputsByPin(OutsideFilterLabel);

		CHECK(InFilterOutput.Num() == 2);
		CHECK(OutFilterOutput.Num() == 2);
		CHECK(GetNumEmptyData(InFilterOutput) == 0);
		CHECK(GetNumEmptyData(OutFilterOutput) == 0);
	}

	SECTION("One point in range [1, 1] inclusive")
	{
		TypedSettings->MinThreshold.AttributeTypes.FloatValue = 1.0f;
		TypedSettings->MaxThreshold.AttributeTypes.FloatValue = 1.0f;
		Execute();

		TArray<FPCGTaggedData> InFilterOutput = Context->OutputData.GetInputsByPin(InsideFilterLabel);
		TArray<FPCGTaggedData> OutFilterOutput = Context->OutputData.GetInputsByPin(OutsideFilterLabel);

		CHECK(InFilterOutput.Num() == 1);
		CHECK(OutFilterOutput.Num() == 3);
		CHECK(GetNumEmptyData(InFilterOutput) == 0);
		CHECK(GetNumEmptyData(OutFilterOutput) == 0);
	}

	SECTION("No points in range [1, 1] exclusive")
	{
		TypedSettings->MinThreshold.bInclusive = false;
		TypedSettings->MaxThreshold.bInclusive = false;
		TypedSettings->MinThreshold.AttributeTypes.FloatValue = 1.0f;
		TypedSettings->MaxThreshold.AttributeTypes.FloatValue = 1.0f;
		Execute();

		TArray<FPCGTaggedData> InFilterOutput = Context->OutputData.GetInputsByPin(InsideFilterLabel);
		TArray<FPCGTaggedData> OutFilterOutput = Context->OutputData.GetInputsByPin(OutsideFilterLabel);

		CHECK(InFilterOutput.Num() == 0);
		CHECK(OutFilterOutput.Num() == 4);
		CHECK(GetNumEmptyData(OutFilterOutput) == 0);
	}
}

/**
 * With bGenerateOutputDataEvenIfEmpty=false and attribute (double) filtering, verify no empty outputs.
 * Tests three thresholds: 0.0 (all pass), 0.333 (3 pass), 1.0 (none pass).
 */
TEST_CASE_METHOD(FPCGAttributeFilterTest, "PCG::AttributeFilter::Params::NoGenerateEmptyOutput", "[PCG][AttributeFilter]")
{
	TypedSettings->Operator = EPCGAttributeFilterOperator::Greater;
	TypedSettings->TargetAttribute.SetAttributeName(DoubleAttributeName);
	TypedSettings->AttributeTypes.Type = EPCGMetadataTypes::Double;
	TypedSettings->bUseConstantThreshold = true;

	for (int32 i = 0; i < 4; ++i)
	{
		FPCGTaggedData& TaggedData = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		TaggedData.Pin = PCGPinConstants::DefaultInputLabel;
		TaggedData.Data = GenerateAttributeSetWithSingleValue((1.0 + i) / 4.0);
	}

	TypedSettings->bGenerateOutputDataEvenIfEmpty = false;

	SECTION("All attributes pass filter (threshold 0.0)")
	{
		TypedSettings->AttributeTypes.DoubleValue = 0.f;
		Execute();

		TArray<FPCGTaggedData> InFilterOutput = Context->OutputData.GetInputsByPin(InsideFilterLabel);
		TArray<FPCGTaggedData> OutFilterOutput = Context->OutputData.GetInputsByPin(OutsideFilterLabel);

		CHECK(InFilterOutput.Num() == 4);
		CHECK(OutFilterOutput.Num() == 0);
		CHECK(GetNumEmptyData(InFilterOutput) == 0);
	}

	SECTION("Some attributes pass filter (threshold 0.333)")
	{
		TypedSettings->AttributeTypes.DoubleValue = 0.333f;
		Execute();

		TArray<FPCGTaggedData> InFilterOutput = Context->OutputData.GetInputsByPin(InsideFilterLabel);
		TArray<FPCGTaggedData> OutFilterOutput = Context->OutputData.GetInputsByPin(OutsideFilterLabel);

		CHECK(InFilterOutput.Num() == 3);
		CHECK(OutFilterOutput.Num() == 1);
		CHECK(GetNumEmptyData(InFilterOutput) == 0);
		CHECK(GetNumEmptyData(OutFilterOutput) == 0);
	}

	SECTION("No attributes pass filter (threshold 1.0)")
	{
		TypedSettings->AttributeTypes.DoubleValue = 1.f;
		Execute();

		TArray<FPCGTaggedData> InFilterOutput = Context->OutputData.GetInputsByPin(InsideFilterLabel);
		TArray<FPCGTaggedData> OutFilterOutput = Context->OutputData.GetInputsByPin(OutsideFilterLabel);

		CHECK(InFilterOutput.Num() == 0);
		CHECK(OutFilterOutput.Num() == 4);
		CHECK(GetNumEmptyData(OutFilterOutput) == 0);
	}
}

/**
 * With bGenerateOutputDataEvenIfEmpty=false and attribute range filtering, verify no empty outputs.
 * Tests [0,1] (all), [0.2,0.6] (some), [1,1] inclusive (1), [1,1] exclusive (none).
 */
TEST_CASE_METHOD(FPCGAttributeFilterRangeTest, "PCG::AttributeFilter::ParamsRange::NoGenerateEmptyOutput", "[PCG][AttributeFilter]")
{
	TypedSettings->TargetAttribute.SetAttributeName(DoubleAttributeName);
	TypedSettings->MinThreshold.bUseConstantThreshold = true;
	TypedSettings->MaxThreshold.bUseConstantThreshold = true;
	TypedSettings->MinThreshold.AttributeTypes.Type = EPCGMetadataTypes::Double;
	TypedSettings->MaxThreshold.AttributeTypes.Type = EPCGMetadataTypes::Double;
	TypedSettings->MinThreshold.bInclusive = true;
	TypedSettings->MaxThreshold.bInclusive = true;

	for (int32 i = 0; i < 4; ++i)
	{
		FPCGTaggedData& TaggedData = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		TaggedData.Pin = PCGPinConstants::DefaultInputLabel;
		TaggedData.Data = GenerateAttributeSetWithSingleValue((1.0 + i) / 4.0);
	}

	TypedSettings->bGenerateOutputDataEvenIfEmpty = false;

	SECTION("All in range [0, 1] inclusive")
	{
		TypedSettings->MinThreshold.AttributeTypes.DoubleValue = 0.0f;
		TypedSettings->MaxThreshold.AttributeTypes.DoubleValue = 1.0f;
		Execute();

		TArray<FPCGTaggedData> InFilterOutput = Context->OutputData.GetInputsByPin(InsideFilterLabel);
		TArray<FPCGTaggedData> OutFilterOutput = Context->OutputData.GetInputsByPin(OutsideFilterLabel);

		CHECK(InFilterOutput.Num() == 4);
		CHECK(OutFilterOutput.Num() == 0);
		CHECK(GetNumEmptyData(InFilterOutput) == 0);
	}

	SECTION("Some in range [0.2, 0.6] inclusive")
	{
		TypedSettings->MinThreshold.AttributeTypes.DoubleValue = 0.2f;
		TypedSettings->MaxThreshold.AttributeTypes.DoubleValue = 0.6f;
		Execute();

		TArray<FPCGTaggedData> InFilterOutput = Context->OutputData.GetInputsByPin(InsideFilterLabel);
		TArray<FPCGTaggedData> OutFilterOutput = Context->OutputData.GetInputsByPin(OutsideFilterLabel);

		CHECK(InFilterOutput.Num() == 2);
		CHECK(OutFilterOutput.Num() == 2);
		CHECK(GetNumEmptyData(InFilterOutput) == 0);
		CHECK(GetNumEmptyData(OutFilterOutput) == 0);
	}

	SECTION("One in range [1, 1] inclusive")
	{
		TypedSettings->MinThreshold.AttributeTypes.DoubleValue = 1.0f;
		TypedSettings->MaxThreshold.AttributeTypes.DoubleValue = 1.0f;
		Execute();

		TArray<FPCGTaggedData> InFilterOutput = Context->OutputData.GetInputsByPin(InsideFilterLabel);
		TArray<FPCGTaggedData> OutFilterOutput = Context->OutputData.GetInputsByPin(OutsideFilterLabel);

		CHECK(InFilterOutput.Num() == 1);
		CHECK(OutFilterOutput.Num() == 3);
		CHECK(GetNumEmptyData(InFilterOutput) == 0);
		CHECK(GetNumEmptyData(OutFilterOutput) == 0);
	}

	SECTION("None in range [1, 1] exclusive")
	{
		TypedSettings->MinThreshold.bInclusive = false;
		TypedSettings->MaxThreshold.bInclusive = false;
		TypedSettings->MinThreshold.AttributeTypes.DoubleValue = 1.0f;
		TypedSettings->MaxThreshold.AttributeTypes.DoubleValue = 1.0f;
		Execute();

		TArray<FPCGTaggedData> InFilterOutput = Context->OutputData.GetInputsByPin(InsideFilterLabel);
		TArray<FPCGTaggedData> OutFilterOutput = Context->OutputData.GetInputsByPin(OutsideFilterLabel);

		CHECK(InFilterOutput.Num() == 0);
		CHECK(OutFilterOutput.Num() == 4);
		CHECK(GetNumEmptyData(OutFilterOutput) == 0);
	}
}

/**
 * N:N point filtering with matching input/filter counts.
 * Creates 3 input point data with densities [0.2, 0.5, 0.8] and 3 filter point data
 * with density thresholds [0.6, 0.4, 0.5]. Uses Greater operator on density.
 * Input 0 (0.2) should fail, inputs 1 and 2 (0.5, 0.8) should pass.
 */
TEST_CASE_METHOD(FPCGAttributeFilterTest, "PCG::AttributeFilter::NtoN::Points::MatchingCount", "[PCG][AttributeFilter]")
{
	TypedSettings->Operator = EPCGAttributeFilterOperator::Greater;
	TypedSettings->TargetAttribute.SetPointProperty(EPCGPointProperties::Density);
	TypedSettings->ThresholdAttribute.SetPointProperty(EPCGPointProperties::Density);
	TypedSettings->bUseConstantThreshold = false;
	TypedSettings->bUseSpatialQuery = false;
	TypedSettings->bFilterEachInputWithEachFilterDataRespectively = true;
	TypedSettings->bGenerateOutputDataEvenIfEmpty = false;

	const float InputDensities[] = { 0.2f, 0.5f, 0.8f };
	for (float Density : InputDensities)
	{
		FPCGTaggedData& TaggedData = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		TaggedData.Pin = PCGPinConstants::DefaultInputLabel;
		TaggedData.Data = GeneratePointDataWithSingleDensity(Density);
	}

	FPCGTaggedData& FilterTaggedData1 = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
	FilterTaggedData1.Pin = FilterLabel;
	FilterTaggedData1.Data = GeneratePointDataWithSingleDensity(0.6f);
	
	FPCGTaggedData& FilterTaggedData2 = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
	FilterTaggedData2.Pin = FilterLabel;
	FilterTaggedData2.Data = GeneratePointDataWithSingleDensity(0.4f);
	
	FPCGTaggedData& FilterTaggedData3 = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
	FilterTaggedData3.Pin = FilterLabel;
	FilterTaggedData3.Data = GeneratePointDataWithSingleDensity(0.5f);

	Execute();

	TArray<FPCGTaggedData> InFilterOutput = Context->OutputData.GetInputsByPin(InsideFilterLabel);
	TArray<FPCGTaggedData> OutFilterOutput = Context->OutputData.GetInputsByPin(OutsideFilterLabel);

	// 0.5 > 0.4 and 0.8 > 0.5 pass; 0.2 > 0.6 fails
	CHECK(InFilterOutput.Num() == 2);
	CHECK(OutFilterOutput.Num() == 1);
}

/**
 * N:N point filtering with mismatched input/filter counts.
 * Creates 3 inputs but only 2 filters. With bFilterEachInputWithEachFilterDataRespectively=true,
 * this should produce a cardinality error and generate no outputs.
 */
TEST_CASE_METHOD(FPCGAttributeFilterTest, "PCG::AttributeFilter::NtoN::Points::MismatchedCount", "[PCG][AttributeFilter]")
{
	TypedSettings->Operator = EPCGAttributeFilterOperator::Greater;
	TypedSettings->TargetAttribute.SetPointProperty(EPCGPointProperties::Density);
	TypedSettings->ThresholdAttribute.SetPointProperty(EPCGPointProperties::Density);
	TypedSettings->bUseConstantThreshold = false;
	TypedSettings->bUseSpatialQuery = false;
	TypedSettings->bFilterEachInputWithEachFilterDataRespectively = true;
	TypedSettings->bGenerateOutputDataEvenIfEmpty = false;

	for (int32 i = 0; i < 3; ++i)
	{
		FPCGTaggedData& TaggedData = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		TaggedData.Pin = PCGPinConstants::DefaultInputLabel;
		TaggedData.Data = GeneratePointDataWithSingleDensity(0.5f);
	}

	// Only 2 filters (mismatched)
	for (int32 i = 0; i < 2; ++i)
	{
		FPCGTaggedData& FilterTaggedData = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		FilterTaggedData.Pin = FilterLabel;
		FilterTaggedData.Data = GeneratePointDataWithSingleDensity(0.3f);
	}

	FSuppressErrorsScope SuppressErrors(*this);
	Execute();

	CHECK(NumErrors > 0);

	TArray<FPCGTaggedData> InFilterOutput = Context->OutputData.GetInputsByPin(InsideFilterLabel);
	TArray<FPCGTaggedData> OutFilterOutput = Context->OutputData.GetInputsByPin(OutsideFilterLabel);

	CHECK(InFilterOutput.Num() == 0);
	CHECK(OutFilterOutput.Num() == 0);
}

/**
 * N:N attribute filtering with matching input/filter counts.
 * Creates 3 param data inputs with int values [10, 50, 90] and 3 filter param data
 * with int thresholds [100, 30, 40]. Uses Greater operator.
 * Input 0 (10 > 100) fails; inputs 1 (50 > 30) and 2 (90 > 30) pass.
 */
TEST_CASE_METHOD(FPCGAttributeFilterTest, "PCG::AttributeFilter::NtoN::Attributes::MatchingCount", "[PCG][AttributeFilter]")
{
	TypedSettings->Operator = EPCGAttributeFilterOperator::Greater;
	TypedSettings->TargetAttribute.SetAttributeName(IntAttributeName);
	TypedSettings->ThresholdAttribute.SetAttributeName(IntAttributeName);
	TypedSettings->bUseConstantThreshold = false;
	TypedSettings->bFilterEachInputWithEachFilterDataRespectively = true;
	TypedSettings->bGenerateOutputDataEvenIfEmpty = false;

	const int32 InputValues[] = { 10, 50, 90 };
	for (int32 Value : InputValues)
	{
		FPCGTaggedData& TaggedData = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		TaggedData.Pin = PCGPinConstants::DefaultInputLabel;
		TaggedData.Data = GenerateAttributeSetWithSingleInt(Value);
	}

	FPCGTaggedData& FilterTaggedData = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
	FilterTaggedData.Pin = FilterLabel;
	FilterTaggedData.Data = GenerateAttributeSetWithSingleInt(100);
	
	FPCGTaggedData& FilterTaggedData2 = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
	FilterTaggedData2.Pin = FilterLabel;
	FilterTaggedData2.Data = GenerateAttributeSetWithSingleInt(30);
	
	FPCGTaggedData& FilterTaggedData3 = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
	FilterTaggedData3.Pin = FilterLabel;
	FilterTaggedData3.Data = GenerateAttributeSetWithSingleInt(40);

	Execute();

	TArray<FPCGTaggedData> InFilterOutput = Context->OutputData.GetInputsByPin(InsideFilterLabel);
	TArray<FPCGTaggedData> OutFilterOutput = Context->OutputData.GetInputsByPin(OutsideFilterLabel);

	// 50 > 30 and 90 > 40 pass; 10 > 100 fails
	CHECK(InFilterOutput.Num() == 2);
	CHECK(OutFilterOutput.Num() == 1);
}

/**
 * N:N attribute filtering with mismatched input/filter counts.
 * Creates 3 inputs but only 2 filters. Should produce a cardinality error.
 */
TEST_CASE_METHOD(FPCGAttributeFilterTest, "PCG::AttributeFilter::NtoN::Attributes::MismatchedCount", "[PCG][AttributeFilter]")
{
	TypedSettings->Operator = EPCGAttributeFilterOperator::Greater;
	TypedSettings->TargetAttribute.SetAttributeName(IntAttributeName);
	TypedSettings->ThresholdAttribute.SetAttributeName(IntAttributeName);
	TypedSettings->bUseConstantThreshold = false;
	TypedSettings->bFilterEachInputWithEachFilterDataRespectively = true;
	TypedSettings->bGenerateOutputDataEvenIfEmpty = false;

	for (int32 i = 0; i < 3; ++i)
	{
		FPCGTaggedData& TaggedData = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		TaggedData.Pin = PCGPinConstants::DefaultInputLabel;
		TaggedData.Data = GenerateAttributeSetWithSingleInt(50);
	}

	// Only 2 filters
	for (int32 i = 0; i < 2; ++i)
	{
		FPCGTaggedData& FilterTaggedData = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		FilterTaggedData.Pin = FilterLabel;
		FilterTaggedData.Data = GenerateAttributeSetWithSingleInt(20);
	}

	FSuppressErrorsScope SuppressErrors(*this);
	Execute();

	CHECK(NumErrors > 0);

	TArray<FPCGTaggedData> InFilterOutput = Context->OutputData.GetInputsByPin(InsideFilterLabel);
	TArray<FPCGTaggedData> OutFilterOutput = Context->OutputData.GetInputsByPin(OutsideFilterLabel);

	CHECK(InFilterOutput.Num() == 0);
	CHECK(OutFilterOutput.Num() == 0);
}

/**
 * N:N range filtering with matching input/filter counts.
 * Creates 3 param data inputs with double values [0.1, 0.5, 0.9] and 3 min/max filter pairs.
 * Min thresholds: [0.0, 0.0, 0.0], Max thresholds: [0.3, 0.3, 0.3].
 * Input 0 (0.1 in [0, 0.3]) passes; inputs 1 (0.5) and 2 (0.9) fail.
 */
TEST_CASE_METHOD(FPCGAttributeFilterRangeTest, "PCG::AttributeFilter::NtoN::Range::MatchingCount", "[PCG][AttributeFilter]")
{
	TypedSettings->TargetAttribute.SetAttributeName(DoubleAttributeName);
	TypedSettings->MinThreshold.ThresholdAttribute.SetAttributeName(DoubleAttributeName);
	TypedSettings->MaxThreshold.ThresholdAttribute.SetAttributeName(DoubleAttributeName);
	TypedSettings->MinThreshold.bUseConstantThreshold = false;
	TypedSettings->MaxThreshold.bUseConstantThreshold = false;
	TypedSettings->MinThreshold.bInclusive = true;
	TypedSettings->MaxThreshold.bInclusive = true;
	TypedSettings->MinThreshold.bFilterEachInputWithEachFilterDataRespectively = true;
	TypedSettings->MaxThreshold.bFilterEachInputWithEachFilterDataRespectively = true;
	TypedSettings->bGenerateOutputDataEvenIfEmpty = false;

	const double InputValues[] = { 0.1, 0.5, 0.9 };
	for (double Value : InputValues)
	{
		FPCGTaggedData& TaggedData = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		TaggedData.Pin = PCGPinConstants::DefaultInputLabel;
		TaggedData.Data = GenerateAttributeSetWithSingleValue(Value);
	}

	// 3 min filters all with value 0.0
	for (int32 i = 0; i < 3; ++i)
	{
		FPCGTaggedData& FilterTaggedData = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		FilterTaggedData.Pin = MinFilterLabel;
		FilterTaggedData.Data = GenerateAttributeSetWithSingleValue(0.0);
	}

	// 3 max filters all with value 0.3
	for (int32 i = 0; i < 3; ++i)
	{
		FPCGTaggedData& FilterTaggedData = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		FilterTaggedData.Pin = MaxFilterLabel;
		FilterTaggedData.Data = GenerateAttributeSetWithSingleValue(0.3);
	}

	Execute();

	TArray<FPCGTaggedData> InFilterOutput = Context->OutputData.GetInputsByPin(InsideFilterLabel);
	TArray<FPCGTaggedData> OutFilterOutput = Context->OutputData.GetInputsByPin(OutsideFilterLabel);

	// Only 0.1 is in [0, 0.3], 0.5 and 0.9 are not
	CHECK(InFilterOutput.Num() == 1);
	CHECK(OutFilterOutput.Num() == 2);
}

/**
 * N:N range filtering with mismatched input/filter counts.
 * Creates 3 inputs but only 2 min filters. Should produce a cardinality error.
 */
TEST_CASE_METHOD(FPCGAttributeFilterRangeTest, "PCG::AttributeFilter::NtoN::Range::MismatchedCount", "[PCG][AttributeFilter]")
{
	TypedSettings->TargetAttribute.SetAttributeName(DoubleAttributeName);
	TypedSettings->MinThreshold.ThresholdAttribute.SetAttributeName(DoubleAttributeName);
	TypedSettings->MaxThreshold.ThresholdAttribute.SetAttributeName(DoubleAttributeName);
	TypedSettings->MinThreshold.bUseConstantThreshold = false;
	TypedSettings->MaxThreshold.bUseConstantThreshold = false;
	TypedSettings->MinThreshold.bInclusive = true;
	TypedSettings->MaxThreshold.bInclusive = true;
	TypedSettings->MinThreshold.bFilterEachInputWithEachFilterDataRespectively = true;
	TypedSettings->MaxThreshold.bFilterEachInputWithEachFilterDataRespectively = true;
	TypedSettings->bGenerateOutputDataEvenIfEmpty = false;

	for (int32 i = 0; i < 3; ++i)
	{
		FPCGTaggedData& TaggedData = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		TaggedData.Pin = PCGPinConstants::DefaultInputLabel;
		TaggedData.Data = GenerateAttributeSetWithSingleValue(0.5);
	}

	// Only 2 min filters (mismatched)
	for (int32 i = 0; i < 2; ++i)
	{
		FPCGTaggedData& FilterTaggedData = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		FilterTaggedData.Pin = MinFilterLabel;
		FilterTaggedData.Data = GenerateAttributeSetWithSingleValue(0.0);
	}

	// 3 max filters
	for (int32 i = 0; i < 3; ++i)
	{
		FPCGTaggedData& FilterTaggedData = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		FilterTaggedData.Pin = MaxFilterLabel;
		FilterTaggedData.Data = GenerateAttributeSetWithSingleValue(1.0);
	}

	FSuppressErrorsScope SuppressErrors(*this);
	Execute();

	CHECK(NumErrors > 0);

	TArray<FPCGTaggedData> InFilterOutput = Context->OutputData.GetInputsByPin(InsideFilterLabel);
	TArray<FPCGTaggedData> OutFilterOutput = Context->OutputData.GetInputsByPin(OutsideFilterLabel);

	CHECK(InFilterOutput.Num() == 0);
	CHECK(OutFilterOutput.Num() == 0);
}
