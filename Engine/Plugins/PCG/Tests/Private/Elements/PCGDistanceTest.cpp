// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGTestsCommon.h"

#include "Data/PCGPointArrayData.h"
#include "Elements/PCGDistance.h"
#include "Helpers/PCGPointHelpers.h"
#include "Metadata/PCGMetadataAttribute.h"

#include <catch2/catch_test_macros.hpp>

#include "TestHarness.h"

class FPCGDistanceBaseTest : public PCGTests::FPCGSingleElementBaseTest<UPCGDistanceSettings>
{
public:
	FPCGDistanceBaseTest()
		: FPCGSingleElementBaseTest()
	{
		TypedSettings->SourceShape = PCGDistanceShape::Center;
		TypedSettings->TargetShape = PCGDistanceShape::Center;
	}

protected:
	UPCGPointArrayData* CreateSourcePoints()
	{
		UPCGPointArrayData* SourceData = NewObject<UPCGPointArrayData>();
		SourceData->SetNumPoints(2);

		TPCGValueRange<FTransform> TransformRange = SourceData->GetTransformValueRange();
		TransformRange[0].SetTranslation(FVector(100, 0, 0));
		TransformRange[1].SetTranslation(FVector(0, 50, 0));

		return SourceData;
	}

	UPCGPointArrayData* CreateSingleTargetPoint(const FVector& Position)
	{
		UPCGPointArrayData* TargetData = NewObject<UPCGPointArrayData>();
		TargetData->SetNumPoints(1);

		TPCGValueRange<FTransform> TransformRange = TargetData->GetTransformValueRange();
		TransformRange[0].SetTranslation(Position);

		return TargetData;
	}

	UPCGPointArrayData* CreateTargetPointWithBounds(const FVector& Position, const FVector& Extents)
	{
		UPCGPointArrayData* TargetData = CreateSingleTargetPoint(Position);
		TargetData->AllocateProperties(EPCGPointNativeProperties::BoundsMin | EPCGPointNativeProperties::BoundsMax);
		FPCGPointValueRanges ValueRanges(TargetData, /*bAllocate=*/false);
		PCGPointHelpers::SetExtents(Extents, ValueRanges.BoundsMinRange[0], ValueRanges.BoundsMaxRange[0]);

		return TargetData;
	}

	void AddSourceData()
	{
		FPCGTaggedData& SourcePin = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		SourcePin.Pin = PCGDistance::SourceLabel;
		SourcePin.Data = CreateSourcePoints();
	}

	void AddTargetData(UPCGData* InTargetData)
	{
		FPCGTaggedData& TargetPin = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		TargetPin.Pin = PCGDistance::TargetLabel;
		TargetPin.Data = InTargetData;
	}

	const UPCGBasePointData* ExecuteAndGetOutput()
	{
		ExecuteElement();
		CHECK(NumErrors == 0);
		CHECK(NumWarnings == 0);

		const TArray<FPCGTaggedData>& Outputs = Context->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);
		REQUIRE_EQUAL(Outputs.Num(), 1);

		const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);
		REQUIRE(OutPointData != nullptr);
		REQUIRE_EQUAL(OutPointData->GetNumPoints(), 2);

		return OutPointData;
	}
};

TEST_CASE_METHOD(FPCGDistanceBaseTest, "PCG::Distance::PointToPoint", "[PCG][Distance]")
{
	AddSourceData();
	AddTargetData(CreateSingleTargetPoint(FVector(0, 0, 0)));

	const UPCGBasePointData* OutPointData = ExecuteAndGetOutput();

	const FPCGMetadataAttribute<double>* DistanceAttribute = OutPointData->Metadata->GetConstTypedAttribute<double>(TypedSettings->OutputAttribute.GetAttributeName());
	REQUIRE(DistanceAttribute != nullptr);

	CHECK(DistanceAttribute->GetValueFromItemKey(OutPointData->GetMetadataEntry(0)) == Catch::Approx(100.0).margin(0.01));
	CHECK(DistanceAttribute->GetValueFromItemKey(OutPointData->GetMetadataEntry(1)) == Catch::Approx(50.0).margin(0.01));
}

TEST_CASE_METHOD(FPCGDistanceBaseTest, "PCG::Distance::SetDensity", "[PCG][Distance]")
{
	TypedSettings->OutputAttribute = FPCGAttributePropertySelector();
	TypedSettings->bSetDensity = true;
	TypedSettings->MaximumDistance = 200.0;

	AddSourceData();
	AddTargetData(CreateSingleTargetPoint(FVector(0, 0, 0)));

	const UPCGBasePointData* OutPointData = ExecuteAndGetOutput();

	CHECK(OutPointData->GetDensity(0) == Catch::Approx(0.5f).margin(0.01f));
	CHECK(OutPointData->GetDensity(1) == Catch::Approx(0.25f).margin(0.01f));
}

TEST_CASE_METHOD(FPCGDistanceBaseTest, "PCG::Distance::PointToSphere", "[PCG][Distance]")
{
	TypedSettings->TargetShape = PCGDistanceShape::SphereBounds;

	const FVector TargetExtents(10.0f);
	const double TargetPointRadius = TargetExtents.Length();

	AddSourceData();
	AddTargetData(CreateTargetPointWithBounds(FVector(0, 0, 0), TargetExtents));

	const UPCGBasePointData* OutPointData = ExecuteAndGetOutput();

	const FPCGMetadataAttribute<double>* DistanceAttribute = OutPointData->Metadata->GetConstTypedAttribute<double>(TypedSettings->OutputAttribute.GetAttributeName());
	REQUIRE(DistanceAttribute != nullptr);

	CHECK(DistanceAttribute->GetValueFromItemKey(OutPointData->GetMetadataEntry(0)) == Catch::Approx(100.0 - TargetPointRadius).margin(0.01));
	CHECK(DistanceAttribute->GetValueFromItemKey(OutPointData->GetMetadataEntry(1)) == Catch::Approx(50.0 - TargetPointRadius).margin(0.01));
}

TEST_CASE_METHOD(FPCGDistanceBaseTest, "PCG::Distance::PointToBox", "[PCG][Distance]")
{
	TypedSettings->TargetShape = PCGDistanceShape::BoxBounds;

	// Should put the box 10 units away from each point
	const FVector TargetExtents(90.0f, 40.0f, 10.0f);

	AddSourceData();
	AddTargetData(CreateTargetPointWithBounds(FVector(0, 0, 0), TargetExtents));

	const UPCGBasePointData* OutPointData = ExecuteAndGetOutput();

	const FPCGMetadataAttribute<double>* DistanceAttribute = OutPointData->Metadata->GetConstTypedAttribute<double>(TypedSettings->OutputAttribute.GetAttributeName());
	REQUIRE(DistanceAttribute != nullptr);

	CHECK(DistanceAttribute->GetValueFromItemKey(OutPointData->GetMetadataEntry(0)) == Catch::Approx(10.0).margin(0.01));
	CHECK(DistanceAttribute->GetValueFromItemKey(OutPointData->GetMetadataEntry(1)) == Catch::Approx(10.0).margin(0.01));
}
