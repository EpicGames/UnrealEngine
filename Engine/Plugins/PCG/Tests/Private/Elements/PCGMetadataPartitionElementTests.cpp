// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGTestsCommon.h"

#include "PCGContext.h"
#include "PCGParamData.h"
#include "Data/PCGPointArrayData.h"
#include "Elements/Metadata/PCGMetadataPartition.h"

#include <catch2/catch_test_macros.hpp>

#include "TestHarness.h"

/**
 * Element-level tests for partition index assignment.
 * Exercises bAssignIndexPartition and bDoNotPartition settings on the element.
 */
TEST_CASE_METHOD(PCGTests::FPCGSingleElementBaseTest<UPCGMetadataPartitionSettings>, "PCG::AttributePartition::Element::PartitionIndex", "[PCG][AttributePartition]")
{
	static const FName PartitionIndexAttributeName = TEXT("PartitionIndex");

	TypedSettings->PartitionAttributeSelectors[0].SetPointProperty(EPCGPointProperties::Density);
	TypedSettings->bAssignIndexPartition = true;
	TypedSettings->PartitionIndexAttributeName = PartitionIndexAttributeName;

	UPCGPointArrayData* PointData = NewObject<UPCGPointArrayData>();
	PointData->SetNumPoints(100);
	TPCGValueRange<float> DensityRange = PointData->GetDensityValueRange();
	for (int32 i = 0; i < 100; ++i)
	{
		DensityRange[i] = static_cast<float>(i % 10);
	}

	FPCGTaggedData& InputTaggedData = InputData.TaggedData.Emplace_GetRef();
	InputTaggedData.Data = PointData;
	InputTaggedData.Pin = PCGPinConstants::DefaultInputLabel;

	SECTION("Partition with index assigns correct index per partition")
	{
		TypedSettings->bDoNotPartition = false;

		ExecuteElement();

		REQUIRE_EQUAL(Context->OutputData.TaggedData.Num(), 10);

		for (int32 i = 0; i < Context->OutputData.TaggedData.Num(); ++i)
		{
			const UPCGBasePointData* OutputPointData = Cast<const UPCGBasePointData>(Context->OutputData.TaggedData[i].Data);
			REQUIRE_NOT_EQUAL(OutputPointData, nullptr);
			CHECK(OutputPointData->GetNumPoints() == 10);

			const FPCGMetadataAttribute<int32>* PartitionIndexAttribute = OutputPointData->Metadata->GetConstTypedAttribute<int32>(PartitionIndexAttributeName);
			REQUIRE_NOT_EQUAL(PartitionIndexAttribute, nullptr);

			const TConstPCGValueRange<float> OutDensityRange = OutputPointData->GetConstDensityValueRange();
			const TConstPCGValueRange<int64> MetadataEntryRange = OutputPointData->GetConstMetadataEntryValueRange();

			const float FirstDensity = OutDensityRange[0];
			for (int32 j = 0; j < OutputPointData->GetNumPoints(); ++j)
			{
				CHECK(OutDensityRange[j] == FirstDensity);
				CHECK(PartitionIndexAttribute->GetValueFromItemKey(MetadataEntryRange[j]) == i);
			}
		}
	}

	SECTION("No partition with index assigns bucket index without splitting")
	{
		TypedSettings->bDoNotPartition = true;

		ExecuteElement();

		REQUIRE_EQUAL(Context->OutputData.TaggedData.Num(), 1);

		const UPCGBasePointData* OutputPointData = Cast<const UPCGBasePointData>(Context->OutputData.TaggedData[0].Data);
		REQUIRE_NOT_EQUAL(OutputPointData, nullptr);
		CHECK(OutputPointData->GetNumPoints() == 100);

		const FPCGMetadataAttribute<int32>* PartitionIndexAttribute = OutputPointData->Metadata->GetConstTypedAttribute<int32>(PartitionIndexAttributeName);
		REQUIRE_NOT_EQUAL(PartitionIndexAttribute, nullptr);

		const TConstPCGValueRange<float> OutDensityRange = OutputPointData->GetConstDensityValueRange();
		const TConstPCGValueRange<int64> MetadataEntryRange = OutputPointData->GetConstMetadataEntryValueRange();

		for (int32 j = 0; j < OutputPointData->GetNumPoints(); ++j)
		{
			const int32 ExpectedIndex = FMath::FloorToInt32(OutDensityRange[j]);
			CHECK(PartitionIndexAttribute->GetValueFromItemKey(MetadataEntryRange[j]) == ExpectedIndex);
		}
	}
}

/**
 * Element-level test for parameter override of partition selectors.
 * Exercises the PartitionAttributeNames override pin with comma-separated selectors.
 */
TEST_CASE_METHOD(PCGTests::FPCGSingleElementBaseTest<UPCGMetadataPartitionSettings>, "PCG::AttributePartition::Element::ParameterOverride", "[PCG][AttributePartition]")
{
	UPCGParamData* OverrideParamData = NewObject<UPCGParamData>();
	FPCGMetadataAttribute<FString>* OverrideAttribute = OverrideParamData->Metadata->CreateAttribute<FString>(
		TEXT(""), TEXT("$Position.X,$Position.Y"), false, false);
	check(OverrideAttribute);
	OverrideParamData->Metadata->AddEntry();

	FPCGTaggedData& OverrideTaggedData = InputData.TaggedData.Emplace_GetRef();
	OverrideTaggedData.Data = OverrideParamData;
	OverrideTaggedData.Pin = GET_MEMBER_NAME_CHECKED(UPCGMetadataPartitionSettings, PartitionAttributeNames);

	UPCGPointArrayData* PointData = NewObject<UPCGPointArrayData>();
	PointData->SetNumPoints(20);
	TPCGValueRange<FTransform> TransformRange = PointData->GetTransformValueRange();
	for (int32 i = 0; i < 20; ++i)
	{
		// Small jitter on Y for second half to verify floating point equality tolerance
		const double Jitter = i >= 10 ? UE_DOUBLE_SMALL_NUMBER : 0.0;
		TransformRange[i].SetLocation(FVector(i % 10, i % 5 + Jitter, 0.0));
	}

	FPCGTaggedData& InputTaggedData = InputData.TaggedData.Emplace_GetRef();
	InputTaggedData.Data = PointData;
	InputTaggedData.Pin = PCGPinConstants::DefaultInputLabel;

	ExecuteElement();

	REQUIRE_EQUAL(Context->OutputData.TaggedData.Num(), 10);

	for (int32 i = 0; i < Context->OutputData.TaggedData.Num(); ++i)
	{
		const UPCGBasePointData* OutputPointData = Cast<const UPCGBasePointData>(Context->OutputData.TaggedData[i].Data);
		REQUIRE_NOT_EQUAL(OutputPointData, nullptr);
		CHECK(OutputPointData->GetNumPoints() == 2);

		const TConstPCGValueRange<FTransform> OutTransformRange = OutputPointData->GetConstTransformValueRange();
		const FVector FirstLoc = OutTransformRange[0].GetLocation();
		for (int32 j = 1; j < OutputPointData->GetNumPoints(); ++j)
		{
			CHECK(FirstLoc.Equals(OutTransformRange[j].GetLocation()));
		}
	}
}
