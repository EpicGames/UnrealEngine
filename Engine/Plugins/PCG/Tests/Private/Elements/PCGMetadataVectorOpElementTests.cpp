// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "TestHarness.h"

#include <catch2/generators/catch_generators.hpp>

#include "PCGContext.h"
#include "PCGModule.h"
#include "PCGParamData.h"
#include "PCGTestsCommon.h"
#include "Elements/Metadata/PCGMetadataVectorOpElement.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

namespace PCGTests
{
class FPCGVectorOpsBaseTest: public PCGTests::FPCGSingleElementBaseTest<UPCGMetadataVectorSettings>
{
public:
	FPCGVectorOpsBaseTest()
		: FPCGSingleElementBaseTest()
	{}
	virtual ~FPCGVectorOpsBaseTest() override = default;
	
	inline static const FName DefaultAttributeName = TEXT("TestAttribute");
	inline static const FVector DefaultTranslation = {1.0, 10.0, 100.0};
	inline static const FPCGAttributePropertyInputSelector DefaultSelector = FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(DefaultAttributeName);

	struct TestParams
	{
		explicit TestParams(const EPCGMetadataVectorOperation Operation, TArray<TArray<FPCGTaggedData>>&& InputDataByPin, const int32 PinToForward = 0) 
			: Operation(Operation)
			, InputDataByPin(std::move(InputDataByPin))
			, PinToForward(PinToForward)
		{}

		EPCGMetadataVectorOperation Operation;
		TArray<TArray<FPCGTaggedData>> InputDataByPin;
		int32 PinToForward = 0;
	};

	template <typename T>
	struct FExpectedValues
	{
		explicit FExpectedValues(const TArray<T>& InExpectedDefaultValues, const TArray<T>& InExpectedResultValues) 
			: ExpectedDefaultValues(InExpectedDefaultValues)
			, ExpectedResultValues(InExpectedResultValues)
		{}
		explicit FExpectedValues(const T& ExpectedDefaultValue, const TArray<T>& InExpectedResultValues) 
			: ExpectedResultValues(InExpectedResultValues)
		{
			for (int Index = 0; Index < ExpectedResultValues.Num(); ++Index)
			{
				ExpectedDefaultValues.Add(ExpectedDefaultValue);
			}
		}

		TArray<T> ExpectedDefaultValues;
		TArray<T> ExpectedResultValues;
	};
	
	void RunTest(const TestParams& Params, const FPCGAttributePropertyInputSelector& InputSelector = DefaultSelector)
	{
		const EPCGMetadataVectorOperation& Operation = Params.Operation;
		const TArray<TArray<FPCGTaggedData>>& InputDataByPin = Params.InputDataByPin;
		const int32 PinToForward = Params.PinToForward;
		
		// Assign the element settings here. Use the same selector for all input sources, since the user can only select one
		TypedSettings->Operation = Params.Operation;
		TypedSettings->InputSource1 = InputSelector;
		TypedSettings->InputSource2 = InputSelector;
		TypedSettings->InputSource3 = InputSelector;
		TypedSettings->OutputTarget.SetAttributeName(DefaultAttributeName);
		TypedSettings->OutputDataFromPin = TypedSettings->GetInputPinLabel(PinToForward);

		// Assign pin labels based on incoming data for the execution
		for (int PinIndex = 0; PinIndex < InputDataByPin.Num(); ++PinIndex)
		{
			TArray<FPCGTaggedData> CurPinInputData = InputDataByPin[PinIndex];
			for (FPCGTaggedData& Data : CurPinInputData)
			{
				Data.Pin = TypedSettings->GetInputPinLabel(PinIndex);
			}

			InputData.TaggedData.Append(std::move(CurPinInputData));
		}
		
		ExecuteElement();
	}

	// Simply apply function to generate some expected output
	template<typename In1, typename In2, typename Out>
	static TArray<Out> GenerateResults(const TArray<In1>& InArray1, const TArray<In2>& InArray2, const TFunction<Out(In1, In2)>& GenerateFunc)
	{
		TArray<Out> Result;
		for (int Index = 0; Index < InArray1.Num(); ++Index)
		{
			Result.Add(GenerateFunc(InArray1[Index], InArray2[Index]));
		}
		return Result;
	}

	// Generate Some data For tests: [Start, Start + Inc, Start + 2*Inc, ... Start+(Num-1)*Inc]
	template<typename T>
	static TArray<T> GenerateValues(const T& Start, const T& Inc, int Num)
	{
		TArray<T> Result;
		T Cur = Start;
		for (int Index = 0; Index < Num; ++Index)
		{
			Result.Add(Cur);
			Cur += Inc;
		}
		return Result;
	}
	
	// Generate param data of the tested type
	template <typename T>
	TArray<TArray<FPCGTaggedData>> GenerateParamData(const TArray<TArray<T>>& ValuesByPin)
	{
		TArray<TArray<FPCGTaggedData>> GeneratedData;

		for (TArray<T> Values : ValuesByPin)
		{
			TArray<FPCGTaggedData>& TaggedData = GeneratedData.Emplace_GetRef();

			for (const T& Value : Values)
			{
				TObjectPtr<UPCGParamData> ParamData = NewObject<UPCGParamData>();
				TaggedData.Emplace_GetRef().Data = ParamData;

				PCGMetadataEntryKey EntryKey = ParamData->Metadata->AddEntry();
				constexpr bool bAllowsInterpolation = true;
				constexpr bool bOverrideParent = false;

				FPCGMetadataAttribute<T>* Attribute = ParamData->Metadata->FindOrCreateAttribute<T>(DefaultAttributeName, PCG::Private::MetadataTraits<T>::ZeroValue(), bAllowsInterpolation, bOverrideParent);
				check(Attribute);

				Attribute->SetValue(EntryKey, Value);
			}
		}

		return GeneratedData;
	}
	
	// Expects Input Data to be an array of Tagged Data on each index per pin and a single value in each Input Data (1 attribute or 1 point data)
	template <typename T>
	void ExecuteTest(const TestParams& Params, const FExpectedValues<T>& Expected, const FPCGAttributePropertyInputSelector& InputSelector = DefaultSelector)
	{
		RunTest(Params, InputSelector);
		CHECK(NumErrors == 0);
		CHECK(NumWarnings == 0);
		
		const EPCGMetadataVectorOperation& Operation = Params.Operation;
		const TArray<TArray<FPCGTaggedData>>& InputDataByPin = Params.InputDataByPin;
		const TArray<T>& ExpectedDefaults = Expected.ExpectedDefaultValues;
		const TArray<T>& ExpectedValues = Expected.ExpectedResultValues;

		REQUIRE(!InputDataByPin.IsEmpty());
		REQUIRE(!ExpectedDefaults.IsEmpty());
		REQUIRE(!ExpectedValues.IsEmpty());

		CHECK(!Context->OutputData.TaggedData.IsEmpty());

		CHECK(Context->OutputData.TaggedData.Num() == ExpectedValues.Num());

		// Cycle through expected results and validate
		for (int I = 0; I < ExpectedValues.Num(); ++I)
		{
			const UPCGData* OutputData = Context->OutputData.TaggedData[I].Data;
			REQUIRE(OutputData != nullptr);

			FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateAttributeSelector(DefaultAttributeName);
			const TUniquePtr<const IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(OutputData, Selector);
			const TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(OutputData, Selector);

			REQUIRE(Accessor.IsValid());
			REQUIRE(Keys.IsValid());

			REQUIRE(Accessor->GetUnderlyingType() == PCG::Private::MetadataTypes<T>::Id);

			T DefaultValue{};
			T OutputValue{};

			FPCGAttributeAccessorKeysEntries DefaultKey(PCGInvalidEntryKey);
			CHECK(Accessor->Get(DefaultValue, 0, DefaultKey));

			// We're only testing one attribute for each iteration
			CHECK(Accessor->Get(OutputValue, 0, *Keys));

			CHECK_THAT(DefaultValue, ApproxMatcher(ExpectedDefaults[I]));
			CHECK_THAT(OutputValue, ApproxMatcher(ExpectedValues[I]));
		}
	}
};
}

TEST_CASE_METHOD(PCGTests::FPCGVectorOpsBaseTest, "PCG::VectorOps::Cross", "[PCG][PathfindingElement]")
{
	FVector DefaultValue{0,0,0};
	auto [InputLeft, InputRight, Output] = GENERATE( 
			table<FVector, FVector, FVector>({
				{{1, 0, 0}, {0, 1, 0}, {0,0,1}},
				{{0, 1, 0}, {1, 0, 0}, {0,0,-1}},
				{{1, 2, 3}, {2, 4, 6}, {0, 0, 0}},
				{{1, 2, 3}, {4, 5, 6}, {-3, 6, -3}},
				{{0, 0, 0}, {6, 7, 8}, {0,0,0}},
				{{2, 0, 0}, {0, 3, 0}, {0,0,6}},
			})
		);
	
	DYNAMIC_SECTION(*FString::Format(TEXT("Cross product with ({0}) and ({1})"), {InputLeft.ToString(), InputRight.ToString()}))
	{
		ExecuteTest<FVector>(TestParams(EPCGMetadataVectorOperation::Cross, 
		GenerateParamData<FVector>({{InputLeft},{InputRight}})), 
		FExpectedValues{
			TArray({DefaultValue}), 
			TArray({Output})});		
	}
}

TEST_CASE_METHOD(PCGTests::FPCGVectorOpsBaseTest, "PCG::VectorOps::RotatorBetweenVec", "[PCG][PathfindingElement]")
{
	FRotator DefaultValue{0,0,0};
	SECTION("Test specific values With Vec3")
	{
		auto [InputLeft, InputRight, Output] = GENERATE( 
				table<FVector, FVector, FRotator>({
					{{0, 0, 0}, {0, 1, 0}, {0,0,0}},
					{{1, 0, 0}, {0, 0, 0}, {0,0,0}},
					{{1, 1, 0}, {2, 2, 0}, {0,0,0}},
					{{1, 0, 0}, {0, 1, 0}, {0,90,0}},
					{{1, 0, 0}, {0, 0, 1}, {90,0,0}},
				})
			);
	
		DYNAMIC_SECTION(*FString::Format(TEXT("Rotator angles between vector with ({0}) and ({1})"), {InputLeft.ToString(), InputRight.ToString()}))
		{
			// Note that this test failing might not mean the result is wrong. There is not in general only one right result for this operation. If it fails, 
			//   it might means that we need to check the results differently (By rotating the src vector with the result, and checking this with the second vector). 
			//   But for now, this test is enough.
			ExecuteTest(TestParams(EPCGMetadataVectorOperation::RotatorBetweenVectors, 
			GenerateParamData<FVector>({{InputLeft},{InputRight}})), 
			FExpectedValues{
				TArray({DefaultValue}), 
				TArray({Output})});		
		}
	}
	SECTION("Test specific values With Vec2 and Vec3")
	{
		auto [InputLeft, InputRight, Output] = GENERATE( 
				table<FVector2D, FVector2D, FRotator>({
					{{0, 0}, {0, 1}, {0,0,0}},
					{{1, 0}, {0, 0}, {0,0,0}},
					{{1, 1}, {2, 2}, {0,0,0}},
					{{1, 0}, {0, 1}, {0,90,0}},
				})
			);
	
		DYNAMIC_SECTION(*FString::Format(TEXT("Rotator angles between vector with ({0}) and ({1})"), {InputLeft.ToString(), InputRight.ToString()}))
		{
			// Note that this test failing might not mean the result is wrong. There is not in general only one right result for this operation. If it fails, 
			//   it might means that we need to check the results differently (By rotating the src vector with the result, and checking this with the second vector). 
			//   But for now, this test is enough.
			ExecuteTest(TestParams(EPCGMetadataVectorOperation::RotatorBetweenVectors, 
			GenerateParamData<FVector2D>({{InputLeft},{InputRight}})), 
			FExpectedValues{
				TArray({DefaultValue}), 
				TArray({Output})});		
		}
	}
	SECTION("Test values in data")
	{
		TArray<FVector> InputA = GenerateValues<FVector>({0, 1, 0}, {1, 1, 0}, 15);
		TArray<FVector> InputB = GenerateValues<FVector>({1, 1, 0}, {0, 1, 0}, 15);
		TArray<FRotator> Output = GenerateResults<FVector, FVector, FRotator>(InputA, InputB, [](const FVector& A, const FVector& B) -> FRotator {return FQuat::FindBetween(A, B).Rotator();});
		SECTION("In one Data")
		{
			ExecuteTest(TestParams(EPCGMetadataVectorOperation::RotatorBetweenVectors, GenerateParamData<FVector>({InputA,InputB})), FExpectedValues<FRotator>{ DefaultValue, Output});		
		}
	}
	SECTION("Error types")
	{
		TArray<FVector4> InputA = GenerateValues<FVector4>(FVector4{0, 1, 0, 1}, FVector4{1, 1, 0, 1}, 5);
		TArray<FVector4> InputB = GenerateValues<FVector4>(FVector4{1, 1, 0, 1}, FVector4{0, 1, 0, 1}, 5);

		
		FSuppressErrorsScope SuppressErrors(*this);
		RunTest(TestParams(EPCGMetadataVectorOperation::RotatorBetweenVectors, GenerateParamData<FVector4>({InputA,InputB})));
		CHECK(NumErrors > 0);
	}
}


TEST_CASE_METHOD(PCGTests::FPCGVectorOpsBaseTest, "PCG::VectorOps::EulerBetweenVec", "[PCG][PathfindingElement]")
{
	FVector DefaultValue{0,0,0};
	SECTION("Test specific values With Vec3")
	{
		auto [InputLeft, InputRight, Output] = GENERATE( 
				table<FVector, FVector, FVector>({
					{{0, 0, 0}, {0, 1, 0}, {0,0,0}},
					{{1, 0, 0}, {0, 0, 0}, {0,0,0}},
					{{1, 1, 0}, {2, 2, 0}, {0,0,0}},
					{{1, 0, 0}, {0, 1, 0}, {0,0,90}},
					{{1, 0, 0}, {0, 0, 1}, {0,90,0}},
				})
			);
	
		DYNAMIC_SECTION(*FString::Format(TEXT("Euler angles between vector with ({0}) and ({1})"), {InputLeft.ToString(), InputRight.ToString()}))
		{
			// Not that this test failing might not mean the result is wrong. There is not in general only one right result for this operation. If it fails, 
			//   it might means that we need to check the results differently (By rotating the src vector with the result, and checking this with the second vector). 
			//   But for now, this test is enough.
			ExecuteTest(TestParams(EPCGMetadataVectorOperation::EulerBetweenVectors, 
			GenerateParamData<FVector>({{InputLeft},{InputRight}})), 
			FExpectedValues{
				TArray({DefaultValue}), 
				TArray({Output})});		
		}
	}
	SECTION("Test specific values With Vec2 and Vec3")
	{
		auto [InputLeft, InputRight, Output] = GENERATE( 
				table<FVector2D, FVector2D, FVector>({
					{{0, 0}, {0, 1}, {0,0,0}},
					{{1, 0}, {0, 0}, {0,0,0}},
					{{1, 1}, {2, 2}, {0,0,0}},
					{{1, 0}, {0, 1}, {0,0,90}},
				})
			);
	
		DYNAMIC_SECTION(*FString::Format(TEXT("Euler angles between vector with ({0}) and ({1})"), {InputLeft.ToString(), InputRight.ToString()}))
		{
			// Not that this test failing might not mean the result is wrong. There is not in general only one right result for this operation. If it fails, 
			//   it might means that we need to check the results differently (By rotating the src vector with the result, and checking this with the second vector). 
			//   But for now, this test is enough.
			ExecuteTest(TestParams(EPCGMetadataVectorOperation::EulerBetweenVectors, 
			GenerateParamData<FVector2D>({{InputLeft},{InputRight}})), 
			FExpectedValues{
				TArray({DefaultValue}), 
				TArray({Output})});		
		}
	}
	SECTION("Test values in data")
	{
		TArray<FVector> InputA = GenerateValues<FVector>({0, 1, 0}, {1, 1, 0}, 15);
		TArray<FVector> InputB = GenerateValues<FVector>({1, 1, 0}, {0, 1, 0}, 15);
		TArray<FVector> Output = GenerateResults<FVector, FVector, FVector>(InputA, InputB, [](const FVector& A, const FVector& B) -> FVector {return FQuat::FindBetween(A, B).Euler();});
		SECTION("In one Data")
		{
			ExecuteTest(TestParams(EPCGMetadataVectorOperation::EulerBetweenVectors, GenerateParamData<FVector>({InputA,InputB})), FExpectedValues<FVector>{ DefaultValue, Output});		
		}
	}
	SECTION("Error types")
	{
		TArray<FVector4> InputA = GenerateValues<FVector4>(FVector4{0, 1, 0, 1}, FVector4{1, 1, 0, 1}, 5);
		TArray<FVector4> InputB = GenerateValues<FVector4>(FVector4{1, 1, 0, 1}, FVector4{0, 1, 0, 1}, 5);

		
		FSuppressErrorsScope SuppressErrors(*this);
		RunTest(TestParams(EPCGMetadataVectorOperation::EulerBetweenVectors, GenerateParamData<FVector4>({InputA,InputB})));
		CHECK(NumErrors > 0);
	}
}

TEST_CASE_METHOD(PCGTests::FPCGVectorOpsBaseTest, "PCG::VectorOps::AngleBetween2DVec", "[PCG][PathfindingElement]")
{
	double DefaultValue{0};
	SECTION("Test specific values With Vec2")
	{
		auto [InputLeft, InputRight, Output] = GENERATE( 
				table<FVector2D, FVector2D, double>({
					{{0, 0}, {0, 1}, 0},
					{{1, 0}, {0, 0}, 0},
					{{1, 1}, {2, 2}, 0},
					{{1, 0}, {0, 1}, 90},
					{{1, 0}, {0, -1}, -90},
					{{1, 0}, {-1, 0}, 180},
					{{1, 0}, {1, 1}, 45},
				})
			);
	
		DYNAMIC_SECTION(*FString::Format(TEXT("Angle between vector with ({0}) and ({1})"), {InputLeft.ToString(), InputRight.ToString()}))
		{
			// Not that this test failing might not mean the result is wrong. There is not in general only one right result for this operation. If it fails, 
			//   it might means that we need to check the results differently (By rotating the src vector with the result, and checking this with the second vector). 
			//   But for now, this test is enough.
			ExecuteTest(TestParams(EPCGMetadataVectorOperation::AngleBetweenVector2D, 
			GenerateParamData<FVector2D>({{InputLeft},{InputRight}})), 
			FExpectedValues{
				TArray({DefaultValue}), 
				TArray({Output})});		
		}
	}
	SECTION("Test values in data")
	{
		TArray<FVector2D> InputA = GenerateValues<FVector2D>({0, 1}, {1, 1}, 15);
		TArray<FVector2D> InputB = GenerateValues<FVector2D>({1, 1}, {0, 1}, 15);
		TArray<double> Output = GenerateResults<FVector2D, FVector2D, double>(InputA, InputB, [](const FVector2D& A, const FVector2D& B) -> double {return FQuat::FindBetween(FVector(A, 0.0), FVector(B, 0.0)).Rotator().Yaw;});
		SECTION("In one Data")
		{
			ExecuteTest(TestParams(EPCGMetadataVectorOperation::AngleBetweenVector2D, GenerateParamData<FVector2D>({InputA,InputB})), FExpectedValues<double>{ DefaultValue, Output});		
		}
	}
	SECTION("Error types")
	{
		SECTION("FVector should not be supported")
		{
			TArray<FVector> InputA = GenerateValues<FVector>(FVector{0, 1, 0}, FVector{1, 0, 1}, 5);
			TArray<FVector> InputB = GenerateValues<FVector>(FVector{1, 1, 0}, FVector{1, 0, 1}, 5);
			
			FSuppressErrorsScope SuppressErrors(*this);
			RunTest(TestParams(EPCGMetadataVectorOperation::AngleBetweenVector2D, GenerateParamData<FVector>({InputA,InputB})));
			CHECK(NumErrors > 0);
		}
		SECTION("FVector4 should not be supported")
		{
			TArray<FVector4> InputA = GenerateValues<FVector4>(FVector4{0, 1, 0, 1}, FVector4{1, 1, 0, 1}, 5);
			TArray<FVector4> InputB = GenerateValues<FVector4>(FVector4{1, 1, 0, 1}, FVector4{0, 1, 0, 1}, 5);
		
			FSuppressErrorsScope SuppressErrors(*this);
			RunTest(TestParams(EPCGMetadataVectorOperation::AngleBetweenVector2D, GenerateParamData<FVector4>({InputA,InputB})));
			CHECK(NumErrors > 0);
		}
	}
}