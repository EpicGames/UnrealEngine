// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "TestHarness.h"

#include <catch2/generators/catch_generators.hpp>

#include "PCGContext.h"
#include "PCGModule.h"
#include "Data/DataView/PCGDataViewInterface.h"
#include "Data/PCGPointData.h"
#include "Elements/PCGPathfindingElement.h"
#include "PCGTestsCommon.h"
#include "Data/PCGPointArrayData.h"

class FPCGPathfindingBaseTest: public PCGTests::FPCGSingleElementBaseTest<UPCGPathfindingSettings>
{
public:
	FPCGPathfindingBaseTest(): 
		FPCGSingleElementBaseTest(),
		BaseStartPos(FRotator{}, FVector{ 0, 0, 0 }, FVector{}),
		BaseGoalPos(FRotator{}, FVector{ 10, 10, 0 }, FVector{})
	{
		TypedSettings->bOutputAsSpline = false;
		TypedSettings->bAcceptPartialPath = true;
		
		TArray<FVector> Points;

		Points.Add({ 0, 0, 0 });
		Points.Add({ 10, 10, 0 });

		//Short path
		Points.Add({ 2.5, 2.5, 0 });
		Points.Add({ 5, 5, 0 });
		MiddleNodeIndex = Points.Num()-1;
		Points.Add({ 7.5, 7.5, 0 });

		//Longer path
		Points.Add({ 1, 0, 0 });
		Points.Add({ 2, 0, 0 });
		Points.Add({ 3, 0, 0 });
		Points.Add({ 4, 0, 0 });
		Points.Add({ 5, 0, 0 });
		Points.Add({ 6, 0, 0 });
		Points.Add({ 7, 0, 0 });
		Points.Add({ 8, 0, 0 });
		Points.Add({ 9, 0, 0 });
		Points.Add({ 10, 0, 0 });
		Points.Add({ 10, 1, 0 });
		Points.Add({ 10, 2, 0 });
		Points.Add({ 10, 3, 0 });
		Points.Add({ 10, 4, 0 });
		Points.Add({ 10, 5, 0 });
		Points.Add({ 10, 6, 0 });
		Points.Add({ 10, 7, 0 });
		Points.Add({ 10, 8, 0 });
		Points.Add({ 10, 9, 0 });

		SimpleData = NewObject<UPCGPointArrayData>();
		SimpleData->SetNumPoints(Points.Num());
		SimpleData->SetDensity(1);
		for (int i = 0; i < Points.Num(); i++)
		{
			SimpleData->GetTransformValueRange()[i] = FTransform(FRotator{}, Points[i], FVector{});
		}
	}

	int GetPathfindingNumPoint()
	{
		ExecuteElement();
		CHECK(NumErrors == 0);
		CHECK(NumWarnings == 0);

		const TArray<FPCGTaggedData>& Inputs = Context->InputData.GetAllSpatialInputs();
		const TArray<FPCGTaggedData>& Outputs = Context->OutputData.GetAllSpatialInputs();

		REQUIRE_EQUAL(Outputs.Num(), 1);

		const FPCGTaggedData& Output = Outputs[0];

		const UPCGBasePointData* OutData = Cast<UPCGBasePointData>(Output.Data);
		REQUIRE(OutData != nullptr);

		return OutData->GetNumPoints();
	};


	int32 MiddleNodeIndex;

	FTransform BaseStartPos;
	FTransform BaseGoalPos;

	TObjectPtr<UPCGBasePointData> SimpleData;
};

TEST_CASE_METHOD(FPCGPathfindingBaseTest, "PCG::PathfindingElement", "[PCG][PathfindingElement]")
{
	auto [searchDistance, expectedPathLength] = GENERATE( 
			table<float, int>({
				{1000, 2},
				{7.1, 3},
				{3.6, 5},
				{1.1, 21}
			})
		);
	
	DYNAMIC_SECTION("BaseTest with search distance = " << searchDistance)
	{
		FPCGTaggedData& SimpleTaggedData = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		SimpleTaggedData.Data = SimpleData;
		SimpleTaggedData.Pin = PCGPinConstants::DefaultInputLabel;

		TypedSettings->Start = BaseStartPos.GetLocation();
		TypedSettings->Goal = BaseGoalPos.GetLocation();

		TypedSettings->SearchDistance = searchDistance;
		CHECK_EQUAL(GetPathfindingNumPoint(), expectedPathLength);
	}
}

TEST_CASE_METHOD(FPCGPathfindingBaseTest, "PCG::PathfindingElement:WithCosts", "[PCG][PathfindingElement]")
{
	
	auto [WithCostFunction, expectedPathLength] = GENERATE( 
			table<bool, int>({
				{false, 5},
				{true, 9}
			})
		);
	
	DYNAMIC_SECTION("With Cost function = " << (WithCostFunction ? "true" : "false"))
	{
		FPCGTaggedData& SimpleTaggedData = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		SimpleTaggedData.Data = SimpleData;
		SimpleTaggedData.Pin = PCGPinConstants::DefaultInputLabel;

		TypedSettings->Start = BaseStartPos.GetLocation();
		TypedSettings->Goal = BaseGoalPos.GetLocation();
		
		if (WithCostFunction)
		{
			const FString CostAttributeName = "Cost";
			// Add an absurd cost to the middle point of the fast path.
			FPCGMetadataAttribute<float>* Attribute = SimpleData->MutableMetadata()->CreateAttribute<float>(CostAttributeName, 1.0, /*bAllowsInterpolation=*/true, /*bOverrideParent=*/false);
			FPCGMetadataDomain* OutputMetadata = SimpleData->MutableMetadata()->GetMetadataDomain(FPCGMetadataDomainID{});
			check(OutputMetadata);
			PCGMetadataEntryKey Key = OutputMetadata->AddEntry();
			Attribute->SetValue(Key, 1000000.0);
			SimpleData->GetMetadataEntryValueRange()[MiddleNodeIndex] = Key;

			TypedSettings->CostFunctionMode = EPCGPathfindingCostFunctionMode::CostMultiplier;
			TypedSettings->CostAttribute =  FPCGAttributePropertySelector::CreateSelectorFromString<FPCGAttributePropertyInputSelector>(CostAttributeName);			
		}

		TypedSettings->SearchDistance = 3.6;
		CHECK_EQUAL(GetPathfindingNumPoint(), expectedPathLength);
	}
}

TEST_CASE_METHOD(FPCGPathfindingBaseTest, "PCG::PathfindingElement:WithStartPosAsInput", "[PCG][PathfindingElement]")
{
	FPCGTaggedData& SimpleTaggedData = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
	SimpleTaggedData.Data = SimpleData;
	SimpleTaggedData.Pin = PCGPinConstants::DefaultInputLabel;

	UPCGBasePointData* StartPointData = NewObject<UPCGPointArrayData>();
	StartPointData->SetNumPoints(1);
	StartPointData->SetDensity(1);
	StartPointData->GetTransformValueRange()[0] = BaseStartPos;
	
	FPCGTaggedData& StartPointTaggedData = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
	StartPointTaggedData.Data = StartPointData;
	StartPointTaggedData.Pin = "Start";
	TypedSettings->bStartLocationsAsInput = true;

	// Make sure the start pos is not used, so set it to an invalid value.
	TypedSettings->Start = FVector{ 1000, 1000, 1000 };
	TypedSettings->Goal = BaseGoalPos.GetLocation();	
	
	TypedSettings->SearchDistance = 3.6;
	CHECK_EQUAL(GetPathfindingNumPoint(), 5);
}

TEST_CASE_METHOD(FPCGPathfindingBaseTest, "PCG::PathfindingElement:WithStartPosAsAttribute", "[PCG][PathfindingElement]")
{
	auto WithCopyFromOriginatingPoints = GENERATE(false, true); 
	
	DYNAMIC_SECTION("With Copy From Orig = " << (WithCopyFromOriginatingPoints ? "true" : "false"))
	{
		FPCGTaggedData& SimpleTaggedData = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		SimpleTaggedData.Data = SimpleData;
		SimpleTaggedData.Pin = PCGPinConstants::DefaultInputLabel;

		const FString PosAttributeName = "ThePosition";

		UPCGBasePointData* StartPointData = NewObject<UPCGPointArrayData>();
		StartPointData->SetNumPoints(1);
		StartPointData->SetDensity(1);
		StartPointData->GetTransformValueRange()[0] = FTransform(FRotator{}, FVector{ 1000, 1000, 1000 }, FVector{});

		FPCGTaggedData& StartPointTaggedData = InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		StartPointTaggedData.Data = StartPointData;
		StartPointTaggedData.Pin = "Start";
		FPCGMetadataAttribute<FVector>* Attribute = StartPointData->MutableMetadata()->CreateAttribute<FVector>(PosAttributeName, FVector(), /*bAllowsInterpolation=*/true, /*bOverrideParent=*/false);
		FPCGMetadataDomain* OutputMetadata = StartPointData->MutableMetadata()->GetMetadataDomain(FPCGMetadataDomainID{});
		check(OutputMetadata);
		PCGMetadataEntryKey Key = OutputMetadata->AddEntry();
		Attribute->SetValue(Key, BaseStartPos.GetLocation());
		StartPointData->GetMetadataEntryValueRange()[0] = Key;

		TypedSettings->bStartLocationsAsInput = true;
		TypedSettings->StartLocationAttribute = FPCGAttributePropertySelector::CreateSelectorFromString<FPCGAttributePropertyInputSelector>(PosAttributeName);

		// Make sure the start pos is not used, so set it to an invalid value.
		TypedSettings->Start = FVector{ 1000, 1000, 1000 };
		TypedSettings->Goal = BaseGoalPos.GetLocation();
		
		TypedSettings->SearchDistance = 3.6;
		TypedSettings->bCopyOriginatingPoints = WithCopyFromOriginatingPoints;
		
		CHECK_EQUAL(GetPathfindingNumPoint(), 5);
		
		const TArray<FPCGTaggedData>& Outputs = Context->OutputData.GetAllSpatialInputs();
		const FPCGTaggedData& Output = Outputs[0];
		const UPCGBasePointData* OutData = Cast<UPCGBasePointData>(Output.Data);
		REQUIRE(OutData != nullptr);
		
		const FPCGMetadataAttribute<FVector>* OutputAttribute = OutData->ConstMetadata()->GetConstTypedAttribute<FVector>(PosAttributeName);
		if (WithCopyFromOriginatingPoints)
		{
			REQUIRE(OutputAttribute);
			FVector StartLocation = OutData->GetTransform(0).GetLocation();
			PCGMetadataEntryKey OutStartPointMetaDataEntry = OutData->GetMetadataEntry(0);
			CHECK(OutStartPointMetaDataEntry == Key);
		}
		else
		{
			CHECK(OutputAttribute == nullptr);
		}
	}
}