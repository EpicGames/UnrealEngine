// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "TestHarness.h"

#include "Elements/PCGSplineSampler.h"

#include <catch2/catch_test_macros.hpp>
#include <catch2/generators/catch_generators.hpp>


TEST_CASE("PCG::SplineSampler", "[PCG][SplineSampler]")
{	
	/* Clock-wise torch shape, to test edge cases
	 * |\/ \/|
	 * |_   _|
	 *   |_|
	 */

	TArray<FVector2D> PolygonPoints =
	{
		FVector2D(0, 0),
		FVector2D(0, 1),
		FVector2D(0, 2),
		FVector2D(1, 1),
		FVector2D(1.5, 2),
		FVector2D(1.5, 2), // Some duplicate points
		FVector2D(2, 1),
		FVector2D(3, 2),
		FVector2D(3, 1),
		FVector2D(3, 0),
		FVector2D(3, 0),
		FVector2D(2.5, 0),
		FVector2D(2, 0),
		FVector2D(2, -1),
		FVector2D(2, -1),
		FVector2D(1, -1),
		FVector2D(1, 0),
		FVector2D(0.5, 0),
	};

	UE::Geometry::FPolygon2d Polygon{MoveTemp(PolygonPoints)};
	if (Polygon.IsClockwise())
	{
		Polygon.Reverse();
	}

	auto [What, Point, bInsidePolygon] = GENERATE(Catch::Generators::table<FString, FVector2D, bool>(
		{
			// Interior point lies inside polygon
			{"Interior point", {0.5, 0.5}, true},
			// Exterior point lies outside polygon
			{"Exterior point", {10.5, 0.5}, false},

			// Y = -1
			// (1-e, -1) lies outside polygon
			{"(1-e, -1)", {1 - UE_KINDA_SMALL_NUMBER, -1}, false},
			// (2+e, -1) lies outside polygon
			{"(2+e, -1)", {2 + UE_KINDA_SMALL_NUMBER, -1}, false},

			// Y = 0
			// (0-e, 0) lies outside polygon
			{"(0-e, 0)", {0 - UE_KINDA_SMALL_NUMBER, 0}, false},
			// (1.5, 0) lies inside polygon
			{"(1.5, 0)", {1.5, 0}, true},
			// (3+e, 0) lies outside polygon
			{"(3+e, 0)", {3 + UE_KINDA_SMALL_NUMBER, 0}, false},

			// Y = 1
			// Test around (0, 1)
			// (0-e, 1) lies outside polygon
			{"(0-e, 1)", {0 - UE_KINDA_SMALL_NUMBER, 1}, false},
			// (0+e, 1) lies inside polygon
			{"(0+e, 1)", { 0 + UE_KINDA_SMALL_NUMBER, 1}, true},
			// Test around (1, 1)
			// (1-e, 1) lies inside polygon
			{"(1-e, 1)", {1 - UE_KINDA_SMALL_NUMBER, 1}, true},
			// (1+e, 1) lies inside polygon
			{"(1+e, 1)", { 1 + UE_KINDA_SMALL_NUMBER, 1}, true},
			// Test around (2, 1)
			// (2-e, 1) lies inside polygon
			{"(2-e, 1)", {2 - UE_KINDA_SMALL_NUMBER, 1}, true},
			// (2+e, 1) lies inside polygon
			{"(2+e, 1)", { 2 + UE_KINDA_SMALL_NUMBER, 1}, true},
			// Test around (3, 1)
			// (3-e, 1) lies inside polygon
			{"(3-e, 1)", {3 - UE_KINDA_SMALL_NUMBER, 1}, true},
			// (3+e, 1) lies outside polygon
			{"(3+e, 1)", { 3 + UE_KINDA_SMALL_NUMBER, 1}, false},

			// Y = 2
			// Test around (0, 2)
			// (0-e, 2) lies outside polygon
			{"(0-e, 2)", {0 - UE_KINDA_SMALL_NUMBER, 2}, false},
			// (0+e, 2) lies outside polygon
			{"(0+e, 2)", { 0 + UE_KINDA_SMALL_NUMBER, 2}, false},
			// Test around (1.5, 2)
			// (1.5, 2) lies outside polygon
			{"(1.5, 2)", {1.5, 2}, false},
			// Test around (3, 2)
			// (3-e, 2) lies outside polygon
			{"(3-e, 2)", {3 - UE_KINDA_SMALL_NUMBER, 2}, false},
			// (3+e, 2) lies outside polygon
			{"(3+e, 2)", { 3 + UE_KINDA_SMALL_NUMBER, 2}, false},
		}));

	DYNAMIC_SECTION(*FString::Printf(TEXT("%s lies %s polygon"), *What, bInsidePolygon ? TEXT("inside") : TEXT("outside")))
	{
		REQUIRE(PCGSplineSamplerHelpers::PointInsidePolygon2D(Polygon, Point) == bInsidePolygon);
	}
}
