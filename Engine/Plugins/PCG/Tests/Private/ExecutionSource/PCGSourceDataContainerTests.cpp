// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "TestHarness.h"

#include <catch2/catch_test_macros.hpp>

#include "PCGTestsCommon.h"
#include "Graph/PCGSourceDataContainer.h"

namespace PCGSourceDataStorageTests
{
	static const FName LabelA = TEXT("LabelA");
	static const FName LabelB = TEXT("LabelB");
	static constexpr uint64 HashA = 42;
	static constexpr uint64 HashB = 99;

	// Test payload values — must satisfy CPCGSourceDataCompatibleType (i.e. USTRUCT with StaticStruct()).
	// FPCGSourceDataStorageKey is a USTRUCT from the header under test.
	static const FPCGSourceDataStorageKey TestValueA(TEXT("PayloadA"), 100);
	static const FPCGSourceDataStorageKey TestValueB(TEXT("PayloadB"), 200);
}

// -------------------------------------------------------------------
// Key: construction, equality, hashing
// -------------------------------------------------------------------

TEST_CASE("PCG::SourceDataStorage::Key", "[PCG][SourceDataStorage]")
{
	using namespace PCGSourceDataStorageTests;

	SECTION("Default construction")
	{
		const FPCGSourceDataStorageKey Key;
		CHECK(Key.Label == NAME_None);
		CHECK(Key.Hash == 0);
	}

	SECTION("Parameterized construction")
	{
		const FPCGSourceDataStorageKey Key(LabelA, HashA);
		CHECK(Key.Label == LabelA);
		CHECK(Key.Hash == HashA);
	}

	SECTION("Label-only construction uses default hash")
	{
		const FPCGSourceDataStorageKey Key(LabelA, 0);
		CHECK(Key.Label == LabelA);
		CHECK(Key.Hash == 0);
	}

	SECTION("Equal keys")
	{
		const FPCGSourceDataStorageKey A(LabelA, HashA);
		const FPCGSourceDataStorageKey B(LabelA, HashA);
		CHECK(A == B);
		CHECK(GetTypeHash(A) == GetTypeHash(B));
	}

	SECTION("Different label")
	{
		const FPCGSourceDataStorageKey A(LabelA, HashA);
		const FPCGSourceDataStorageKey B(LabelB, HashA);
		CHECK_FALSE(A == B);
	}

	SECTION("Different hash")
	{
		const FPCGSourceDataStorageKey A(LabelA, HashA);
		const FPCGSourceDataStorageKey B(LabelA, HashB);
		CHECK_FALSE(A == B);
	}

	SECTION("TMap lookup")
	{
		TMap<FPCGSourceDataStorageKey, int32> Map;
		const FPCGSourceDataStorageKey Key(LabelA, HashA);
		Map.Add(Key, 7);

		const int32* Found = Map.Find(Key);
		REQUIRE(Found != nullptr);
		CHECK(*Found == 7);

		const FPCGSourceDataStorageKey Missing(LabelB, HashB);
		CHECK(Map.Find(Missing) == nullptr);
	}
}

// -------------------------------------------------------------------
// Value: type-safe access via FSharedStruct
// -------------------------------------------------------------------

TEST_CASE_METHOD(PCGTests::FPCGBaseTest, "PCG::SourceDataStorage::Value", "[PCG][SourceDataStorage]")
{
	using namespace PCGSourceDataStorageTests;

	FPCGSourceDataStorageValue Value;
	Value.Data = FSharedStruct::Make(TestValueA);

	SECTION("GetAs returns const shared struct")
	{
		const FConstSharedStruct Result = Value.GetAs<FPCGSourceDataStorageKey>();
		REQUIRE(Result.IsValid());
		const FPCGSourceDataStorageKey* ResultPtr = Result.GetPtr<FPCGSourceDataStorageKey>();
		REQUIRE(ResultPtr != nullptr);
		CHECK(*ResultPtr == TestValueA);
	}

	SECTION("GetAsMutable returns mutable shared struct")
	{
		FSharedStruct Result = Value.GetAsMutable<FPCGSourceDataStorageKey>();
		REQUIRE(Result.IsValid());
		FPCGSourceDataStorageKey* ResultPtr = Result.GetPtr<FPCGSourceDataStorageKey>();
		REQUIRE(ResultPtr != nullptr);
		*ResultPtr = TestValueB;

		// Verify mutation persisted
		const FConstSharedStruct Verify = Value.GetAs<FPCGSourceDataStorageKey>();
		REQUIRE(Verify.IsValid());
		const FPCGSourceDataStorageKey* VerifyPtr = Verify.GetPtr<FPCGSourceDataStorageKey>();
		REQUIRE(VerifyPtr != nullptr);
		CHECK(*VerifyPtr == TestValueB);
	}

	// @todo_pcg: These can only be verified in WITH_EDITOR build, which runtime tests are not editor build.
	// SECTION("GetAs with wrong stored type returns invalid")
	// {
	// 	// FPCGSourceDataStorageValue is a different USTRUCT than FPCGSourceDataStorageKey
	// 	const FConstSharedStruct Result = Value.GetAs<FPCGSourceDataStorageValue>();
	// 	CHECK_FALSE(Result.IsValid());
	// }

	// SECTION("GetAsMutable with wrong stored type returns invalid")
	// {
	// 	const FSharedStruct Result = Value.GetAsMutable<FPCGSourceDataStorageValue>();
	// 	CHECK_FALSE(Result.IsValid());
	// }

	SECTION("GetAs on empty value returns invalid")
	{
		FPCGSourceDataStorageValue EmptyValue;
		CHECK_FALSE(EmptyValue.GetAs<FPCGSourceDataStorageKey>().IsValid());
	}

	SECTION("GetAsMutable on empty value returns invalid")
	{
		FPCGSourceDataStorageValue EmptyValue;
		CHECK_FALSE(EmptyValue.GetAsMutable<FPCGSourceDataStorageKey>().IsValid());
	}
}

// -------------------------------------------------------------------
// Container: non-editor API (Get, GetCrc, Num, IsEmpty, operator=)
// -------------------------------------------------------------------

TEST_CASE("PCG::SourceDataStorage::Container", "[PCG][SourceDataStorage]")
{
	using namespace PCGSourceDataStorageTests;

	FPCGSourceDataContainer Container;

	SECTION("Default state")
	{
		CHECK(Container.IsEmpty());
		CHECK(Container.Num() == 0);
		CHECK(Container.GetDirtyGeneration() == 0);
	}

	SECTION("Get non-existent key returns invalid")
	{
		const FPCGSourceDataStorageKey Key(LabelA, HashA);
		CHECK_FALSE(Container.Get<FPCGSourceDataStorageKey>(Key).IsValid());
	}
}
