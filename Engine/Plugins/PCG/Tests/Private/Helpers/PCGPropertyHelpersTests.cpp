// Copyright Epic Games, Inc. All Rights Reserved.

// =============================================================================
// PCG Property Helpers Tests
//
// Covers both directions of attribute-set extraction:
//
//  1. ExtractAttributeSet* (param data -> containers):
//     PCGPropertyHelpers::ExtractAttributeSetToContainers and
//     PCGPropertyHelpers::ExtractAttributeSetAsArrayOfStructs(UScriptStruct).
//     Both APIs delegate to the same core routine, so we exercise that routine
//     via FInstancedPropertyBag (its underlying UPropertyBag acts as the target
//     struct), which removes the need for hardcoded USTRUCT types.
//
//  2. ExtractPropertyAsAttributeSet (containers -> param data):
//     The forward direction. Exercises the EPCGStructExtractorBehavior /
//     EPCGObjectExtractorBehavior / EPCGContainerExtractorBehavior enums and
//     the bDiscardLeafStructProperty leaf path. Uses USTRUCT/UCLASS fixtures
//     declared in PCGPropertyHelpersTestsTypes.h.
//
// Test summary - ExtractAttributeSet (param data -> containers):
//  - BasicExtraction:           Single float attribute extracted into one bag instance.
//  - MultiType:                 Float / int32 / FString / FVector attributes extracted together.
//  - MultipleEntries:           N entries produce N containers with correct per-entry values.
//  - EmptyParamData:            Attribute set with no entries yields an empty TArray result.
//  - MismatchedContainerCount:  Container count that doesn't match entry count returns false.
//  - EmptyContainers:           Zero containers on a non-empty param data returns false.
//  - NameMapping:               OptionalNameMapping rewrites the attribute name lookup.
//  - MissingRequired:           Non-defaultable missing attribute returns false and logs an error.
//  - MissingDefaultable:        Defaultable missing attribute is skipped silently.
//  - TypeCast:                  Float attribute -> Double property via AllowBroadcastAndConstructible.
//  - InstancedStructOverload:   ExtractAttributeSetAsArrayOfStructs(UScriptStruct*) returns TArray<FInstancedStruct>.
//  - NestedStructsNameClash:    Two members of the same nested struct type are disambiguated via full path "StructA/Value" vs "StructB/Value".
//  - ValueAndStructNameClash:   Top-level attribute and nested-struct member that share a name resolve via full path or a top-level-only selector.
//
// Test summary - ExtractPropertyAsAttributeSet (containers -> param data):
//  - Struct_Legacy_Root_FlattensSupportedMembers:    Default + NAME_None drops unsupported leaf structs and arrays.
//  - Struct_Legacy_Named_SupportedKeptAsSingle:      Default on FVector keeps the struct as a single attribute.
//  - Struct_Legacy_Named_UnsupportedFlattens:        Default on a custom struct flattens it into per-member attributes.
//  - Struct_Extract_Named_FlattensFVector:           Extract on FVector forces flattening to X / Y / Z.
//  - Struct_Extract_Root_KeepsLeafStructAttribute:   Extract + bDiscardLeafStructProperty=false keeps unsupported nested structs as Struct attributes.
//  - Struct_NoExtract_Named_KeptAsSingle:            NoExtract on a custom struct keeps it as a single attribute via the new generic accessors.
//  - Container_Legacy_Named_TopArrayFlattens:        Default ContainerExtractorBehavior unwraps a top-level TArray to one entry per element.
//  - Container_NoFlatten_Named_KeptAsArray:          NoFlatten keeps the TArray as a single attribute, one entry.
//  - Container_FlattenAll_Root_NestedArrayMember:    FlattenAll exposes array members of the outer struct (regression vs Legacy where they are dropped).
//  - Object_Extract_Root_FlattensClassMembers:       Extract on a UClass container flattens the class's UPROPERTYs.
// =============================================================================

#include "PCGTestsCommon.h"
#include "PCGPropertyHelpersTestsTypes.h"

#include "PCGParamData.h"
#include "Helpers/PCGPropertyHelpers.h"
#include "Metadata/PCGAttributePropertySelector.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/PCGMetadataAttributeTpl.h"

#include "StructUtils/InstancedStruct.h"
#include "StructUtils/PropertyBag.h"

#include <catch2/catch_test_macros.hpp>

#include "TestHarness.h"

namespace PCGPropertyHelpersTests
{
	static const FName FloatName    = TEXT("FloatProp");
	static const FName IntName      = TEXT("IntProp");
	static const FName StringName   = TEXT("StringProp");
	static const FName VectorName   = TEXT("VectorProp");
	static const FName DoubleName   = TEXT("DoubleProp");

	/** Builds a UPropertyBag with the supplied descs. */
	const UPropertyBag* GetOrCreateBag(TConstArrayView<FPropertyBagPropertyDesc> InDescs)
	{
		const UPropertyBag* Bag = UPropertyBag::GetOrCreateFromDescs(InDescs);
		REQUIRE(Bag != nullptr);
		return Bag;
	}

	/**
	 * Allocates NumInstances FInstancedPropertyBag instances of the given layout
	 * and returns their raw memory pointers so they can be passed to the extraction function.
	 */
	void InitInstances(const UPropertyBag* InBag, int32 NumInstances, TArray<FInstancedPropertyBag>& OutInstances, TArray<void*>& OutMemory)
	{
		OutInstances.Reset();
		OutMemory.Reset();
		OutInstances.Reserve(NumInstances);
		OutMemory.Reserve(NumInstances);
		for (int32 i = 0; i < NumInstances; ++i)
		{
			FInstancedPropertyBag& Instance = OutInstances.Emplace_GetRef();
			Instance.InitializeFromBagStruct(InBag);
			REQUIRE(Instance.IsValid());
			OutMemory.Add(Instance.GetMutableValue().GetMemory());
		}
	}

}

// Extracting a single float attribute into a single-instance property bag recovers the attribute value.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::PropertyHelpers::ExtractAttributeSet::BasicExtraction", "[PCG][PropertyHelpers]")
{
	using namespace PCGPropertyHelpersTests;

	const FPropertyBagPropertyDesc FloatDesc(FloatName, EPropertyBagPropertyType::Float);
	const UPropertyBag* Bag = GetOrCreateBag({FloatDesc});

	UPCGParamData* ParamData = NewObject<UPCGParamData>();
	REQUIRE(ParamData != nullptr);

	const float ExpectedValue = 1.25f;
	FPCGMetadataAttribute<float>* FloatAttr = ParamData->Metadata->CreateAttribute<float>(FloatName, 0.0f, true, false);
	REQUIRE(FloatAttr != nullptr);
	const PCGMetadataEntryKey Key = ParamData->Metadata->AddEntry();
	FloatAttr->SetValue(Key, ExpectedValue);

	TArray<FInstancedPropertyBag> Instances;
	TArray<void*> Memory;
	InitInstances(Bag, /*NumInstances=*/1, Instances, Memory);

	REQUIRE(PCGPropertyHelpers::ExtractAttributeSetToContainers(ParamData, Bag, Memory, /*OptionalNameMapping=*/nullptr, &Context));

	const TValueOrError<float, EPropertyBagResult> Read = Instances[0].GetValueFloat(FloatName);
	REQUIRE(Read.IsValid());
	REQUIRE_EQUAL(Read.GetValue(), ExpectedValue);
}

// Extracting several heterogeneous attributes (float / int32 / FString / FVector) in one call populates every matching field.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::PropertyHelpers::ExtractAttributeSet::MultiType", "[PCG][PropertyHelpers]")
{
	using namespace PCGPropertyHelpersTests;

	const FPropertyBagPropertyDesc FloatDesc(FloatName, EPropertyBagPropertyType::Float);
	const FPropertyBagPropertyDesc IntDesc(IntName, EPropertyBagPropertyType::Int32);
	const FPropertyBagPropertyDesc StringDesc(StringName, EPropertyBagPropertyType::String);
	const FPropertyBagPropertyDesc VectorDesc(VectorName, EPropertyBagPropertyType::Struct, TBaseStructure<FVector>::Get());
	const UPropertyBag* Bag = GetOrCreateBag({FloatDesc, IntDesc, StringDesc, VectorDesc});

	UPCGParamData* ParamData = NewObject<UPCGParamData>();
	REQUIRE(ParamData != nullptr);

	FPCGMetadataAttribute<float>* FloatAttr = ParamData->Metadata->CreateAttribute<float>(FloatName, 0.0f, true, false);
	FPCGMetadataAttribute<int32>* IntAttr = ParamData->Metadata->CreateAttribute<int32>(IntName, 0, true, false);
	FPCGMetadataAttribute<FString>* StringAttr = ParamData->Metadata->CreateAttribute<FString>(StringName, FString{}, true, false);
	FPCGMetadataAttribute<FVector>* VectorAttr = ParamData->Metadata->CreateAttribute<FVector>(VectorName, FVector::ZeroVector, true, false);
	REQUIRE(FloatAttr);
	REQUIRE(IntAttr);
	REQUIRE(StringAttr);
	REQUIRE(VectorAttr);

	const float ExpectedFloat = -3.5f;
	const int32 ExpectedInt = 42;
	const FString ExpectedString = TEXT("hello");
	const FVector ExpectedVector(1.0, 2.0, 3.0);

	const PCGMetadataEntryKey Key = ParamData->Metadata->AddEntry();
	FloatAttr->SetValue(Key, ExpectedFloat);
	IntAttr->SetValue(Key, ExpectedInt);
	StringAttr->SetValue(Key, ExpectedString);
	VectorAttr->SetValue(Key, ExpectedVector);

	TArray<FInstancedPropertyBag> Instances;
	TArray<void*> Memory;
	InitInstances(Bag, /*NumInstances=*/1, Instances, Memory);

	REQUIRE(PCGPropertyHelpers::ExtractAttributeSetToContainers(ParamData, Bag, Memory, /*OptionalNameMapping=*/nullptr, &Context));

	REQUIRE(Instances[0].GetValueFloat(FloatName).GetValue() == ExpectedFloat);
	REQUIRE(Instances[0].GetValueInt32(IntName).GetValue() == ExpectedInt);
	REQUIRE(Instances[0].GetValueString(StringName).GetValue() == ExpectedString);

	const TValueOrError<FStructView, EPropertyBagResult> VectorRead = Instances[0].GetValueStruct(VectorName, TBaseStructure<FVector>::Get());
	REQUIRE(VectorRead.IsValid());
	const FVector* ReadVector = VectorRead.GetValue().GetPtr<FVector>();
	REQUIRE(ReadVector != nullptr);
	REQUIRE(*ReadVector == ExpectedVector);
}

// N param data entries produce N populated containers, each with its own per-entry value.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::PropertyHelpers::ExtractAttributeSet::MultipleEntries", "[PCG][PropertyHelpers]")
{
	using namespace PCGPropertyHelpersTests;

	constexpr int32 NumEntries = 5;

	const FPropertyBagPropertyDesc FloatDesc(FloatName, EPropertyBagPropertyType::Float);
	const FPropertyBagPropertyDesc IntDesc(IntName, EPropertyBagPropertyType::Int32);
	const UPropertyBag* Bag = GetOrCreateBag({FloatDesc, IntDesc});

	UPCGParamData* ParamData = NewObject<UPCGParamData>();
	REQUIRE(ParamData != nullptr);

	FPCGMetadataAttribute<float>* FloatAttr = ParamData->Metadata->CreateAttribute<float>(FloatName, 0.0f, true, false);
	FPCGMetadataAttribute<int32>* IntAttr = ParamData->Metadata->CreateAttribute<int32>(IntName, 0, true, false);
	REQUIRE(FloatAttr);
	REQUIRE(IntAttr);

	TArray<float> ExpectedFloats;
	TArray<int32> ExpectedInts;
	ExpectedFloats.Reserve(NumEntries);
	ExpectedInts.Reserve(NumEntries);
	for (int32 i = 0; i < NumEntries; ++i)
	{
		const float FloatVal = 0.5f * static_cast<float>(i) - 1.0f;
		const int32 IntVal = i * 10;
		ExpectedFloats.Add(FloatVal);
		ExpectedInts.Add(IntVal);

		const PCGMetadataEntryKey Key = ParamData->Metadata->AddEntry();
		FloatAttr->SetValue(Key, FloatVal);
		IntAttr->SetValue(Key, IntVal);
	}

	TArray<FInstancedPropertyBag> Instances;
	TArray<void*> Memory;
	InitInstances(Bag, NumEntries, Instances, Memory);

	REQUIRE(PCGPropertyHelpers::ExtractAttributeSetToContainers(ParamData, Bag, Memory, /*OptionalNameMapping=*/nullptr, &Context));

	for (int32 i = 0; i < NumEntries; ++i)
	{
		REQUIRE(Instances[i].GetValueFloat(FloatName).GetValue() == ExpectedFloats[i]);
		REQUIRE(Instances[i].GetValueInt32(IntName).GetValue() == ExpectedInts[i]);
	}
}

// Empty param data returns an empty TArray from the FInstancedStruct overload without invoking the shared extraction.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::PropertyHelpers::ExtractAttributeSet::EmptyParamData", "[PCG][PropertyHelpers]")
{
	using namespace PCGPropertyHelpersTests;

	const FPropertyBagPropertyDesc FloatDesc(FloatName, EPropertyBagPropertyType::Float);
	const UPropertyBag* Bag = GetOrCreateBag({FloatDesc});

	UPCGParamData* ParamData = NewObject<UPCGParamData>();
	REQUIRE(ParamData != nullptr);
	// No entries added on purpose.
	FPCGMetadataAttribute<float>* FloatAttr = ParamData->Metadata->CreateAttribute<float>(FloatName, 0.0f, true, false);
	REQUIRE(FloatAttr != nullptr);

	const TArray<FInstancedStruct> Result = PCGPropertyHelpers::ExtractAttributeSetAsArrayOfStructs(ParamData, Bag, /*OptionalNameMapping=*/nullptr, &Context);
	REQUIRE(Result.IsEmpty());
}

// Supplying a different number of containers than param data entries fails the extraction.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::PropertyHelpers::ExtractAttributeSet::MismatchedContainerCount", "[PCG][PropertyHelpers]")
{
	using namespace PCGPropertyHelpersTests;

	const FPropertyBagPropertyDesc FloatDesc(FloatName, EPropertyBagPropertyType::Float);
	const UPropertyBag* Bag = GetOrCreateBag({FloatDesc});

	UPCGParamData* ParamData = NewObject<UPCGParamData>();
	REQUIRE(ParamData != nullptr);

	FPCGMetadataAttribute<float>* FloatAttr = ParamData->Metadata->CreateAttribute<float>(FloatName, 0.0f, true, false);
	REQUIRE(FloatAttr != nullptr);
	for (int32 i = 0; i < 3; ++i)
	{
		const PCGMetadataEntryKey Key = ParamData->Metadata->AddEntry();
		FloatAttr->SetValue(Key, static_cast<float>(i));
	}

	// 2 containers, 3 entries.
	TArray<FInstancedPropertyBag> Instances;
	TArray<void*> Memory;
	InitInstances(Bag, /*NumInstances=*/2, Instances, Memory);

	REQUIRE_FALSE(PCGPropertyHelpers::ExtractAttributeSetToContainers(ParamData, Bag, Memory, /*OptionalNameMapping=*/nullptr, &Context));
}

// A non-empty param data with an empty containers array fails the extraction.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::PropertyHelpers::ExtractAttributeSet::EmptyContainers", "[PCG][PropertyHelpers]")
{
	using namespace PCGPropertyHelpersTests;

	const FPropertyBagPropertyDesc FloatDesc(FloatName, EPropertyBagPropertyType::Float);
	const UPropertyBag* Bag = GetOrCreateBag({FloatDesc});

	UPCGParamData* ParamData = NewObject<UPCGParamData>();
	REQUIRE(ParamData != nullptr);
	FPCGMetadataAttribute<float>* FloatAttr = ParamData->Metadata->CreateAttribute<float>(FloatName, 0.0f, true, false);
	REQUIRE(FloatAttr != nullptr);
	const PCGMetadataEntryKey Key = ParamData->Metadata->AddEntry();
	FloatAttr->SetValue(Key, 42.0f);

	TArray<void*> EmptyMemory;
	REQUIRE_FALSE(PCGPropertyHelpers::ExtractAttributeSetToContainers(ParamData, Bag, EmptyMemory, /*OptionalNameMapping=*/nullptr, &Context));
}

// The OptionalNameMapping argument makes a property look up a differently-named attribute in the param data.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::PropertyHelpers::ExtractAttributeSet::NameMapping", "[PCG][PropertyHelpers]")
{
	using namespace PCGPropertyHelpersTests;

	const FPropertyBagPropertyDesc FloatDesc(FloatName, EPropertyBagPropertyType::Float);
	const UPropertyBag* Bag = GetOrCreateBag({FloatDesc});

	// Param data attribute has an intentionally different name.
	const FName AttributeNameInParam = TEXT("RealAttrName");

	UPCGParamData* ParamData = NewObject<UPCGParamData>();
	REQUIRE(ParamData != nullptr);
	FPCGMetadataAttribute<float>* FloatAttr = ParamData->Metadata->CreateAttribute<float>(AttributeNameInParam, 0.0f, true, false);
	REQUIRE(FloatAttr != nullptr);
	const PCGMetadataEntryKey Key = ParamData->Metadata->AddEntry();
	FloatAttr->SetValue(Key, 7.75f);

	TArray<FInstancedPropertyBag> Instances;
	TArray<void*> Memory;
	InitInstances(Bag, /*NumInstances=*/1, Instances, Memory);

	// Route the property named FloatName in the struct to the attribute named AttributeNameInParam in the param data.
	TMap<FName, TTuple<FName, bool>> NameMapping;
	NameMapping.Emplace(FloatName, TTuple<FName, bool>{AttributeNameInParam, /*bCanBeDefaulted=*/true});

	REQUIRE(PCGPropertyHelpers::ExtractAttributeSetToContainers(ParamData, Bag, Memory, &NameMapping, &Context));
	REQUIRE(Instances[0].GetValueFloat(FloatName).GetValue() == 7.75f);
}

// A required (non-defaultable) property that is missing from the param data causes the extraction to fail and log an error.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::PropertyHelpers::ExtractAttributeSet::MissingRequired", "[PCG][PropertyHelpers]")
{
	using namespace PCGPropertyHelpersTests;

	const FPropertyBagPropertyDesc FloatDesc(FloatName, EPropertyBagPropertyType::Float);
	const FPropertyBagPropertyDesc IntDesc(IntName, EPropertyBagPropertyType::Int32);
	const UPropertyBag* Bag = GetOrCreateBag({FloatDesc, IntDesc});

	UPCGParamData* ParamData = NewObject<UPCGParamData>();
	REQUIRE(ParamData != nullptr);
	// Only create the float attribute, leave IntProp absent.
	FPCGMetadataAttribute<float>* FloatAttr = ParamData->Metadata->CreateAttribute<float>(FloatName, 0.0f, true, false);
	REQUIRE(FloatAttr != nullptr);
	const PCGMetadataEntryKey Key = ParamData->Metadata->AddEntry();
	FloatAttr->SetValue(Key, 1.0f);

	TArray<FInstancedPropertyBag> Instances;
	TArray<void*> Memory;
	InitInstances(Bag, /*NumInstances=*/1, Instances, Memory);

	// Mark IntProp as required (bCanBeDefaulted = false).
	TMap<FName, TTuple<FName, bool>> NameMapping;
	NameMapping.Emplace(IntName, TTuple<FName, bool>{IntName, /*bCanBeDefaulted=*/false});

	PCGTests::FPCGTestsLogOutputDevice LogCapture(/*bInSuppressErrors=*/true);
	REQUIRE_FALSE(PCGPropertyHelpers::ExtractAttributeSetToContainers(ParamData, Bag, Memory, &NameMapping, &Context));
	REQUIRE(LogCapture.NbMessageReceived.Contains(ELogVerbosity::Error));
	REQUIRE(LogCapture.NbMessageReceived[ELogVerbosity::Error] >= 1);
}

// A defaultable property that is missing from the param data is silently skipped, and the other properties still extract.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::PropertyHelpers::ExtractAttributeSet::MissingDefaultable", "[PCG][PropertyHelpers]")
{
	using namespace PCGPropertyHelpersTests;

	const FPropertyBagPropertyDesc FloatDesc(FloatName, EPropertyBagPropertyType::Float);
	const FPropertyBagPropertyDesc IntDesc(IntName, EPropertyBagPropertyType::Int32);
	const UPropertyBag* Bag = GetOrCreateBag({FloatDesc, IntDesc});

	UPCGParamData* ParamData = NewObject<UPCGParamData>();
	REQUIRE(ParamData != nullptr);
	FPCGMetadataAttribute<float>* FloatAttr = ParamData->Metadata->CreateAttribute<float>(FloatName, 0.0f, true, false);
	REQUIRE(FloatAttr != nullptr);
	const PCGMetadataEntryKey Key = ParamData->Metadata->AddEntry();
	FloatAttr->SetValue(Key, 2.5f);

	TArray<FInstancedPropertyBag> Instances;
	TArray<void*> Memory;
	InitInstances(Bag, /*NumInstances=*/1, Instances, Memory);

	// No mapping: default behavior is bCanBeDefaulted = true.
	PCGTests::FPCGTestsLogOutputDevice LogCapture(/*bInSuppressErrors=*/true);
	REQUIRE(PCGPropertyHelpers::ExtractAttributeSetToContainers(ParamData, Bag, Memory, /*OptionalNameMapping=*/nullptr, &Context));
	REQUIRE_FALSE(LogCapture.NbMessageReceived.Contains(ELogVerbosity::Error));

	// FloatProp was written; IntProp kept its zero-initialized default.
	REQUIRE(Instances[0].GetValueFloat(FloatName).GetValue() == 2.5f);
	REQUIRE(Instances[0].GetValueInt32(IntName).GetValue() == 0);
}

// A float attribute successfully fills a double property via AllowBroadcastAndConstructible inside the extraction path.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::PropertyHelpers::ExtractAttributeSet::TypeCast", "[PCG][PropertyHelpers]")
{
	using namespace PCGPropertyHelpersTests;

	const FPropertyBagPropertyDesc DoubleDesc(DoubleName, EPropertyBagPropertyType::Double);
	const UPropertyBag* Bag = GetOrCreateBag({DoubleDesc});

	UPCGParamData* ParamData = NewObject<UPCGParamData>();
	REQUIRE(ParamData != nullptr);

	// Attribute shares the property's name but stores floats.
	FPCGMetadataAttribute<float>* FloatAttr = ParamData->Metadata->CreateAttribute<float>(DoubleName, 0.0f, true, false);
	REQUIRE(FloatAttr != nullptr);
	const PCGMetadataEntryKey Key = ParamData->Metadata->AddEntry();
	FloatAttr->SetValue(Key, 3.5f);

	TArray<FInstancedPropertyBag> Instances;
	TArray<void*> Memory;
	InitInstances(Bag, /*NumInstances=*/1, Instances, Memory);

	REQUIRE(PCGPropertyHelpers::ExtractAttributeSetToContainers(ParamData, Bag, Memory, /*OptionalNameMapping=*/nullptr, &Context));
	REQUIRE(Instances[0].GetValueDouble(DoubleName).GetValue() == 3.5);
}

// The UScriptStruct overload of ExtractAttributeSetAsArrayOfStructs returns a correctly-sized TArray<FInstancedStruct>
// whose entries carry the expected per-row values.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::PropertyHelpers::ExtractAttributeSet::InstancedStructOverload", "[PCG][PropertyHelpers]")
{
	using namespace PCGPropertyHelpersTests;

	constexpr int32 NumEntries = 3;

	const FPropertyBagPropertyDesc FloatDesc(FloatName, EPropertyBagPropertyType::Float);
	const UPropertyBag* Bag = GetOrCreateBag({FloatDesc});

	UPCGParamData* ParamData = NewObject<UPCGParamData>();
	REQUIRE(ParamData != nullptr);
	FPCGMetadataAttribute<float>* FloatAttr = ParamData->Metadata->CreateAttribute<float>(FloatName, 0.0f, true, false);
	REQUIRE(FloatAttr != nullptr);

	const float Values[NumEntries] = { 1.0f, 2.0f, 3.0f };
	for (int32 i = 0; i < NumEntries; ++i)
	{
		const PCGMetadataEntryKey Key = ParamData->Metadata->AddEntry();
		FloatAttr->SetValue(Key, Values[i]);
	}

	const TArray<FInstancedStruct> Result = PCGPropertyHelpers::ExtractAttributeSetAsArrayOfStructs(ParamData, Bag, /*OptionalNameMapping=*/nullptr, &Context);
	REQUIRE(Result.Num() == NumEntries);

	// Read each entry back via the UPropertyBag property to avoid hard-coding a struct type.
	const FPropertyBagPropertyDesc* AddedDesc = Bag->FindPropertyDescByName(FloatName);
	REQUIRE(AddedDesc != nullptr);
	REQUIRE(AddedDesc->CachedProperty != nullptr);

	for (int32 i = 0; i < NumEntries; ++i)
	{
		REQUIRE(Result[i].GetScriptStruct() == Bag);
		const void* InstanceMemory = Result[i].GetMemory();
		REQUIRE(InstanceMemory != nullptr);
		const float* ReadValue = AddedDesc->CachedProperty->ContainerPtrToValuePtr<float>(InstanceMemory);
		REQUIRE(ReadValue != nullptr);
		REQUIRE(*ReadValue == Values[i]);
	}
}

// When a struct contains two members of the same nested struct type, both have a "Value" field so the short name is ambiguous.
// The extraction must use the full property path (e.g. "StructA/Value", "StructB/Value") to route each attribute to the correct field.
// Also test that not using the full path result in error, and using mapping works.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::PropertyHelpers::ExtractAttributeSet::NestedStructsNameClash", "[PCG][PropertyHelpers]")
{
	using namespace PCGPropertyHelpersTests;

	static const FName ValueName = TEXT("Value");
	static const FName StructAName = TEXT("StructA");
	static const FName StructBName = TEXT("StructB");

	// Inner bag containing a single Value.
	const FPropertyBagPropertyDesc InnerValueDesc(ValueName, EPropertyBagPropertyType::Float);
	const UPropertyBag* InnerBag = GetOrCreateBag({InnerValueDesc});

	// Outer bag with two fields of the inner struct type — both expose a "Value" sub-field, creating a short-name clash.
	const FPropertyBagPropertyDesc StructADesc(StructAName, EPropertyBagPropertyType::Struct, InnerBag);
	const FPropertyBagPropertyDesc StructBDesc(StructBName, EPropertyBagPropertyType::Struct, InnerBag);
	const UPropertyBag* OuterBag = GetOrCreateBag({StructADesc, StructBDesc});

	SECTION("Using full path succeeds")
	{
		// Param data with attributes keyed by the full property path.
		const FName StructAValueAttrName = TEXT("StructA/Value");
		const FName StructBValueAttrName = TEXT("StructB/Value");

		UPCGParamData* ParamData = NewObject<UPCGParamData>();
		REQUIRE(ParamData != nullptr);
		FPCGMetadataAttribute<float>* StructAValueAttr = ParamData->Metadata->CreateAttribute<float>(StructAValueAttrName, 0.0f, true, false);
		FPCGMetadataAttribute<float>* StructBValueAttr = ParamData->Metadata->CreateAttribute<float>(StructBValueAttrName, 0.0f, true, false);
		REQUIRE(StructAValueAttr);
		REQUIRE(StructBValueAttr);

		constexpr float StructAExpected = 10.0f;
		constexpr float StructBExpected = 20.0f;

		const PCGMetadataEntryKey Key = ParamData->Metadata->AddEntry();
		StructAValueAttr->SetValue(Key, StructAExpected);
		StructBValueAttr->SetValue(Key, StructBExpected);

		TArray<FInstancedPropertyBag> Instances;
		TArray<void*> Memory;
		InitInstances(OuterBag, /*NumInstances=*/1, Instances, Memory);

		REQUIRE(PCGPropertyHelpers::ExtractAttributeSetToContainers(ParamData, OuterBag, Memory, /*OptionalNameMapping=*/nullptr, &Context));

		// Read both inner values back through the nested struct property chain.
		const FPropertyBagPropertyDesc* StructAOuterDesc = OuterBag->FindPropertyDescByName(StructAName);
		const FPropertyBagPropertyDesc* StructBOuterDesc = OuterBag->FindPropertyDescByName(StructBName);
		const FPropertyBagPropertyDesc* InnerValueAddedDesc = InnerBag->FindPropertyDescByName(ValueName);
		REQUIRE(StructAOuterDesc != nullptr);
		REQUIRE(StructBOuterDesc != nullptr);
		REQUIRE(InnerValueAddedDesc != nullptr);
		REQUIRE(StructAOuterDesc->CachedProperty != nullptr);
		REQUIRE(StructBOuterDesc->CachedProperty != nullptr);
		REQUIRE(InnerValueAddedDesc->CachedProperty != nullptr);

		const void* OuterMemory = Instances[0].GetValue().GetMemory();
		const void* StructAMemory = StructAOuterDesc->CachedProperty->ContainerPtrToValuePtr<void>(OuterMemory);
		const void* StructBMemory = StructBOuterDesc->CachedProperty->ContainerPtrToValuePtr<void>(OuterMemory);
		const float* StructAValue = InnerValueAddedDesc->CachedProperty->ContainerPtrToValuePtr<float>(StructAMemory);
		const float* StructBValue = InnerValueAddedDesc->CachedProperty->ContainerPtrToValuePtr<float>(StructBMemory);
		REQUIRE(StructAValue != nullptr);
		REQUIRE(StructBValue != nullptr);
		REQUIRE(*StructAValue == StructAExpected);
		REQUIRE(*StructBValue == StructBExpected);
	}
	
	SECTION("Using just the property name fails")
	{
		UPCGParamData* ParamData = NewObject<UPCGParamData>();
		REQUIRE(ParamData != nullptr);
		FPCGMetadataAttribute<float>* StructValueAttr = ParamData->Metadata->CreateAttribute<float>(ValueName, 0.0f, true, false);
		REQUIRE(StructValueAttr);

		constexpr float StructExpected = 10.0f;

		const PCGMetadataEntryKey Key = ParamData->Metadata->AddEntry();
		StructValueAttr->SetValue(Key, StructExpected);

		TArray<FInstancedPropertyBag> Instances;
		TArray<void*> Memory;
		InitInstances(OuterBag, /*NumInstances=*/1, Instances, Memory);

		PCGTests::FPCGTestsLogOutputDevice LogCapture(/*bInSuppressErrors=*/true);
		REQUIRE(PCGPropertyHelpers::ExtractAttributeSetToContainers(ParamData, OuterBag, Memory, /*OptionalNameMapping=*/nullptr, &Context));
		REQUIRE(LogCapture.NbMessageReceived.Contains(ELogVerbosity::Warning));
	}
	
	SECTION("Using mapping succeeds")
	{
		UPCGParamData* ParamData = NewObject<UPCGParamData>();
		REQUIRE(ParamData != nullptr);
		FPCGMetadataAttribute<float>* StructValueAttr = ParamData->Metadata->CreateAttribute<float>(ValueName, 0.0f, true, false);
		REQUIRE(StructValueAttr);

		constexpr float StructExpected = 10.0f;

		const PCGMetadataEntryKey Key = ParamData->Metadata->AddEntry();
		StructValueAttr->SetValue(Key, StructExpected);

		TArray<FInstancedPropertyBag> Instances;
		TArray<void*> Memory;
		InitInstances(OuterBag, /*NumInstances=*/1, Instances, Memory);
		
		const FName StructAValueAttrName = TEXT("StructA/Value");
		
		TMap<FName, TTuple<FName, bool>> NameMapping;
		NameMapping.Emplace(StructAValueAttrName, TTuple<FName, bool>{ValueName, /*bCanBeDefaulted=*/false});

		{
			// We should have no warnings.
			PCGTests::FPCGTestsLogOutputDevice LogCapture(/*bInSuppressErrors=*/true);
			REQUIRE(PCGPropertyHelpers::ExtractAttributeSetToContainers(ParamData, OuterBag, Memory, /*OptionalNameMapping=*/&NameMapping, &Context));
			REQUIRE_FALSE(LogCapture.NbMessageReceived.Contains(ELogVerbosity::Warning));
		}
		
		// Read both inner values back through the nested struct property chain.
		const FPropertyBagPropertyDesc* StructAOuterDesc = OuterBag->FindPropertyDescByName(StructAName);
		const FPropertyBagPropertyDesc* StructBOuterDesc = OuterBag->FindPropertyDescByName(StructBName);
		const FPropertyBagPropertyDesc* InnerValueAddedDesc = InnerBag->FindPropertyDescByName(ValueName);
		REQUIRE(StructAOuterDesc != nullptr);
		REQUIRE(StructBOuterDesc != nullptr);
		REQUIRE(InnerValueAddedDesc != nullptr);
		REQUIRE(StructAOuterDesc->CachedProperty != nullptr);
		REQUIRE(StructBOuterDesc->CachedProperty != nullptr);
		REQUIRE(InnerValueAddedDesc->CachedProperty != nullptr);

		const void* OuterMemory = Instances[0].GetValue().GetMemory();
		const void* StructAMemory = StructAOuterDesc->CachedProperty->ContainerPtrToValuePtr<void>(OuterMemory);
		const void* StructBMemory = StructBOuterDesc->CachedProperty->ContainerPtrToValuePtr<void>(OuterMemory);
		const float* StructAValue = InnerValueAddedDesc->CachedProperty->ContainerPtrToValuePtr<float>(StructAMemory);
		const float* StructBValue = InnerValueAddedDesc->CachedProperty->ContainerPtrToValuePtr<float>(StructBMemory);
		REQUIRE(StructAValue != nullptr);
		REQUIRE(StructBValue != nullptr);
		REQUIRE(*StructAValue == StructExpected);
		REQUIRE(*StructBValue == 0.0f); // B Value is untouched.
	}
}

// When a struct contains a property that have the same name than a property at the top level, we should make sure that it works
// and it doesn't log warnings
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::PropertyHelpers::ExtractAttributeSet::ValueAndStructNameClash", "[PCG][PropertyHelpers]")
{
	using namespace PCGPropertyHelpersTests;

	static const FName ValueName = TEXT("Value");
	static const FName StructName = TEXT("Struct");

	const FPropertyBagPropertyDesc InnerValueDesc(ValueName, EPropertyBagPropertyType::Float);
	const UPropertyBag* InnerBag = GetOrCreateBag({InnerValueDesc});

	const FPropertyBagPropertyDesc StructDesc(StructName, EPropertyBagPropertyType::Struct, InnerBag);
	const UPropertyBag* OuterBag = GetOrCreateBag({InnerValueDesc, StructDesc});

	SECTION("Using full path succeeds")
	{
		const FName StructValueAttrName = TEXT("Struct/Value");
		const FName ValueAttrName = TEXT("Value");

		UPCGParamData* ParamData = NewObject<UPCGParamData>();
		REQUIRE(ParamData != nullptr);
		FPCGMetadataAttribute<float>* StructValueAttr = ParamData->Metadata->CreateAttribute<float>(StructValueAttrName, 0.0f, true, false);
		FPCGMetadataAttribute<float>* ValueAttr = ParamData->Metadata->CreateAttribute<float>(ValueAttrName, 0.0f, true, false);
		REQUIRE(StructValueAttr);
		REQUIRE(ValueAttr);

		constexpr float StructExpected = 10.0f;
		constexpr float ValueExpected = 20.0f;

		const PCGMetadataEntryKey Key = ParamData->Metadata->AddEntry();
		StructValueAttr->SetValue(Key, StructExpected);
		ValueAttr->SetValue(Key, ValueExpected);

		TArray<FInstancedPropertyBag> Instances;
		TArray<void*> Memory;
		InitInstances(OuterBag, /*NumInstances=*/1, Instances, Memory);

		{
			PCGTests::FPCGTestsLogOutputDevice LogCapture(/*bInSuppressErrors=*/true);
			REQUIRE(PCGPropertyHelpers::ExtractAttributeSetToContainers(ParamData, OuterBag, Memory, /*OptionalNameMapping=*/nullptr, &Context));
			REQUIRE_FALSE(LogCapture.NbMessageReceived.Contains(ELogVerbosity::Warning));
		}

		// Read both inner values back through the nested struct property chain.
		const FPropertyBagPropertyDesc* StructOuterDesc = OuterBag->FindPropertyDescByName(StructName);
		const FPropertyBagPropertyDesc* ValueAddedDesc = OuterBag->FindPropertyDescByName(ValueName);
		const FPropertyBagPropertyDesc* InnerValueAddedDesc = InnerBag->FindPropertyDescByName(ValueName);
		REQUIRE(StructOuterDesc != nullptr);
		REQUIRE(ValueAddedDesc != nullptr);
		REQUIRE(InnerValueAddedDesc != nullptr);
		REQUIRE(StructOuterDesc->CachedProperty != nullptr);
		REQUIRE(ValueAddedDesc->CachedProperty != nullptr);
		REQUIRE(InnerValueAddedDesc->CachedProperty != nullptr);

		const void* OuterMemory = Instances[0].GetValue().GetMemory();
		const void* StructMemory = StructOuterDesc->CachedProperty->ContainerPtrToValuePtr<void>(OuterMemory);
		const float* Value = ValueAddedDesc->CachedProperty->ContainerPtrToValuePtr<float>(OuterMemory);
		const float* StructValue = InnerValueAddedDesc->CachedProperty->ContainerPtrToValuePtr<float>(StructMemory);
		REQUIRE(StructValue != nullptr);
		REQUIRE(Value != nullptr);
		REQUIRE(*StructValue == StructExpected);
		REQUIRE(*Value == ValueExpected);
	}
	
	SECTION("Using just the property name succeeds and change only top level value.")
	{
		UPCGParamData* ParamData = NewObject<UPCGParamData>();
		REQUIRE(ParamData != nullptr);
		FPCGMetadataAttribute<float>* ValueAttr = ParamData->Metadata->CreateAttribute<float>(ValueName, 0.0f, true, false);
		REQUIRE(ValueAttr);

		constexpr float ValueExpected = 10.0f;

		const PCGMetadataEntryKey Key = ParamData->Metadata->AddEntry();
		ValueAttr->SetValue(Key, ValueExpected);

		TArray<FInstancedPropertyBag> Instances;
		TArray<void*> Memory;
		InitInstances(OuterBag, /*NumInstances=*/1, Instances, Memory);

		{
			PCGTests::FPCGTestsLogOutputDevice LogCapture(/*bInSuppressErrors=*/true);
			REQUIRE(PCGPropertyHelpers::ExtractAttributeSetToContainers(ParamData, OuterBag, Memory, /*OptionalNameMapping=*/nullptr, &Context));
			REQUIRE_FALSE(LogCapture.NbMessageReceived.Contains(ELogVerbosity::Warning));
		}
		
		// Read both inner values back through the nested struct property chain.
		const FPropertyBagPropertyDesc* StructOuterDesc = OuterBag->FindPropertyDescByName(StructName);
		const FPropertyBagPropertyDesc* ValueAddedDesc = OuterBag->FindPropertyDescByName(ValueName);
		const FPropertyBagPropertyDesc* InnerValueAddedDesc = InnerBag->FindPropertyDescByName(ValueName);
		REQUIRE(StructOuterDesc != nullptr);
		REQUIRE(ValueAddedDesc != nullptr);
		REQUIRE(InnerValueAddedDesc != nullptr);
		REQUIRE(StructOuterDesc->CachedProperty != nullptr);
		REQUIRE(ValueAddedDesc->CachedProperty != nullptr);
		REQUIRE(InnerValueAddedDesc->CachedProperty != nullptr);

		const void* OuterMemory = Instances[0].GetValue().GetMemory();
		const void* StructMemory = StructOuterDesc->CachedProperty->ContainerPtrToValuePtr<void>(OuterMemory);
		const float* Value = ValueAddedDesc->CachedProperty->ContainerPtrToValuePtr<float>(OuterMemory);
		const float* StructValue = InnerValueAddedDesc->CachedProperty->ContainerPtrToValuePtr<float>(StructMemory);
		REQUIRE(StructValue != nullptr);
		REQUIRE(Value != nullptr);
		REQUIRE(*StructValue == 0.0f);
		REQUIRE(*Value == ValueExpected);
	}
}

// ---- ExtractPropertyAsAttributeSet (containers -> param data) -----------------

// Default StructExtractorBehavior (Legacy) on a NAME_None selector enumerates
// outer struct members. Supported leaves (float, FVector, FObjectProperty soft path)
// are emitted; unsupported leaf structs and arrays are silently dropped because
// Config.bDiscardLeafStructProperty is true and Config.bExtractArrays is false.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::PropertyHelpers::ExtractPropertyAsAttributeSet::Struct_Legacy_Root_FlattensSupportedMembers", "[PCG][PropertyHelpers]")
{
	FPCGPropertyHelpersTestsOuterStruct Container;
	Container.ScalarFloat = 1.5f;
	Container.VectorValue = FVector(1.0, 2.0, 3.0);
	Container.InnerStruct.InnerFloat = 10.0f;
	Container.IntArray = { 100, 200, 300 };

	PCGPropertyHelpers::FExtractorParameters Params(
		&Container,
		FPCGPropertyHelpersTestsOuterStruct::StaticStruct(),
		FPCGAttributePropertySelector{},
		/*OutputAttributeName=*/NAME_None,
		/*bPropertyNeedsToBeVisible=*/false);

	UPCGParamData* ParamData = PCGPropertyHelpers::ExtractPropertyAsAttributeSet(Params, &Context);
	REQUIRE(ParamData != nullptr);
	REQUIRE(ParamData->Metadata != nullptr);

	const UPCGMetadata* Metadata = ParamData->ConstMetadata();

	// Supported leaves are emitted.
	CHECK(Metadata->HasAttribute(TEXT("ScalarFloat")));
	CHECK(Metadata->HasAttribute(TEXT("VectorValue")));
	CHECK(Metadata->HasAttribute(TEXT("ObjectMember")));

	// Unsupported leaf structs are dropped.
	CHECK(!Metadata->HasAttribute(TEXT("InnerStruct")));
	CHECK(!Metadata->HasAttribute(TEXT("InnerWithArray")));

	// Top-level arrays are dropped (Config.bExtractArrays is false in Legacy).
	CHECK(!Metadata->HasAttribute(TEXT("IntArray")));

	// Spot-check ScalarFloat round-trip. The extraction path stores float UPROPERTYs
	// as Double attributes, so read as double via the base templated GetValueFromItemKey.
	const FPCGMetadataAttributeBase* ScalarFloatBase = Metadata->GetConstAttribute(TEXT("ScalarFloat"));
	REQUIRE(ScalarFloatBase != nullptr);
	CHECK(ScalarFloatBase->GetValueFromItemKey<double>(PCGFirstEntryKey) == 1.5);
}

// Legacy on a struct that the legacy accessors already support (FVector) keeps the
// struct as a single attribute named after OutputAttributeName. This is the
// pre-CL default behavior and must not regress.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::PropertyHelpers::ExtractPropertyAsAttributeSet::Struct_Legacy_Named_SupportedKeptAsSingle", "[PCG][PropertyHelpers]")
{
	FPCGPropertyHelpersTestsOuterStruct Container;
	Container.VectorValue = FVector(1.0, 2.0, 3.0);

	FPCGAttributePropertySelector Selector;
	Selector.SetAttributeName(TEXT("VectorValue"));

	static const FName OutAttrName = TEXT("Out");
	PCGPropertyHelpers::FExtractorParameters Params(
		&Container,
		FPCGPropertyHelpersTestsOuterStruct::StaticStruct(),
		Selector,
		OutAttrName,
		/*bPropertyNeedsToBeVisible=*/false);

	UPCGParamData* ParamData = PCGPropertyHelpers::ExtractPropertyAsAttributeSet(Params, &Context);
	REQUIRE(ParamData != nullptr);

	const FPCGMetadataAttributeBase* AttrBase = ParamData->ConstMetadata()->GetConstAttribute(OutAttrName);
	REQUIRE(AttrBase != nullptr);

	// Single FVector attribute, one entry.
	REQUIRE(ParamData->Metadata->GetItemCountForChild() == 1);
	CHECK(AttrBase->GetValueFromItemKey<FVector>(PCGFirstEntryKey) == FVector(1.0, 2.0, 3.0));
}

// Legacy on a custom struct that the legacy accessors do NOT support flattens it
// into per-member attributes. This is the legacy "force-extract unsupported"
// fallback that the new code preserves.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::PropertyHelpers::ExtractPropertyAsAttributeSet::Struct_Legacy_Named_UnsupportedFlattens", "[PCG][PropertyHelpers]")
{
	FPCGPropertyHelpersTestsOuterStruct Container;
	Container.InnerStruct.InnerFloat = 7.5f;
	Container.InnerStruct.InnerInt = 42;

	FPCGAttributePropertySelector Selector;
	Selector.SetAttributeName(TEXT("InnerStruct"));

	PCGPropertyHelpers::FExtractorParameters Params(
		&Container,
		FPCGPropertyHelpersTestsOuterStruct::StaticStruct(),
		Selector,
		/*OutputAttributeName=*/NAME_None,
		/*bPropertyNeedsToBeVisible=*/false);

	UPCGParamData* ParamData = PCGPropertyHelpers::ExtractPropertyAsAttributeSet(Params, &Context);
	REQUIRE(ParamData != nullptr);

	const UPCGMetadata* Metadata = ParamData->ConstMetadata();
	const FPCGMetadataAttributeBase* InnerFloatBase = Metadata->GetConstAttribute(TEXT("InnerFloat"));
	const FPCGMetadataAttributeBase* InnerIntBase = Metadata->GetConstAttribute(TEXT("InnerInt"));
	REQUIRE(InnerFloatBase != nullptr);
	REQUIRE(InnerIntBase != nullptr);

	// InnerFloat (source UPROPERTY is float) is stored as a Double attribute.
	CHECK(InnerFloatBase->GetValueFromItemKey<double>(PCGFirstEntryKey) == 7.5);
	CHECK(InnerIntBase->GetValueFromItemKey<int32>(PCGFirstEntryKey) == 42);
}

// Extract on a struct that legacy accessors normally treat as a single attribute
// (FVector) forces the new code to recurse into its members and emit X / Y / Z.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::PropertyHelpers::ExtractPropertyAsAttributeSet::Struct_Extract_Named_FlattensFVector", "[PCG][PropertyHelpers]")
{
	FPCGPropertyHelpersTestsOuterStruct Container;
	Container.VectorValue = FVector(1.0, 2.0, 3.0);

	FPCGAttributePropertySelector Selector;
	Selector.SetAttributeName(TEXT("VectorValue"));

	PCGPropertyHelpers::FExtractorParameters Params(
		&Container,
		FPCGPropertyHelpersTestsOuterStruct::StaticStruct(),
		Selector,
		/*OutputAttributeName=*/NAME_None,
		/*bPropertyNeedsToBeVisible=*/false);
	Params.StructExtractorBehavior = EPCGStructExtractorBehavior::Extract;

	UPCGParamData* ParamData = PCGPropertyHelpers::ExtractPropertyAsAttributeSet(Params, &Context);
	REQUIRE(ParamData != nullptr);

	const UPCGMetadata* Metadata = ParamData->ConstMetadata();
	REQUIRE(Metadata->HasAttribute(TEXT("X")));
	REQUIRE(Metadata->HasAttribute(TEXT("Y")));
	REQUIRE(Metadata->HasAttribute(TEXT("Z")));

	// The whole-vector attribute should NOT be present — we recursed past it.
	CHECK(!Metadata->HasAttribute(TEXT("VectorValue")));

	const FPCGMetadataAttributeBase* XBase = Metadata->GetConstAttribute(TEXT("X"));
	REQUIRE(XBase != nullptr);
	CHECK(XBase->GetValueFromItemKey<double>(PCGFirstEntryKey) == 1.0);
}

// Extract + NAME_None at root: bDiscardLeafStructProperty=false keeps unsupported
// nested structs as Struct-typed attributes via the new generic accessors. This is
// the new behavior that does NOT exist in Legacy.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::PropertyHelpers::ExtractPropertyAsAttributeSet::Struct_Extract_Root_KeepsLeafStructAttribute", "[PCG][PropertyHelpers]")
{
	FPCGPropertyHelpersTestsOuterStruct Container;
	Container.ScalarFloat = 1.5f;

	PCGPropertyHelpers::FExtractorParameters Params(
		&Container,
		FPCGPropertyHelpersTestsOuterStruct::StaticStruct(),
		FPCGAttributePropertySelector{},
		/*OutputAttributeName=*/NAME_None,
		/*bPropertyNeedsToBeVisible=*/false);
	Params.StructExtractorBehavior = EPCGStructExtractorBehavior::Extract;

	UPCGParamData* ParamData = PCGPropertyHelpers::ExtractPropertyAsAttributeSet(Params, &Context);
	REQUIRE(ParamData != nullptr);

	const UPCGMetadata* Metadata = ParamData->ConstMetadata();

	// Supported leaves still emitted.
	CHECK(Metadata->HasAttribute(TEXT("ScalarFloat")));
	CHECK(Metadata->HasAttribute(TEXT("VectorValue")));

	// Unsupported leaf structs survive as Struct attributes (vs Legacy where they would be dropped).
	CHECK(Metadata->HasAttribute(TEXT("InnerStruct")));
	CHECK(Metadata->HasAttribute(TEXT("InnerWithArray")));
}

// NoExtract on a struct property keeps it as a single attribute, even if the
// legacy accessors don't support it — the new generic accessors handle it.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::PropertyHelpers::ExtractPropertyAsAttributeSet::Struct_NoExtract_Named_KeptAsSingle", "[PCG][PropertyHelpers]")
{
	FPCGPropertyHelpersTestsOuterStruct Container;
	Container.InnerStruct.InnerFloat = 7.5f;
	Container.InnerStruct.InnerInt = 13;

	FPCGAttributePropertySelector Selector;
	Selector.SetAttributeName(TEXT("InnerStruct"));

	static const FName OutAttrName = TEXT("Out");
	PCGPropertyHelpers::FExtractorParameters Params(
		&Container,
		FPCGPropertyHelpersTestsOuterStruct::StaticStruct(),
		Selector,
		OutAttrName,
		/*bPropertyNeedsToBeVisible=*/false);
	Params.StructExtractorBehavior = EPCGStructExtractorBehavior::NoExtract;

	UPCGParamData* ParamData = PCGPropertyHelpers::ExtractPropertyAsAttributeSet(Params, &Context);
	REQUIRE(ParamData != nullptr);

	const UPCGMetadata* Metadata = ParamData->ConstMetadata();

	// One single attribute carrying the struct as-is, no flattening.
	CHECK(Metadata->HasAttribute(OutAttrName));
	CHECK(!Metadata->HasAttribute(TEXT("InnerFloat")));
	CHECK(!Metadata->HasAttribute(TEXT("InnerInt")));
}

// Legacy ContainerExtractorBehavior unwraps a top-level TArray selector to one
// entry per array element. Pre-CL behavior — must not regress.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::PropertyHelpers::ExtractPropertyAsAttributeSet::Container_Legacy_Named_TopArrayFlattens", "[PCG][PropertyHelpers]")
{
	FPCGPropertyHelpersTestsOuterStruct Container;
	Container.IntArray = { 100, 200, 300 };

	FPCGAttributePropertySelector Selector;
	Selector.SetAttributeName(TEXT("IntArray"));

	static const FName OutAttrName = TEXT("Out");
	PCGPropertyHelpers::FExtractorParameters Params(
		&Container,
		FPCGPropertyHelpersTestsOuterStruct::StaticStruct(),
		Selector,
		OutAttrName,
		/*bPropertyNeedsToBeVisible=*/false);

	UPCGParamData* ParamData = PCGPropertyHelpers::ExtractPropertyAsAttributeSet(Params, &Context);
	REQUIRE(ParamData != nullptr);

	const FPCGMetadataAttributeBase* AttrBase = ParamData->ConstMetadata()->GetConstAttribute(OutAttrName);
	REQUIRE(AttrBase != nullptr);

	// One entry per array element.
	REQUIRE(ParamData->Metadata->GetItemCountForChild() == 3);

	CHECK(AttrBase->GetValueFromItemKey<int32>(PCGFirstEntryKey + 0) == 100);
	CHECK(AttrBase->GetValueFromItemKey<int32>(PCGFirstEntryKey + 1) == 200);
	CHECK(AttrBase->GetValueFromItemKey<int32>(PCGFirstEntryKey + 2) == 300);
}

// NoFlatten keeps the TArray as a single TArray-typed attribute with one entry,
// rather than fanning the elements out into separate entries.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::PropertyHelpers::ExtractPropertyAsAttributeSet::Container_NoFlatten_Named_KeptAsArray", "[PCG][PropertyHelpers]")
{
	FPCGPropertyHelpersTestsOuterStruct Container;
	Container.IntArray = { 100, 200, 300 };

	FPCGAttributePropertySelector Selector;
	Selector.SetAttributeName(TEXT("IntArray"));

	static const FName OutAttrName = TEXT("Out");
	PCGPropertyHelpers::FExtractorParameters Params(
		&Container,
		FPCGPropertyHelpersTestsOuterStruct::StaticStruct(),
		Selector,
		OutAttrName,
		/*bPropertyNeedsToBeVisible=*/false);
	Params.ContainerExtractorBehavior = EPCGContainerExtractorBehavior::NoFlattenLast;

	UPCGParamData* ParamData = PCGPropertyHelpers::ExtractPropertyAsAttributeSet(Params, &Context);
	REQUIRE(ParamData != nullptr);

	const FPCGMetadataAttributeBase* AttrBase = ParamData->ConstMetadata()->GetConstAttribute(OutAttrName);
	REQUIRE(AttrBase != nullptr);

	// Single entry — the whole array.
	CHECK(ParamData->Metadata->GetItemCountForChild() == 1);
}

// FlattenLast exposes array members of the outer struct (Config.bExtractArrays=true)
// during root extraction, where Legacy would silently drop them, but it is not flatten
// because it is an extraction.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::PropertyHelpers::ExtractPropertyAsAttributeSet::Container_FlattenAll_Root_NestedArrayMember", "[PCG][PropertyHelpers]")
{
	FPCGPropertyHelpersTestsOuterStruct Container;
	Container.IntArray = { 1, 2, 3 };

	PCGPropertyHelpers::FExtractorParameters Params(
		&Container,
		FPCGPropertyHelpersTestsOuterStruct::StaticStruct(),
		FPCGAttributePropertySelector{},
		/*OutputAttributeName=*/NAME_None,
		/*bPropertyNeedsToBeVisible=*/false);
	Params.StructExtractorBehavior = EPCGStructExtractorBehavior::Extract;
	Params.ContainerExtractorBehavior = EPCGContainerExtractorBehavior::FlattenLast;

	UPCGParamData* ParamData = PCGPropertyHelpers::ExtractPropertyAsAttributeSet(Params, &Context);
	REQUIRE(ParamData != nullptr);
	
	REQUIRE(ParamData->Metadata->GetItemCountForChild() == 1);

	// IntArray member appears (it is dropped under Legacy).
	CHECK(ParamData->ConstMetadata()->HasAttribute(TEXT("IntArray")));
}

// Extract on a UClass container at NAME_None flattens the class's UPROPERTYs
// directly. ExtractRoot drives bShouldExtract; the UClass acts as Parameters.Class.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::PropertyHelpers::ExtractPropertyAsAttributeSet::Object_Extract_Root_FlattensClassMembers", "[PCG][PropertyHelpers]")
{
	UPCGPropertyHelpersTestsObject* TestObject = NewObject<UPCGPropertyHelpersTestsObject>();
	REQUIRE(TestObject != nullptr);
	TestObject->ObjectFloat = 9.5f;
	TestObject->ObjectInt = 7;
	TestObject->ObjectString = TEXT("hello");

	PCGPropertyHelpers::FExtractorParameters Params(
		TestObject,
		UPCGPropertyHelpersTestsObject::StaticClass(),
		FPCGAttributePropertySelector{},
		/*OutputAttributeName=*/NAME_None,
		/*bPropertyNeedsToBeVisible=*/false);
	Params.ObjectExtractorBehavior = EPCGObjectExtractorBehavior::Extract;

	UPCGParamData* ParamData = PCGPropertyHelpers::ExtractPropertyAsAttributeSet(Params, &Context);
	REQUIRE(ParamData != nullptr);

	const UPCGMetadata* Metadata = ParamData->ConstMetadata();
	REQUIRE(Metadata->HasAttribute(TEXT("ObjectFloat")));
	REQUIRE(Metadata->HasAttribute(TEXT("ObjectInt")));
	REQUIRE(Metadata->HasAttribute(TEXT("ObjectString")));

	// ObjectFloat (source UPROPERTY is float) is stored as a Double attribute.
	const FPCGMetadataAttributeBase* ObjectFloatBase = Metadata->GetConstAttribute(TEXT("ObjectFloat"));
	REQUIRE(ObjectFloatBase != nullptr);
	CHECK(ObjectFloatBase->GetValueFromItemKey<double>(PCGFirstEntryKey) == 9.5);
}