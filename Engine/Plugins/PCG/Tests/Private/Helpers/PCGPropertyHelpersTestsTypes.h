// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "PCGPropertyHelpersTestsTypes.generated.h"

/** Custom struct that is NOT supported by the legacy property accessors — used to exercise the bDiscardLeafStructProperty / new generic accessor path. */
USTRUCT()
struct FPCGPropertyHelpersTestsInnerStruct
{
	GENERATED_BODY()

	UPROPERTY()
	float InnerFloat = 0.f;

	UPROPERTY()
	int32 InnerInt = 0;
};

/** Custom struct that contains an array — used for the FlattenAll nested-array tests. */
USTRUCT()
struct FPCGPropertyHelpersTestsInnerWithArray
{
	GENERATED_BODY()

	UPROPERTY()
	int32 InnerScalar = 0;

	UPROPERTY()
	TArray<float> InnerArray;
};

/**
 * Top-level struct used by ExtractPropertyAsAttributeSet tests. Mixes:
 *  - A scalar leaf supported by old accessors (ScalarFloat).
 *  - A struct supported by old accessors (FVector).
 *  - An unsupported nested struct (InnerStruct) — covers bDiscardLeafStructProperty.
 *  - A nested struct that owns an array (InnerWithArray) — covers nested array under a struct.
 *  - A top-level array (IntArray) — covers ContainerExtractorBehavior on a named selector.
 *  - A UObject pointer (ObjectMember) — covers FObjectProperty as a soft-path leaf.
 */
USTRUCT()
struct FPCGPropertyHelpersTestsOuterStruct
{
	GENERATED_BODY()

	UPROPERTY()
	float ScalarFloat = 0.f;

	UPROPERTY()
	FVector VectorValue = FVector::ZeroVector;

	UPROPERTY()
	FPCGPropertyHelpersTestsInnerStruct InnerStruct;

	UPROPERTY()
	FPCGPropertyHelpersTestsInnerWithArray InnerWithArray;

	UPROPERTY()
	TArray<int32> IntArray;

	UPROPERTY()
	TObjectPtr<UObject> ObjectMember;
};

/** UClass fixture used to exercise EPCGObjectExtractorBehavior::Extract when the container itself is a UObject. */
UCLASS()
class UPCGPropertyHelpersTestsObject : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	float ObjectFloat = 0.f;

	UPROPERTY()
	int32 ObjectInt = 0;

	UPROPERTY()
	FString ObjectString;
};
