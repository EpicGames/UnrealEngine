// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"

#include "TestCommon/Initialization.h"

#include "catch2/catch_tag_alias_autoregistrar.hpp"
#include <catch2/catch_test_macros.hpp>

#include "SlateGlobals.h"
#include "HAL/IConsoleManager.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "Styling/SlateWidgetStyleContainerBase.h"
#include "UObject/CoreRedirects.h"
#include "UObject/UObjectGlobals.h"

//CATCH_REGISTER_TAG_ALIAS("[@PCGTests]", "[PCG]~[!benchmark]")

// Adds redirects to a non-existent file for all the Engine .uasset files that
// will be automatically loaded by the various engine systems. This is done to
// prevent attempted deserialization of assets that can only be loaded when
// the project is built with WITH_EDITORONLY_DATA. Unfortunately turning this
// flag on also requires WITH_EDITOR which is currently extremely difficult to
// build outside of the editor.
static void PreventLoadingOfEditorOnlyData()
{
	const TCHAR* const IncompatiblePackages[] = {
		TEXT("/Engine/EngineResources/DefaultTexture"),
		TEXT("/Engine/EngineResources/DefaultTextureCube"),
		TEXT("/Engine/EngineResources/DefaultVolumeTexture"),
		TEXT("/Engine/EngineFonts/RobotoDistanceField"),
		TEXT("/Engine/EngineMaterials/DefaultTextMaterialOpaque"),
		TEXT("/Engine/EngineDamageTypes/DmgTypeBP_Environmental"),
		TEXT("/Engine/EngineSky/VolumetricClouds/m_SimpleVolumetricCloud_Inst"),
		TEXT("/Engine/EngineMeshes/Sphere"),
		TEXT("/Engine/EngineResources/WhiteSquareTexture"),
		TEXT("/Engine/EngineResources/GradientTexture0"),
		TEXT("/Engine/EngineResources/Black"),
		TEXT("/Engine/EngineDebugMaterials/VolumeToRender"),
		TEXT("/Engine/EngineDebugMaterials/M_VolumeRenderSphereTracePP"),
		TEXT("/Engine/EngineFonts/Roboto"),
		TEXT("/Engine/EngineMaterials/Widget3DPassThrough_Translucent"),
		TEXT("/Engine/EngineMaterials/Widget3DPassThrough_Translucent_OneSided"),
		TEXT("/Engine/EngineMaterials/Widget3DPassThrough_Opaque"),
		TEXT("/Engine/EngineMaterials/Widget3DPassThrough_Opaque_OneSided"),
		TEXT("/Engine/EngineMaterials/Widget3DPassThrough_Masked"),
		TEXT("/Engine/EngineMaterials/Widget3DPassThrough_Masked_OneSided"),
		TEXT("/Engine/BasicShapes/Cube"),
		TEXT("/CameraCalibrationCore/Materials/M_DefaultEvenSquare"),
		TEXT("/CameraCalibrationCore/Materials/M_DefaultOddSquare"),
	};
	FCoreRedirectObjectName InvalidName(NAME_None, NAME_None, TEXT("/Engine/DoesNotExist"));
	TArray<FCoreRedirect> NewRedirects;
	for (const TCHAR* PackageName : IncompatiblePackages)
	{
		NewRedirects.Emplace(ECoreRedirectFlags::Type_Package,
			FCoreRedirectObjectName(NAME_None, NAME_None, PackageName),
			InvalidName);
	}
	FCoreRedirects::Initialize();
	FCoreRedirects::AddRedirectList(NewRedirects, TEXT("PCGTests.PreventLoadingOfEditorOnlyData"));
	FCoreRedirects::AddKnownMissing(ECoreRedirectFlags::Type_Package, InvalidName);

	// With Zen Loader, we need this CVar to support redirectors.
	if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("s.AllowPackageRedirectorSupport"), false))
	{
		CVar->Set(1);
	}
}

GROUP_BEFORE_GLOBAL(Catch::DefaultGroup)
{
	// This will set the verbosity of the log temporarily while we init.
	// It will skip the warnings for Slate and Streaming, while skipping error for UObjectGlobals
	// as there are errors while contructing some CDOs about missing assets, which is irrelevant in our case.
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogSlate, ELogVerbosity::Error);
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogSlateStyle, ELogVerbosity::Error);
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogStreaming, ELogVerbosity::Error);
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogUObjectGlobals, ELogVerbosity::Fatal);

	PreventLoadingOfEditorOnlyData();
	InitAll(true, true);
}

GROUP_AFTER_GLOBAL(Catch::DefaultGroup)
{
	CleanupAll();
	GIsRunning = false;
}
