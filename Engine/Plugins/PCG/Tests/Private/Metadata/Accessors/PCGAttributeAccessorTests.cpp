// Copyright Epic Games, Inc. All Rights Reserved.

// =============================================================================
// PCG Attribute Accessor Tests
//
// Tests for IPCGAttributeAccessor Get/Set operations on PCG metadata attributes.
// Uses float, FVector, and FString attributes with entry-based and point-based keys.
//
// Test summary:
//  - SingleGetDefault:                  Single Get on default (invalid) entry key returns attribute default values
//  - SingleGet:                         Single Get on valid entry keys returns correct per-entry values
//  - SingleGetOutsideRange:             Single Get with out-of-range indices wraps around
//  - GetRangeDefault:                   GetRange on default entry key fills arrays with default values
//  - GetRange:                          GetRange on valid entry keys fills arrays with correct values
//  - GetRangeOutsideRange:              GetRange with out-of-range indices wraps around
//  - SingleSetDefault:                  Single Set on default entry key updates the attribute default value
//  - SingleSet:                         Single Set on valid entry keys writes values to specific entries
//  - SingleSetInvalidKey:               Single Set on invalid entry key creates a new entry instead of modifying default
//  - SetRange:                          SetRange writes an array of values to consecutive entries
//  - SetRangeOutsideRange:              SetRange with out-of-range indices or too many values fails
//  - SingleGetPoints:                   Single Get using point-based keys resolves metadata entry indirection
//  - GetRangePoints:                    GetRange using point-based keys respects reverse entry order
//  - SingleSetPoints:                   Single Set using point-based keys writes through entry indirection
//  - SetRangePoints:                    SetRange using point-based keys writes through entry indirection
//  - GetWithConstructibleType:          Get/Set with constructible type conversion (float<->double) using AllowConstructible flag
//  - GetWithBroadcastableType:          Get/Set with broadcastable type conversion (float->FVector/FString) using AllowBroadcast flag
//  - GetWithBroadcastAndConstructibleType: Get/Set with double->FVector4 conversion using AllowBroadcastAndConstructible flag
//  - SetRangeOfValuesIntoSingleContainer: SetRange of scalar values into array/set container with AllowRangeOfValuesIntoSingleContainer flag
//  - SoftObjectPathToStringWithStrictType: FSoftObjectPath/FSoftClassPath to FString conversion with StrictType flag
//  - CopyToSameType:                     CopyTo copies values between same-type accessors (float, vector, string)
//  - CopyToPartialRange:                 CopyTo copies a sub-range leaving untouched entries at their defaults
//  - CopyToConstructibleType:            CopyTo with AllowConstructible copies between constructible types (float->double)
//  - CopyToConstructibleType::Array:     CopyTo with AllowConstructible copies between constructible array types (TArray<float>->TArray<double>)
//  - CopyToBroadcastableType:            CopyTo with AllowBroadcast copies between broadcastable types (float->FVector)
//  - CopyToBroadcastableType::Array:     CopyTo with AllowBroadcast copies between broadcastable array types (TArray<float>->TArray<FVector>)
//  - CopyToSameType::Array:              CopyTo copies between same-type array attributes (TArray<float>->TArray<float>) over multiple entries
//  - CopyToSingleValueIntoArray:         CopyTo packs N source single values into one dest array entry under AllowRangeOfValuesIntoSingleContainer (SameType, Constructible, Broadcast)
//  - CompressedArrayRoundTrip:           SetRange/GetRange round-trip on a compressed TArray<FString> attribute preserves duplicate array payloads
//  - TypeErasedGetRangeSingleValue:      Public type-erased GetRange via FOutValuesByValue — SameType, Constructible, Broadcast, StrictType-fail, struct (FPCGPoint SameType), and complex conversion (FString -> FSoftObjectPath via Constructible)
//  - TypeErasedGetRangeArray:            Public type-erased GetRange via FOutValuesAsArray — SameType, Constructible, Broadcast (regression guard for ConvertToSingleValue + TransformFunc arg-order), Buffers/OutValues count mismatch, struct (TArray<FPCGPoint> SameType), and complex conversion (TArray<FString> -> TArray<FSoftObjectPath> via Constructible)
//  - TypeErasedGetRangeSet:              Public type-erased GetRange via FOutValuesAsSet — non-SameType conversion returns false (unsupported branch)
//  - TypeErasedGetRangeMap:              Public type-erased GetRange via FOutValuesAsMap — non-SameType conversion returns false (unsupported branch)
//  - TypeErasedSetRangeSingleValue:      Public type-erased SetRange via FInValuesByValue — SameType, Constructible, Broadcast, struct (FPCGPoint SameType), and complex conversion (FString -> FSoftObjectPath via Constructible)
//  - TypeErasedSetRangeArray:            Public type-erased SetRange via FInValuesAsArray — SameType, Constructible, Broadcast, struct (TArray<FPCGPoint> SameType), and complex conversion (TArray<FString> -> TArray<FSoftObjectPath> via Constructible)
//  - TypeErasedSetRangeSet:              Public type-erased SetRange via FInValuesAsSet — SameType only (only supported mode)
//  - TypeErasedSetRangeMap:              Public type-erased SetRange via FInValuesAsMap — SameType only (only supported mode)
//  - TypeErasedSetRangeIntoSingleContainer: Public type-erased SetRange's AllowRangeOfValuesIntoSingleContainer path — scalars→Array (SameType + Constructible), scalars→Set with varied element sizes (regression guard for inner-element-size arithmetic), and the explicit no-fall-through guard for non-SameType set conversion
// =============================================================================

#include "PCGTestsCommon.h"

#include "Data/PCGPointData.h"
#include "Metadata/Accessors/IPCGAttributeAccessorTpl.h"
#include "Metadata/Accessors/PCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/PCGMetadataAttributeTestsCommonHelper.h"

#include <catch2/catch_test_macros.hpp>

#include "TestHarness.h"

namespace PCGAttributeAccessorTests
{
	constexpr int32 ValuesSize = 3;

	const float FloatValues[ValuesSize] = { 0.57441f, -0.289f, 0.1587f};
	const FVector VectorValues[ValuesSize] = { FVector(1.024f, 0.445f, 0.587f), FVector(4.6470f, 3.19874f, 9.6648f), FVector(4.690f, 7.874f, 8.668f) };
	const FString StringValues[ValuesSize] = { FString(TEXT("Str1")), FString(TEXT("Foo")), FString(TEXT("MyStr")) };

	template <typename T, size_t N>
	TArrayView<T> MakeArrayView(T(& Array)[N])
	{
		return TArrayView<T>(Array, N);
	}

	template <typename T, size_t N>
	TArrayView<const T> MakeConstArrayView(T(&Array)[N])
	{
		return TArrayView<const T>(Array, N);
	}

	struct AttributeData
	{
		explicit AttributeData(UPCGBasePointData* InPointData)
			: PointData(InPointData)
		{
			FloatAttribute = PointData->Metadata->CreateAttribute<float>(TEXT("FloatAttribute"), 0.0f, true, false);
			VectorAttribute = PointData->Metadata->CreateAttribute<FVector>(TEXT("VectorAttribute"), FVector::ZeroVector, false, false);
			StringAttribute = PointData->Metadata->CreateAttribute<FString>(TEXT("StringAttribute"), FString(), false, false);

			// All points will point to a given entry, in reverse order, except the last one, that will point to DefaultValue
			PointData->SetNumPoints(ValuesSize + 1);
			TPCGValueRange<int64> MetadataEntryRange = PointData->GetMetadataEntryValueRange();

			for (int32 i = 0; i < ValuesSize; ++i)
			{
				PCGMetadataEntryKey EntryKey = PointData->Metadata->AddEntry();
				FloatAttribute->SetValue(EntryKey, FloatValues[i]);
				VectorAttribute->SetValue(EntryKey, VectorValues[i]);
				StringAttribute->SetValue(EntryKey, StringValues[i]);
				MetadataEntryRange[ValuesSize - i - 1] = EntryKey;
			}

			FloatAccessor = PCGAttributeAccessorHelpers::CreateAccessor(FloatAttribute, PointData->Metadata);
			VectorAccessor = PCGAttributeAccessorHelpers::CreateAccessor(VectorAttribute, PointData->Metadata);
			StringAccessor = PCGAttributeAccessorHelpers::CreateAccessor(StringAttribute, PointData->Metadata);
		}

		void ResetTempFloats()
		{
			TempFloats[0] = 1.0f;
			TempFloats[1] = 2.0f;
			TempFloats[2] = 3.0f;
		}

		TObjectPtr<UPCGBasePointData> PointData;
		FPCGMetadataAttribute<float>* FloatAttribute;
		FPCGMetadataAttribute<FVector>* VectorAttribute;
		FPCGMetadataAttribute<FString>* StringAttribute;

		TUniquePtr<IPCGAttributeAccessor> FloatAccessor;
		TUniquePtr<IPCGAttributeAccessor> VectorAccessor;
		TUniquePtr<IPCGAttributeAccessor> StringAccessor;

		float TempFloats[ValuesSize] = { 1.0f, 2.0f, 3.0f };
		FVector TempVectors[ValuesSize] = { FVector(0.1f, 0.1f, 0.1f), FVector(0.1f, 0.2f, 0.1f), FVector(0.1f, 0.2f, 0.3f) };
		FString TempStrings[ValuesSize] = { TEXT("Bla"), TEXT("BlaBla"), TEXT("Blou") };
		int32 TempInvalidInts[ValuesSize] = { 5, 6, 7 };
	};
}

// Tests that getting a single value using the default (invalid) entry key returns the attribute's default value for each type (float, vector, string), and fails when requesting an int from a float accessor.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::SingleGetDefault", "[PCG][Accessor][Attribute]")
{
	using namespace PCGAttributeAccessorTests;

	AttributeData Data(CreatePointData());
	FPCGAttributeAccessorKeysEntries DefaultEntry(PCGInvalidEntryKey);

	REQUIRE_EQUAL(DefaultEntry.GetNum(), 1);

	SECTION("Get default float attribute")
	{
		REQUIRE(Data.FloatAccessor->Get(*Data.TempFloats, DefaultEntry));
		REQUIRE_EQUAL(*Data.TempFloats, 0.0f);
	}

	SECTION("Get default vector attribute")
	{
		REQUIRE(Data.VectorAccessor->Get(*Data.TempVectors, DefaultEntry));
		REQUIRE_EQUAL(*Data.TempVectors, FVector::ZeroVector);
	}

	SECTION("Get default string attribute")
	{
		REQUIRE(Data.StringAccessor->Get(*Data.TempStrings, DefaultEntry));
		REQUIRE_EQUAL(*Data.TempStrings, FString{});
	}

	SECTION("Get default float with int fails")
	{
		REQUIRE_FALSE(Data.FloatAccessor->Get(*Data.TempInvalidInts, DefaultEntry));
		REQUIRE_EQUAL(*Data.TempInvalidInts, 5);
	}
}

// Tests that getting a single value by entry index returns the correct stored value for float, vector, and string attributes, and fails when requesting an int from a float accessor.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::SingleGet", "[PCG][Accessor][Attribute]")
{
	using namespace PCGAttributeAccessorTests;

	AttributeData Data(CreatePointData());
	FPCGAttributeAccessorKeysEntries Keys(Data.FloatAttribute->GetMetadata());

	REQUIRE_EQUAL(Keys.GetNum(), ValuesSize);

	for (int32 i = 0; i < ValuesSize; ++i)
	{
		REQUIRE(Data.FloatAccessor->Get(Data.TempFloats[i], i, Keys));
		REQUIRE_EQUAL(Data.TempFloats[i], FloatValues[i]);

		REQUIRE(Data.VectorAccessor->Get(Data.TempVectors[i], i, Keys));
		REQUIRE_EQUAL(Data.TempVectors[i], VectorValues[i]);

		REQUIRE(Data.StringAccessor->Get(Data.TempStrings[i], i, Keys));
		REQUIRE_EQUAL(Data.TempStrings[i], StringValues[i]);

		REQUIRE_FALSE(Data.FloatAccessor->Get(Data.TempInvalidInts[i], i, Keys));
		REQUIRE_EQUAL(Data.TempInvalidInts[i], 5 + i);
	}
}

// Tests that getting a single value with an index beyond the entry count wraps around using modulo.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::SingleGetOutsideRange", "[PCG][Accessor][Attribute]")
{
	using namespace PCGAttributeAccessorTests;

	AttributeData Data(CreatePointData());
	FPCGAttributeAccessorKeysEntries Keys(Data.FloatAttribute->GetMetadata());

	float DefaultFloat = 1.0f;

	// 3 -> 7 wrapping to 0 -> 3 and back to 0
	for (int32 i = 0; i < ValuesSize + 1; ++i)
	{
		REQUIRE(Data.FloatAccessor->Get(DefaultFloat, i + ValuesSize, Keys));
		REQUIRE_EQUAL(DefaultFloat, FloatValues[i % ValuesSize]);
	}
}

// Tests that GetRange using the default (invalid) entry key fills the output array with default values for each type, fails for int from float accessor, and works correctly with a sub-range starting at index 1.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::GetRangeDefault", "[PCG][Accessor][Attribute]")
{
	using namespace PCGAttributeAccessorTests;

	AttributeData Data(CreatePointData());
	FPCGAttributeAccessorKeysEntries DefaultEntry(PCGInvalidEntryKey);

	SECTION("Full range")
	{
		int32 Index = 0;

		SECTION("Float")
		{
			REQUIRE(Data.FloatAccessor->GetRange(MakeArrayView(Data.TempFloats), Index, DefaultEntry));
			REQUIRE_EQUAL(Data.TempFloats[0], 0.0f);
			REQUIRE_EQUAL(Data.TempFloats[1], 0.0f);
			REQUIRE_EQUAL(Data.TempFloats[2], 0.0f);
		}

		SECTION("Vector")
		{
			REQUIRE(Data.VectorAccessor->GetRange(MakeArrayView(Data.TempVectors), Index, DefaultEntry));
			REQUIRE_EQUAL(Data.TempVectors[0], FVector::ZeroVector);
			REQUIRE_EQUAL(Data.TempVectors[1], FVector::ZeroVector);
			REQUIRE_EQUAL(Data.TempVectors[2], FVector::ZeroVector);
		}

		SECTION("String")
		{
			REQUIRE(Data.StringAccessor->GetRange(MakeArrayView(Data.TempStrings), Index, DefaultEntry));
			REQUIRE_EQUAL(Data.TempStrings[0], FString{});
			REQUIRE_EQUAL(Data.TempStrings[1], FString{});
			REQUIRE_EQUAL(Data.TempStrings[2], FString{});
		}

		SECTION("Invalid int type")
		{
			REQUIRE_FALSE(Data.FloatAccessor->GetRange(MakeArrayView(Data.TempInvalidInts), Index, DefaultEntry));
			REQUIRE_EQUAL(Data.TempInvalidInts[0], 5);
			REQUIRE_EQUAL(Data.TempInvalidInts[1], 6);
			REQUIRE_EQUAL(Data.TempInvalidInts[2], 7);
		}
	}

	SECTION("Smaller range from index 1")
	{
		int32 Index = 1;

		REQUIRE(Data.FloatAccessor->GetRange(TArrayView<float>(Data.TempFloats, 2), Index, DefaultEntry));
		REQUIRE_EQUAL(Data.TempFloats[0], 0.0f);
		REQUIRE_EQUAL(Data.TempFloats[1], 0.0f);
		REQUIRE_EQUAL(Data.TempFloats[2], 3.0f);
	}
}

// Tests that GetRange on valid entry keys fills the output array with correct stored values for float, vector, and string, fails for int from float accessor, and works with a sub-range starting at index 1.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::GetRange", "[PCG][Accessor][Attribute]")
{
	using namespace PCGAttributeAccessorTests;

	AttributeData Data(CreatePointData());
	FPCGAttributeAccessorKeysEntries Keys(Data.FloatAttribute->GetMetadata());

	SECTION("Full range")
	{
		int32 Index = 0;

		SECTION("Float")
		{
			REQUIRE(Data.FloatAccessor->GetRange(MakeArrayView(Data.TempFloats), Index, Keys));
			REQUIRE_EQUAL(Data.TempFloats[0], FloatValues[0]);
			REQUIRE_EQUAL(Data.TempFloats[1], FloatValues[1]);
			REQUIRE_EQUAL(Data.TempFloats[2], FloatValues[2]);
		}

		SECTION("Vector")
		{
			REQUIRE(Data.VectorAccessor->GetRange(MakeArrayView(Data.TempVectors), Index, Keys));
			REQUIRE_EQUAL(Data.TempVectors[0], VectorValues[0]);
			REQUIRE_EQUAL(Data.TempVectors[1], VectorValues[1]);
			REQUIRE_EQUAL(Data.TempVectors[2], VectorValues[2]);
		}

		SECTION("String")
		{
			REQUIRE(Data.StringAccessor->GetRange(MakeArrayView(Data.TempStrings), Index, Keys));
			REQUIRE_EQUAL(Data.TempStrings[0], StringValues[0]);
			REQUIRE_EQUAL(Data.TempStrings[1], StringValues[1]);
			REQUIRE_EQUAL(Data.TempStrings[2], StringValues[2]);
		}

		SECTION("Invalid int type")
		{
			REQUIRE_FALSE(Data.FloatAccessor->GetRange(MakeArrayView(Data.TempInvalidInts), Index, Keys));
			REQUIRE_EQUAL(Data.TempInvalidInts[0], 5);
			REQUIRE_EQUAL(Data.TempInvalidInts[1], 6);
			REQUIRE_EQUAL(Data.TempInvalidInts[2], 7);
		}
	}

	SECTION("Smaller range from index 1")
	{
		int32 Index = 1;

		REQUIRE(Data.FloatAccessor->GetRange(TArrayView<float>(Data.TempFloats, 2), Index, Keys));
		REQUIRE_EQUAL(Data.TempFloats[0], FloatValues[1]);
		REQUIRE_EQUAL(Data.TempFloats[1], FloatValues[2]);
		REQUIRE_EQUAL(Data.TempFloats[2], 3.0f);
	}
}

// Tests that GetRange with out-of-range indices wraps around correctly, including when requesting more values than entries exist.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::GetRangeOutsideRange", "[PCG][Accessor][Attribute]")
{
	using namespace PCGAttributeAccessorTests;

	AttributeData Data(CreatePointData());
	FPCGAttributeAccessorKeysEntries Keys(Data.FloatAttribute->GetMetadata());

	SECTION("Wrapping 3 -> 6 to 0 -> 3")
	{
		int32 OutsideRangeIndex = ValuesSize;

		REQUIRE(Data.FloatAccessor->GetRange(MakeArrayView(Data.TempFloats), OutsideRangeIndex, Keys));
		REQUIRE_EQUAL(Data.TempFloats[0], FloatValues[0]);
		REQUIRE_EQUAL(Data.TempFloats[1], FloatValues[1]);
		REQUIRE_EQUAL(Data.TempFloats[2], FloatValues[2]);
	}

	SECTION("Wrapping 4, 5, 7 to 1, 2, 0")
	{
		int32 OutsideRangeIndex = ValuesSize + 1;

		REQUIRE(Data.FloatAccessor->GetRange(MakeArrayView(Data.TempFloats), OutsideRangeIndex, Keys));
		REQUIRE_EQUAL(Data.TempFloats[0], FloatValues[1]);
		REQUIRE_EQUAL(Data.TempFloats[1], FloatValues[2]);
		REQUIRE_EQUAL(Data.TempFloats[2], FloatValues[0]);
	}

	SECTION("Gathering more values wraps")
	{
		constexpr int32 MoreFloatValuesSize = ValuesSize * 3;
		float MoreFloatValues[MoreFloatValuesSize];
		for (int32 i = 0; i < MoreFloatValuesSize; ++i)
		{
			MoreFloatValues[i] = float(i);
		}

		REQUIRE(Data.FloatAccessor->GetRange(MakeArrayView(MoreFloatValues), 0, Keys));

		for (int32 i = 0; i < MoreFloatValuesSize; ++i)
		{
			REQUIRE_EQUAL(MoreFloatValues[i], FloatValues[i % ValuesSize]);
		}
	}
}

// Tests that setting a single value on the default entry key with AllowSetDefaultValue flag updates the attribute's default value for float, vector, and string.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::SingleSetDefault", "[PCG][Accessor][Attribute]")
{
	using namespace PCGAttributeAccessorTests;

	AttributeData Data(CreatePointData());
	FPCGAttributeAccessorKeysEntries DefaultEntry(PCGInvalidEntryKey);

	SECTION("Float")
	{
		float NewDefaultFloat = 0.5f;
		REQUIRE(Data.FloatAccessor->Set(NewDefaultFloat, DefaultEntry, EPCGAttributeAccessorFlags::AllowSetDefaultValue));
		REQUIRE_EQUAL(Data.FloatAttribute->GetValue(PCGDefaultValueKey), NewDefaultFloat);
	}

	SECTION("Vector")
	{
		FVector NewDefaultVector = FVector(0.5f, 0.2f, 0.1f);
		REQUIRE(Data.VectorAccessor->Set(NewDefaultVector, DefaultEntry, EPCGAttributeAccessorFlags::AllowSetDefaultValue));
		REQUIRE_EQUAL(Data.VectorAttribute->GetValue(PCGDefaultValueKey), NewDefaultVector);
	}

	SECTION("String")
	{
		FString NewDefaultString(TEXT("AAA"));
		REQUIRE(Data.StringAccessor->Set(NewDefaultString, DefaultEntry, EPCGAttributeAccessorFlags::AllowSetDefaultValue));
		REQUIRE_EQUAL(Data.StringAttribute->GetValue(PCGDefaultValueKey), NewDefaultString);
	}
}

// Tests that setting a single value at a specific entry index writes correctly for float, vector, and string, and fails when the index is outside the valid range.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::SingleSet", "[PCG][Accessor][Attribute]")
{
	using namespace PCGAttributeAccessorTests;

	AttributeData Data(CreatePointData());
	FPCGAttributeAccessorKeysEntries Keys(const_cast<FPCGMetadataDomain*>(Data.FloatAttribute->GetMetadataDomain()));

	SECTION("Set float attribute")
	{
		REQUIRE(Data.FloatAccessor->Set(*Data.TempFloats, Keys));
		REQUIRE_EQUAL(Data.FloatAttribute->GetValueFromItemKey(0), *Data.TempFloats);
	}

	SECTION("Set vector attribute at index 1")
	{
		REQUIRE(Data.VectorAccessor->Set(*Data.TempVectors, /*Index=*/ 1, Keys));
		REQUIRE_EQUAL(Data.VectorAttribute->GetValueFromItemKey(1), *Data.TempVectors);
	}

	SECTION("Set string attribute at index 2")
	{
		REQUIRE(Data.StringAccessor->Set(*Data.TempStrings, /*Index=*/ 2, Keys));
		REQUIRE_EQUAL(Data.StringAttribute->GetValueFromItemKey(2), *Data.TempStrings);
	}

	SECTION("Set float attribute outside range is invalid")
	{
		float InvalidFloat = 2.147f;
		REQUIRE_FALSE(Data.FloatAccessor->Set(InvalidFloat, /*Index=*/ 3, Keys));
	}
}

// Tests that setting a value on an invalid entry key (without AllowSetDefaultValue) creates a new metadata entry rather than modifying the default value.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::SingleSetInvalidKey", "[PCG][Accessor][Attribute]")
{
	using namespace PCGAttributeAccessorTests;

	AttributeData Data(CreatePointData());
	FPCGAttributeAccessorKeysEntries DefaultEntry(PCGInvalidEntryKey);

	float NewDefaultFloat = 0.5f;

	REQUIRE(Data.FloatAccessor->Set(NewDefaultFloat, DefaultEntry));
	REQUIRE_EQUAL(Data.FloatAttribute->GetValue(PCGDefaultValueKey), 0.0f);
	REQUIRE_EQUAL(Data.PointData->Metadata->GetItemCountForChild(), ValuesSize + 1);
	REQUIRE_EQUAL(Data.FloatAttribute->GetValueFromItemKey(ValuesSize), NewDefaultFloat);
}

// Tests that SetRange writes an array of values to consecutive entries for float, vector, and string attributes.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::SetRange", "[PCG][Accessor][Attribute]")
{
	using namespace PCGAttributeAccessorTests;

	AttributeData Data(CreatePointData());
	FPCGAttributeAccessorKeysEntries Keys(const_cast<FPCGMetadataDomain*>(Data.FloatAttribute->GetMetadataDomain()));

	const int32 Index = 0;

	SECTION("Float")
	{
		REQUIRE(Data.FloatAccessor->SetRange(MakeConstArrayView(Data.TempFloats), Index, Keys));
		REQUIRE_EQUAL(Data.FloatAttribute->GetValueFromItemKey(0), Data.TempFloats[0]);
		REQUIRE_EQUAL(Data.FloatAttribute->GetValueFromItemKey(1), Data.TempFloats[1]);
		REQUIRE_EQUAL(Data.FloatAttribute->GetValueFromItemKey(2), Data.TempFloats[2]);
	}

	SECTION("Vector")
	{
		REQUIRE(Data.VectorAccessor->SetRange(MakeConstArrayView(Data.TempVectors), Index, Keys));
		REQUIRE_EQUAL(Data.VectorAttribute->GetValueFromItemKey(0), Data.TempVectors[0]);
		REQUIRE_EQUAL(Data.VectorAttribute->GetValueFromItemKey(1), Data.TempVectors[1]);
		REQUIRE_EQUAL(Data.VectorAttribute->GetValueFromItemKey(2), Data.TempVectors[2]);
	}

	SECTION("String")
	{
		REQUIRE(Data.StringAccessor->SetRange(MakeConstArrayView(Data.TempStrings), Index, Keys));
		REQUIRE_EQUAL(Data.StringAttribute->GetValueFromItemKey(0), Data.TempStrings[0]);
		REQUIRE_EQUAL(Data.StringAttribute->GetValueFromItemKey(1), Data.TempStrings[1]);
		REQUIRE_EQUAL(Data.StringAttribute->GetValueFromItemKey(2), Data.TempStrings[2]);
	}
}

// Tests that SetRange fails when providing too many values, when the index offset is outside the valid range, or when the index plus array size exceeds the range.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::SetRangeOutsideRange", "[PCG][Accessor][Attribute]")
{
	using namespace PCGAttributeAccessorTests;

	AttributeData Data(CreatePointData());
	FPCGAttributeAccessorKeysEntries Keys(Data.FloatAttribute->GetMetadata());

	const int32 Index = 0;

	SECTION("Too many values")
	{
		float TooManyFloats[ValuesSize + 1] = { 1.0f, 2.0f, 3.0f, 4.0f };
		REQUIRE_FALSE(Data.FloatAccessor->SetRange(MakeConstArrayView(TooManyFloats), Index, Keys));
	}

	SECTION("Index offset outside range")
	{
		REQUIRE_FALSE(Data.FloatAccessor->SetRange(MakeConstArrayView(Data.TempFloats), Index + 999, Keys));
	}

	SECTION("Index offset valid but too many values")
	{
		REQUIRE_FALSE(Data.FloatAccessor->SetRange(MakeConstArrayView(Data.TempFloats), Index + 1, Keys));
	}
}

// Tests that getting a single value using point-based keys correctly resolves the metadata entry indirection and returns the value associated with each point.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::SingleGetPoints", "[PCG][Accessor][Attribute]")
{
	using namespace PCGAttributeAccessorTests;

	AttributeData Data(CreatePointData());

	const TConstPCGValueRange<int64> MetadataEntryRange = Data.PointData->GetConstMetadataEntryValueRange();
	TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(Data.PointData, FPCGAttributePropertySelector());

	REQUIRE_EQUAL(Keys->GetNum(), Data.PointData->GetNumPoints());
	REQUIRE(Keys->IsReadOnly());

	for (int32 i = 0; i < 4; ++i)
	{
		REQUIRE(Data.FloatAccessor->Get(*Data.TempFloats, i, *Keys));
		REQUIRE_EQUAL(*Data.TempFloats, Data.FloatAttribute->GetValueFromItemKey(MetadataEntryRange[i]));
	}
}

// Tests that GetRange using point-based keys returns values in point order (reverse entry order), with the last point returning the default value.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::GetRangePoints", "[PCG][Accessor][Attribute]")
{
	using namespace PCGAttributeAccessorTests;

	AttributeData Data(CreatePointData());

	const TConstPCGValueRange<int64> MetadataEntryRange = Data.PointData->GetConstMetadataEntryValueRange();
	TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(Data.PointData, FPCGAttributePropertySelector());

	REQUIRE(Keys->IsReadOnly());

	float TempFloats[4] = { 1.0f, 2.0f, 3.0f, 4.0f };
	const int32 Index = 0;

	// Points entry are in reverse order, and last one is the default value.
	REQUIRE(Data.FloatAccessor->GetRange(MakeArrayView(TempFloats), Index, *Keys));
	REQUIRE_EQUAL(TempFloats[0], FloatValues[2]);
	REQUIRE_EQUAL(TempFloats[1], FloatValues[1]);
	REQUIRE_EQUAL(TempFloats[2], FloatValues[0]);
	REQUIRE_EQUAL(TempFloats[3], 0.0f);
}

// Tests that setting a single value using point-based keys writes through the metadata entry indirection, preserves existing entry keys, and allocates a new entry for a point that had the default entry.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::SingleSetPoints", "[PCG][Accessor][Attribute]")
{
	using namespace PCGAttributeAccessorTests;

	AttributeData Data(CreatePointData());

	const TConstPCGValueRange<int64> MetadataEntryRange = Data.PointData->GetConstMetadataEntryValueRange();
	TUniquePtr<IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateKeys(Data.PointData, FPCGAttributePropertySelector());

	REQUIRE_EQUAL(Keys->GetNum(), Data.PointData->GetNumPoints());
	REQUIRE_FALSE(Keys->IsReadOnly());

	float TempFloats[4] = { 1.0f, 2.0f, 3.0f, 4.0f };
	const int32 Index = 0;

	for (int32 i = 0; i < 4; ++i)
	{
		REQUIRE(Data.FloatAccessor->Set(TempFloats[i], i, *Keys));
		REQUIRE_EQUAL(Data.FloatAttribute->GetValueFromItemKey(MetadataEntryRange[i]), TempFloats[i]);
	}

	// Points should not have a different entry key, if it was not invalid in the first place
	REQUIRE_EQUAL(MetadataEntryRange[0], PCGMetadataEntryKey(2));
	REQUIRE_EQUAL(MetadataEntryRange[1], PCGMetadataEntryKey(1));
	REQUIRE_EQUAL(MetadataEntryRange[2], PCGFirstEntryKey);

	// Also last point was a default entry. Making sure that we have one more entry and was set correctly
	REQUIRE_EQUAL(Data.PointData->Metadata->GetItemCountForChild(), ValuesSize + 1);
	REQUIRE_EQUAL(MetadataEntryRange[3], PCGMetadataEntryKey(3));
}

// Tests that SetRange using point-based keys writes values through entry indirection, preserves existing entry keys, and allocates a new entry for a point that had the default entry.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::SetRangePoints", "[PCG][Accessor][Attribute]")
{
	using namespace PCGAttributeAccessorTests;

	AttributeData Data(CreatePointData());

	const TConstPCGValueRange<int64> MetadataEntryRange = Data.PointData->GetConstMetadataEntryValueRange();
	TUniquePtr<IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateKeys(Data.PointData, FPCGAttributePropertySelector());

	float TempFloats[4] = { 1.0f, 2.0f, 3.0f, 4.0f };
	const int32 Index = 0;

	REQUIRE(Data.FloatAccessor->SetRange(MakeConstArrayView(TempFloats), Index, *Keys));
	REQUIRE_EQUAL(Data.FloatAttribute->GetValueFromItemKey(MetadataEntryRange[0]), TempFloats[0]);
	REQUIRE_EQUAL(Data.FloatAttribute->GetValueFromItemKey(MetadataEntryRange[1]), TempFloats[1]);
	REQUIRE_EQUAL(Data.FloatAttribute->GetValueFromItemKey(MetadataEntryRange[2]), TempFloats[2]);
	REQUIRE_EQUAL(Data.FloatAttribute->GetValueFromItemKey(MetadataEntryRange[3]), TempFloats[3]);

	// Points should not have a different entry key, if it was not invalid in the first place
	REQUIRE_EQUAL(MetadataEntryRange[0], PCGMetadataEntryKey(2));
	REQUIRE_EQUAL(MetadataEntryRange[1], PCGMetadataEntryKey(1));
	REQUIRE_EQUAL(MetadataEntryRange[2], PCGFirstEntryKey);

	// Also last point was a default entry. Making sure that we have one more entry and was set correctly
	REQUIRE_EQUAL(Data.PointData->Metadata->GetItemCountForChild(), ValuesSize + 1);
	REQUIRE_EQUAL(MetadataEntryRange[3], PCGMetadataEntryKey(3));
}

// Tests Get/Set with constructible type conversion (float<->double). Verifies that conversion fails without AllowConstructible and succeeds with it, for both single and range operations.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::GetWithConstructibleType", "[PCG][Accessor][Attribute]")
{
	UPCGBasePointData* PointData = CreatePointData();

	// float -> double is constructible (std::is_constructible_v<double, float>)
	constexpr float FloatValue = 1.5f;
	FPCGMetadataAttribute<float>* FloatAttribute = PointData->Metadata->CreateAttribute<float>(TEXT("FloatAttribute"), FloatValue, true, false);
	TUniquePtr<IPCGAttributeAccessor> FloatAccessor = PCGAttributeAccessorHelpers::CreateAccessor(FloatAttribute, PointData->Metadata);
	REQUIRE(FloatAccessor);

	// Single point with default entry to test default value access
	PointData->SetNumPoints(1);

	SECTION("Get double from float accessor without AllowConstructible fails")
	{
		TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(PointData, FPCGAttributePropertySelector());
		REQUIRE(Keys);
		double TempDouble = 0.0;
		REQUIRE_FALSE(FloatAccessor->Get(TempDouble, *Keys));
	}

	SECTION("Get double from float accessor with AllowConstructible succeeds")
	{
		TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(PointData, FPCGAttributePropertySelector());
		REQUIRE(Keys);
		double TempDouble = 0.0;
		REQUIRE(FloatAccessor->Get(TempDouble, *Keys, EPCGAttributeAccessorFlags::AllowConstructible));
		REQUIRE_EQUAL(TempDouble, static_cast<double>(FloatValue));
	}

	SECTION("Set double into float accessor without AllowConstructible fails")
	{
		TPCGValueRange<int64> MetadataEntryRange = PointData->GetMetadataEntryValueRange();
		MetadataEntryRange[0] = PointData->Metadata->AddEntry();
		TUniquePtr<IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateKeys(PointData, FPCGAttributePropertySelector());
		REQUIRE(Keys);

		const double DoubleValue = 2.5;
		REQUIRE_FALSE(FloatAccessor->Set(DoubleValue, *Keys));
	}

	SECTION("Set double into float accessor with AllowConstructible succeeds")
	{
		TPCGValueRange<int64> MetadataEntryRange = PointData->GetMetadataEntryValueRange();
		MetadataEntryRange[0] = PointData->Metadata->AddEntry();
		TUniquePtr<IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateKeys(PointData, FPCGAttributePropertySelector());
		REQUIRE(Keys);

		const double DoubleValue = 2.5;
		REQUIRE(FloatAccessor->Set(DoubleValue, *Keys, EPCGAttributeAccessorFlags::AllowConstructible));
		REQUIRE_EQUAL(FloatAttribute->GetValueFromItemKey(MetadataEntryRange[0]), static_cast<float>(DoubleValue));
	}

	SECTION("GetRange double from float accessor with AllowConstructible succeeds")
	{
		constexpr int32 NumElements = 128;
		FRandomStream RandomStream(42);

		PointData->SetNumPoints(NumElements);
		TPCGValueRange<int64> MetadataEntryRange = PointData->GetMetadataEntryValueRange();
		for (int32 i = 0; i < NumElements; ++i)
		{
			MetadataEntryRange[i] = PointData->Metadata->AddEntry();
			FloatAttribute->SetValue(MetadataEntryRange[i], RandomStream.FRand());
		}

		TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(PointData, FPCGAttributePropertySelector());
		REQUIRE(Keys);
		REQUIRE_EQUAL(Keys->GetNum(), NumElements);

		TArray<double> OutDoubles;
		OutDoubles.SetNum(NumElements);

		REQUIRE(FloatAccessor->GetRange(TArrayView<double>(OutDoubles), 0, *Keys, EPCGAttributeAccessorFlags::AllowConstructible));

		for (int32 i = 0; i < NumElements; ++i)
		{
			REQUIRE_EQUAL(OutDoubles[i], static_cast<double>(FloatAttribute->GetValueFromItemKey(MetadataEntryRange[i])));
		}
	}

	SECTION("SetRange double into float accessor with AllowConstructible succeeds")
	{
		constexpr int32 NumElements = 128;
		FRandomStream RandomStream(42);

		PointData->SetNumPoints(NumElements);
		TPCGValueRange<int64> MetadataEntryRange = PointData->GetMetadataEntryValueRange();
		for (int32 i = 0; i < NumElements; ++i)
		{
			MetadataEntryRange[i] = PointData->Metadata->AddEntry();
		}

		TArray<double> InDoubles;
		InDoubles.SetNum(NumElements);
		for (int32 i = 0; i < NumElements; ++i)
		{
			InDoubles[i] = RandomStream.FRand() * 100.0;
		}

		TUniquePtr<IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateKeys(PointData, FPCGAttributePropertySelector());
		REQUIRE(Keys);
		REQUIRE_EQUAL(Keys->GetNum(), NumElements);

		REQUIRE(FloatAccessor->SetRange(TArrayView<const double>(InDoubles), 0, *Keys, EPCGAttributeAccessorFlags::AllowConstructible));

		for (int32 i = 0; i < NumElements; ++i)
		{
			REQUIRE_EQUAL(FloatAttribute->GetValueFromItemKey(MetadataEntryRange[i]), static_cast<float>(InDoubles[i]));
		}
	}
}

// Tests Get/Set with constructible type conversion in arrays (double->float).
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::GetWithConstructibleType::Array", "[PCG][Accessor][Attribute]")
{
	UPCGBasePointData* PointData = CreatePointData();
	static const FName FloatArrayAttributeName = "FloatArrayAttribute";
	// float -> double is constructible (std::is_constructible_v<double, float>)
	FPCGMetadataAttributeBase* FloatArrayAttribute = PointData->Metadata->CreateAttribute<TArray<float>>(FloatArrayAttributeName, {}, true, false);
	TUniquePtr<IPCGAttributeAccessor> FloatArrayAccessor = PCGAttributeAccessorHelpers::CreateAccessor(PointData, FPCGAttributePropertySelector::CreateAttributeSelector(FloatArrayAttributeName));
	REQUIRE(FloatArrayAttribute);
	REQUIRE(FloatArrayAccessor);

	SECTION("GetRange double from float accessor with AllowConstructible succeeds")
	{
		constexpr int32 NumElements = 128;
		FRandomStream RandomStream(42);

		PointData->SetNumPoints(NumElements);
		TPCGValueRange<int64> MetadataEntryRange = PointData->GetMetadataEntryValueRange();
		TArray<float> LocalValues;
		
		for (int32 i = 0; i < NumElements; ++i)
		{
			MetadataEntryRange[i] = PointData->Metadata->AddEntry();
			const int32 ArrayNum = RandomStream.RandRange(2, 5);
			LocalValues.SetNumUninitialized(ArrayNum);
			for (int j = 0; j < ArrayNum; ++j)
			{
				LocalValues[j] = RandomStream.FRand();
			}
			
			FloatArrayAttribute->SetValue(MetadataEntryRange[i], LocalValues);
		}

		TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(PointData, FPCGAttributePropertySelector::CreateAttributeSelector(FloatArrayAttributeName));
		REQUIRE(Keys);
		REQUIRE_EQUAL(Keys->GetNum(), NumElements);

		TArray<PCG::TPCGArrayAccessorWrapper<double>> OutDoubles;
		OutDoubles.SetNum(NumElements);

		REQUIRE(FloatArrayAccessor->GetRange(TArrayView<PCG::TPCGArrayAccessorWrapper<double>>(OutDoubles), 0, *Keys, EPCGAttributeAccessorFlags::AllowConstructible));

		for (int32 i = 0; i < NumElements; ++i)
		{
			REQUIRE(OutDoubles[i].IsOwningMemory());
			TConstArrayView<float> AttributeValues = FloatArrayAttribute->GetValueFromItemKey<TConstArrayView<float>>(MetadataEntryRange[i]);
			REQUIRE(AttributeValues.Num() == OutDoubles[i].GetView().Num());
			CHECK_THAT(AttributeValues, Catch::Matchers::RangeEquals(OutDoubles[i].GetView(), [](const float a, const double b) { return static_cast<double>(a) == b; }));
		}
	}

	SECTION("SetRange double into float accessor with AllowConstructible succeeds")
	{
		constexpr int32 NumElements = 128;
		FRandomStream RandomStream(42);

		PointData->SetNumPoints(NumElements);
		TPCGValueRange<int64> MetadataEntryRange = PointData->GetMetadataEntryValueRange();
		TArray<TArray<double>> LocalValues;
		LocalValues.SetNum(NumElements);
		
		for (int32 i = 0; i < NumElements; ++i)
		{
			MetadataEntryRange[i] = PointData->Metadata->AddEntry();
			const int32 ArrayNum = RandomStream.RandRange(2, 5);
			LocalValues[i].SetNumUninitialized(ArrayNum);
			for (int j = 0; j < ArrayNum; ++j)
			{
				LocalValues[i][j] = static_cast<double>(RandomStream.FRand());
			}
		}

		TUniquePtr<IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateKeys(PointData, FPCGAttributePropertySelector::CreateAttributeSelector(FloatArrayAttributeName));
		REQUIRE(Keys);
		REQUIRE_EQUAL(Keys->GetNum(), NumElements);

		REQUIRE(FloatArrayAccessor->SetRange<TArray<double>>(LocalValues, 0, *Keys, EPCGAttributeAccessorFlags::AllowConstructible));

		for (int32 i = 0; i < NumElements; ++i)
		{
			TConstArrayView<float> AttributeValues = FloatArrayAttribute->GetValueFromItemKey<TConstArrayView<float>>(MetadataEntryRange[i]);
			REQUIRE(AttributeValues.Num() == LocalValues[i].Num());
			CHECK_THAT(AttributeValues, Catch::Matchers::RangeEquals(LocalValues[i], [](const float a, const double b) { return static_cast<double>(a) == b; }));
		}
	}
}

// Tests Get/Set with broadcastable type conversion (float->FVector, float->FString). Verifies that conversion fails without AllowBroadcast and succeeds with it, for both single and range operations.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::GetWithBroadcastableType", "[PCG][Accessor][Attribute]")
{
	UPCGBasePointData* PointData = CreatePointData();

	// float -> FVector is broadcastable, float -> FString is broadcastable
	constexpr float FloatValue = 3.14f;
	FPCGMetadataAttribute<float>* FloatAttribute = PointData->Metadata->CreateAttribute<float>(TEXT("FloatAttribute"), FloatValue, true, false);
	TUniquePtr<IPCGAttributeAccessor> FloatAccessor = PCGAttributeAccessorHelpers::CreateAccessor(FloatAttribute, PointData->Metadata);
	REQUIRE(FloatAccessor);

	// Single point with default entry to test default value access
	PointData->SetNumPoints(1);

	SECTION("Get FVector from float accessor without AllowBroadcast fails")
	{
		TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(PointData, FPCGAttributePropertySelector());
		REQUIRE(Keys);
		FVector TempVector = FVector::ZeroVector;
		REQUIRE_FALSE(FloatAccessor->Get(TempVector, *Keys));
	}

	SECTION("Get FVector from float accessor with AllowBroadcast succeeds")
	{
		TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(PointData, FPCGAttributePropertySelector());
		REQUIRE(Keys);
		FVector TempVector = FVector::ZeroVector;
		REQUIRE(FloatAccessor->Get(TempVector, *Keys, EPCGAttributeAccessorFlags::AllowBroadcast));
		REQUIRE_EQUAL(TempVector, FVector(FloatValue));
	}

	SECTION("Get FString from float accessor with AllowBroadcast succeeds")
	{
		TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(PointData, FPCGAttributePropertySelector());
		REQUIRE(Keys);
		FString TempString;
		REQUIRE(FloatAccessor->Get(TempString, *Keys, EPCGAttributeAccessorFlags::AllowBroadcast));
		REQUIRE_FALSE(TempString.IsEmpty());
	}

	SECTION("Set FVector into float accessor without AllowBroadcast fails")
	{
		TPCGValueRange<int64> MetadataEntryRange = PointData->GetMetadataEntryValueRange();
		MetadataEntryRange[0] = PointData->Metadata->AddEntry();
		TUniquePtr<IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateKeys(PointData, FPCGAttributePropertySelector());
		REQUIRE(Keys);

		const FVector VectorValue(5.0f, 6.0f, 7.0f);
		REQUIRE_FALSE(FloatAccessor->Set(VectorValue, *Keys));
	}

	SECTION("Set float into FVector accessor with AllowBroadcast succeeds")
	{
		FPCGMetadataAttribute<FVector>* VectorAttribute = PointData->Metadata->CreateAttribute<FVector>(TEXT("VectorAttribute"), FVector::ZeroVector, false, false);
		TUniquePtr<IPCGAttributeAccessor> VectorAccessor = PCGAttributeAccessorHelpers::CreateAccessor(VectorAttribute, PointData->Metadata);
		REQUIRE(VectorAccessor);

		TPCGValueRange<int64> MetadataEntryRange = PointData->GetMetadataEntryValueRange();
		MetadataEntryRange[0] = PointData->Metadata->AddEntry();
		TUniquePtr<IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateKeys(PointData, FPCGAttributePropertySelector());
		REQUIRE(Keys);

		REQUIRE(VectorAccessor->Set(FloatValue, *Keys, EPCGAttributeAccessorFlags::AllowBroadcast));
		REQUIRE_EQUAL(VectorAttribute->GetValueFromItemKey(MetadataEntryRange[0]), FVector(FloatValue));
	}

	SECTION("GetRange FVector from float accessor with AllowBroadcast succeeds")
	{
		constexpr int32 NumElements = 128;
		FRandomStream RandomStream(42);

		PointData->SetNumPoints(NumElements);
		TPCGValueRange<int64> MetadataEntryRange = PointData->GetMetadataEntryValueRange();
		for (int32 i = 0; i < NumElements; ++i)
		{
			MetadataEntryRange[i] = PointData->Metadata->AddEntry();
			FloatAttribute->SetValue(MetadataEntryRange[i], RandomStream.FRand());
		}

		TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(PointData, FPCGAttributePropertySelector());
		REQUIRE(Keys);
		REQUIRE_EQUAL(Keys->GetNum(), NumElements);

		TArray<FVector> OutVectors;
		OutVectors.SetNum(NumElements);

		REQUIRE(FloatAccessor->GetRange(TArrayView<FVector>(OutVectors), 0, *Keys, EPCGAttributeAccessorFlags::AllowBroadcast));

		for (int32 i = 0; i < NumElements; ++i)
		{
			const float SourceFloat = FloatAttribute->GetValueFromItemKey(MetadataEntryRange[i]);
			REQUIRE_EQUAL(OutVectors[i], FVector(SourceFloat));
		}
	}

	SECTION("SetRange float into FVector4 accessor with AllowBroadcast succeeds")
	{
		constexpr int32 NumElements = 1024;
		FRandomStream RandomStream(42);

		FPCGMetadataAttribute<FVector4>* Vector4Attribute = PointData->Metadata->CreateAttribute<FVector4>(TEXT("Vector4Attribute"), FVector4::Zero(), false, false);
		TUniquePtr<IPCGAttributeAccessor> Vector4Accessor = PCGAttributeAccessorHelpers::CreateAccessor(Vector4Attribute, PointData->Metadata);
		REQUIRE(Vector4Accessor);

		PointData->SetNumPoints(NumElements);
		TPCGValueRange<int64> MetadataEntryRange = PointData->GetMetadataEntryValueRange();
		for (int32 i = 0; i < NumElements; ++i)
		{
			MetadataEntryRange[i] = PointData->Metadata->AddEntry();
		}

		TArray<float> InFloats;
		InFloats.SetNum(NumElements);
		for (int32 i = 0; i < NumElements; ++i)
		{
			InFloats[i] = RandomStream.FRand() * 100.0f;
		}

		TUniquePtr<IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateKeys(PointData, FPCGAttributePropertySelector());
		REQUIRE(Keys);
		REQUIRE_EQUAL(Keys->GetNum(), NumElements);

		REQUIRE(Vector4Accessor->SetRange(TArrayView<const float>(InFloats), 0, *Keys, EPCGAttributeAccessorFlags::AllowBroadcast));

		for (int32 i = 0; i < NumElements; ++i)
		{
			REQUIRE_EQUAL(Vector4Attribute->GetValueFromItemKey(MetadataEntryRange[i]), FVector4(InFloats[i], InFloats[i], InFloats[i], InFloats[i]));
		}
	}
}

/**
 * Tests Get/Set with broadcastable and constructible type conversion (double->FVector4). Verifies that double->FVector4 conversion succeeds with AllowBroadcastAndConstructible, for both single and range operations and it
 * is the broadcast operation that works, to be sure we are following the previous behavior before the refactor of the accessors.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::GetWithBroadcastAndConstructibleType", "[PCG][Accessor][Attribute]")
{
	UPCGBasePointData* PointData = CreatePointData();

	constexpr double DoubleValue = 2.71828;
	FPCGMetadataAttribute<double>* DoubleAttribute = PointData->Metadata->CreateAttribute<double>(TEXT("DoubleAttribute"), DoubleValue, true, false);
	TUniquePtr<IPCGAttributeAccessor> DoubleAccessor = PCGAttributeAccessorHelpers::CreateAccessor(DoubleAttribute, PointData->Metadata);
	REQUIRE(DoubleAccessor);

	// Single point with default entry to test default value access
	PointData->SetNumPoints(1);

	SECTION("Get FVector4 from double accessor with AllowBroadcastAndConstructible succeeds")
	{
		TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(PointData, FPCGAttributePropertySelector());
		REQUIRE(Keys);
		FVector4 TempVector4 = FVector4::Zero();
		REQUIRE(DoubleAccessor->Get(TempVector4, *Keys, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible));
		REQUIRE_EQUAL(TempVector4, FVector4(DoubleValue, DoubleValue, DoubleValue, DoubleValue));
	}

	SECTION("GetRange FVector4 from double accessor with AllowBroadcastAndConstructible succeeds")
	{
		constexpr int32 NumElements = 128;
		FRandomStream RandomStream(42);

		PointData->SetNumPoints(NumElements);
		TPCGValueRange<int64> MetadataEntryRange = PointData->GetMetadataEntryValueRange();
		for (int32 i = 0; i < NumElements; ++i)
		{
			MetadataEntryRange[i] = PointData->Metadata->AddEntry();
			DoubleAttribute->SetValue(MetadataEntryRange[i], static_cast<double>(RandomStream.FRand()) * 100.0);
		}

		TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(PointData, FPCGAttributePropertySelector());
		REQUIRE(Keys);
		REQUIRE_EQUAL(Keys->GetNum(), NumElements);

		TArray<FVector4> OutVector4s;
		OutVector4s.SetNum(NumElements);

		REQUIRE(DoubleAccessor->GetRange(TArrayView<FVector4>(OutVector4s), 0, *Keys, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible));

		for (int32 i = 0; i < NumElements; ++i)
		{
			const double SourceDouble = DoubleAttribute->GetValueFromItemKey(MetadataEntryRange[i]);
			REQUIRE_EQUAL(OutVector4s[i], FVector4(SourceDouble, SourceDouble, SourceDouble, SourceDouble));
		}
	}
}

// Tests that SetRange with AllowRangeOfValuesIntoSingleContainer writes multiple scalar values into a single array or set container attribute.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::SetRangeOfValuesIntoSingleContainer", "[PCG][Accessor][Attribute]")
{
	UPCGBasePointData* PointData = CreatePointData();

	// Single point with a single metadata entry
	PointData->SetNumPoints(1);
	TPCGValueRange<int64> MetadataEntryRange = PointData->GetMetadataEntryValueRange();
	MetadataEntryRange[0] = PointData->Metadata->AddEntry();

	constexpr int32 NumValues = 8;
	FRandomStream RandomStream(42);
	TArray<float> InFloats;
	InFloats.SetNum(NumValues);
	for (int32 i = 0; i < NumValues; ++i)
	{
		InFloats[i] = RandomStream.FRand() * 100.0f;
	}

	SECTION("SetRange of floats into TArray<float> accessor succeeds")
	{
		FPCGMetadataAttributeBase* ArrayAttribute = PointData->Metadata->CreateAttribute<TArray<float>>(TEXT("ArrayFloatAttribute"), TArray<float>{});
		TUniquePtr<IPCGAttributeAccessor> ArrayAccessor = PCGAttributeAccessorHelpers::CreateAccessor(ArrayAttribute, PointData->Metadata);
		REQUIRE(ArrayAccessor);

		TUniquePtr<IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateKeys(PointData, FPCGAttributePropertySelector());
		REQUIRE(Keys);
		REQUIRE_EQUAL(Keys->GetNum(), 1);

		REQUIRE(ArrayAccessor->SetRange(TArrayView<const float>(InFloats), 0, *Keys, EPCGAttributeAccessorFlags::AllowRangeOfValuesIntoSingleContainer));

		// Read back via GetRange using the array wrapper — GetDefaultAttributeDesc has no specialization for TConstArrayView.
		PCG::TPCGArrayAccessorWrapper<float> OutWrapper;
		TUniquePtr<const IPCGAttributeAccessorKeys> ConstKeys = PCGAttributeAccessorHelpers::CreateConstKeys(PointData, FPCGAttributePropertySelector());
		REQUIRE(ConstKeys);
		REQUIRE(ArrayAccessor->GetRange(MakeArrayView(&OutWrapper, 1), 0, *ConstKeys));

		REQUIRE_EQUAL(OutWrapper.GetView().Num(), NumValues);
		for (int32 i = 0; i < NumValues; ++i)
		{
			REQUIRE_EQUAL(OutWrapper.GetView()[i], InFloats[i]);
		}
	}

	SECTION("SetRange of floats into TArray<float> accessor without flag fails")
	{
		FPCGMetadataAttributeBase* ArrayAttribute = PointData->Metadata->CreateAttribute<TArray<float>>(TEXT("ArrayFloatAttribute"), TArray<float>{});
		TUniquePtr<IPCGAttributeAccessor> ArrayAccessor = PCGAttributeAccessorHelpers::CreateAccessor(ArrayAttribute, PointData->Metadata);
		REQUIRE(ArrayAccessor);

		TUniquePtr<IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateKeys(PointData, FPCGAttributePropertySelector());
		REQUIRE(Keys);

		REQUIRE_FALSE(ArrayAccessor->SetRange(TArrayView<const float>(InFloats), 0, *Keys));
	}

	SECTION("SetRange of floats into TSet<float> accessor succeeds")
	{
		FPCGMetadataAttributeBase* SetAttribute = PointData->Metadata->CreateAttribute<TSet<float>>(TEXT("SetFloatAttribute"), TSet<float>{});
		TUniquePtr<IPCGAttributeAccessor> SetAccessor = PCGAttributeAccessorHelpers::CreateAccessor(SetAttribute, PointData->Metadata);
		REQUIRE(SetAccessor);

		TUniquePtr<IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateKeys(PointData, FPCGAttributePropertySelector());
		REQUIRE(Keys);
		REQUIRE_EQUAL(Keys->GetNum(), 1);

		REQUIRE(SetAccessor->SetRange(TArrayView<const float>(InFloats), 0, *Keys, EPCGAttributeAccessorFlags::AllowRangeOfValuesIntoSingleContainer));

		// Read back via GetRange with TScriptSetWrapper<float>
		PCG::TScriptSetWrapper<float> OutSetWrapper;
		TUniquePtr<const IPCGAttributeAccessorKeys> ConstKeys = PCGAttributeAccessorHelpers::CreateConstKeys(PointData, FPCGAttributePropertySelector());
		REQUIRE(ConstKeys);
		REQUIRE(SetAccessor->GetRange(MakeArrayView(&OutSetWrapper, 1), 0, *ConstKeys));

		REQUIRE_EQUAL(OutSetWrapper.Num(), NumValues);
		for (int32 i = 0; i < NumValues; ++i)
		{
			REQUIRE(OutSetWrapper.Contains(InFloats[i]));
		}
	}

	SECTION("SetRange fails when Keys has more than one entry")
	{
		FPCGMetadataAttributeBase* ArrayAttribute = PointData->Metadata->CreateAttribute<TArray<float>>(TEXT("ArrayFloatAttribute"), TArray<float>{});
		TUniquePtr<IPCGAttributeAccessor> ArrayAccessor = PCGAttributeAccessorHelpers::CreateAccessor(ArrayAttribute, PointData->Metadata);
		REQUIRE(ArrayAccessor);

		// Add a second point so Keys has 2 entries
		PointData->SetNumPoints(2);
		TPCGValueRange<int64> EntryRange = PointData->GetMetadataEntryValueRange();
		EntryRange[1] = PointData->Metadata->AddEntry();

		TUniquePtr<IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateKeys(PointData, FPCGAttributePropertySelector());
		REQUIRE(Keys);
		REQUIRE_EQUAL(Keys->GetNum(), 2);

		REQUIRE_FALSE(ArrayAccessor->SetRange(TArrayView<const float>(InFloats), 0, *Keys, EPCGAttributeAccessorFlags::AllowRangeOfValuesIntoSingleContainer));
	}
}

// Tests that FSoftObjectPath and FSoftClassPath attributes can be read as FString even with StrictType flag (legacy special case), and that non-string type access fails with StrictType.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::SoftObjectPathToStringWithStrictType", "[PCG][Accessor][Attribute]")
{
	UPCGBasePointData* PointData = CreatePointData();

	const FSoftObjectPath SoftPath(TEXT("/Game/Test/MyAsset"));
	FPCGMetadataAttribute<FSoftObjectPath>* SoftPathAttribute = PointData->Metadata->CreateAttribute<FSoftObjectPath>(TEXT("SoftPathAttribute"), SoftPath, false, false);
	TUniquePtr<IPCGAttributeAccessor> SoftPathAccessor = PCGAttributeAccessorHelpers::CreateAccessor(SoftPathAttribute, PointData->Metadata);
	REQUIRE(SoftPathAccessor);

	// Single point with default entry to test default value access
	PointData->SetNumPoints(1);
	TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(PointData, FPCGAttributePropertySelector());
	REQUIRE(Keys);

	SECTION("Get FString from FSoftObjectPath with StrictType succeeds due to legacy special case")
	{
		FString TempString;
		REQUIRE(SoftPathAccessor->Get(TempString, *Keys, EPCGAttributeAccessorFlags::StrictType));
		REQUIRE_EQUAL(TempString, SoftPath.ToString());
	}

	SECTION("Get FString from FSoftObjectPath with no flags also succeeds")
	{
		FString TempString;
		// Without any flags, the conversion is still invalid (no AllowBroadcast), but the special case kicks in
		REQUIRE(SoftPathAccessor->Get(TempString, *Keys));
		REQUIRE_EQUAL(TempString, SoftPath.ToString());
	}

	SECTION("Get FString from FSoftClassPath with StrictType succeeds")
	{
		const FSoftClassPath SoftClassPath(TEXT("/Game/Test/MyClass"));
		FPCGMetadataAttribute<FSoftClassPath>* SoftClassPathAttribute = PointData->Metadata->CreateAttribute<FSoftClassPath>(TEXT("SoftClassPathAttribute"), SoftClassPath, false, false);
		TUniquePtr<IPCGAttributeAccessor> SoftClassPathAccessor = PCGAttributeAccessorHelpers::CreateAccessor(SoftClassPathAttribute, PointData->Metadata);
		REQUIRE(SoftClassPathAccessor);

		FString TempString;
		REQUIRE(SoftClassPathAccessor->Get(TempString, *Keys, EPCGAttributeAccessorFlags::StrictType));
		REQUIRE_EQUAL(TempString, SoftClassPath.ToString());
	}

	SECTION("Get non-string type from FSoftObjectPath with StrictType fails")
	{
		float TempFloat = 0.0f;
		REQUIRE_FALSE(SoftPathAccessor->Get(TempFloat, *Keys, EPCGAttributeAccessorFlags::StrictType));
	}
}

// Tests that CopyTo copies values between accessors of the same underlying type (float->float, vector->vector, string->string).
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::CopyToSameType", "[PCG][Accessor][Attribute]")
{
	using namespace PCGAttributeAccessorTests;

	AttributeData SrcData(CreatePointData());

	TUniquePtr<const IPCGAttributeAccessorKeys> SrcKeys = PCGAttributeAccessorHelpers::CreateConstKeys(SrcData.PointData, FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("FloatAttribute")));
	REQUIRE(SrcKeys);
	REQUIRE_EQUAL(SrcKeys->GetNum(), ValuesSize+1);

	// Destination data with entries (values at defaults)
	auto CopyAndValidate = [this, &SrcKeys]<typename T>(const IPCGAttributeAccessor& SrcAccessor, FName Name, T DestDefaultValue, const T(&Values)[3])
	{
		UPCGBasePointData* DestPointData = CreatePointData();
		FPCGMetadataAttribute<T>* DestAttr = DestPointData->MutableMetadata()->CreateAttribute<T>(Name, DestDefaultValue, false, false);
		DestPointData->SetNumPoints(ValuesSize+1);
		TPCGValueRange<int64> DestMetadataEntryRange = DestPointData->GetMetadataEntryValueRange();

		TUniquePtr<IPCGAttributeAccessor> DestAccessor = PCGAttributeAccessorHelpers::CreateAccessor(DestAttr, DestPointData->MutableMetadata());
		TUniquePtr<IPCGAttributeAccessorKeys> DestKeys = PCGAttributeAccessorHelpers::CreateKeys(DestPointData, FPCGAttributePropertySelector::CreateAttributeSelector(Name));
		REQUIRE(DestKeys);
		REQUIRE_EQUAL(DestKeys->GetNum(), ValuesSize+1);
		
		REQUIRE(SrcAccessor.CopyTo(*SrcKeys, *DestAccessor, *DestKeys, 0, ValuesSize));
		for (int32 i = 0; i < ValuesSize; ++i)
		{
			// Reverse order for values.
			REQUIRE_EQUAL(DestAttr->GetValueFromItemKey(DestMetadataEntryRange[i]), Values[ValuesSize-i-1]);
		}
		
		REQUIRE_EQUAL(DestAttr->GetValueFromItemKey(DestMetadataEntryRange[3]), DestDefaultValue);
	};

	SECTION("Float")
	{
		CopyAndValidate(*SrcData.FloatAccessor, TEXT("DestFloat"), 0.1f, FloatValues);
	}

	SECTION("Vector")
	{
		CopyAndValidate(*SrcData.VectorAccessor, TEXT("DestVector"), FVector::OneVector, VectorValues);
	}

	SECTION("String")
	{
		CopyAndValidate(*SrcData.StringAccessor, TEXT("DestString"), FString{TEXT("Journey before Destination")}, StringValues);
	}
}

// Tests that CopyTo can copy a sub-range starting at a non-zero index, leaving entries outside the range at their attribute defaults.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::CopyToPartialRange", "[PCG][Accessor][Attribute]")
{
	using namespace PCGAttributeAccessorTests;

	// Source with sequential mapping
	AttributeData SrcData(CreatePointData());
	TUniquePtr<const IPCGAttributeAccessorKeys> SrcKeys = PCGAttributeAccessorHelpers::CreateConstKeys(SrcData.PointData, FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("FloatAttribute")));
	REQUIRE(SrcKeys);

	// Destination with distinctive default
	constexpr float DestDefault = -1.0f;
	UPCGBasePointData* DestPointData = CreatePointData();
	FPCGMetadataAttribute<float>* DestFloatAttr = DestPointData->MutableMetadata()->CreateAttribute<float>(TEXT("DestFloat"), DestDefault, true, false);

	DestPointData->SetNumPoints(ValuesSize + 1);
	TPCGValueRange<int64> DestMetadataEntryRange = DestPointData->GetMetadataEntryValueRange();

	TUniquePtr<IPCGAttributeAccessor> DestFloatAccessor = PCGAttributeAccessorHelpers::CreateAccessor(DestFloatAttr, DestPointData->MutableMetadata());
	TUniquePtr<IPCGAttributeAccessorKeys> DestKeys = PCGAttributeAccessorHelpers::CreateKeys(DestPointData, FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("DestFloat")));
	REQUIRE(DestKeys);

	SECTION("Copy from index 1 with count 2")
	{
		REQUIRE(SrcData.FloatAccessor->CopyTo(*SrcKeys, *DestFloatAccessor, *DestKeys, 1, 2));
		// Entry 0 and 3 should be untouched (default)
		REQUIRE_EQUAL(DestFloatAttr->GetValueFromItemKey(DestMetadataEntryRange[0]), DestDefault);
		REQUIRE_EQUAL(DestFloatAttr->GetValueFromItemKey(DestMetadataEntryRange[3]), DestDefault);
		// Entries 1 and 2 should have source values (reverse order)
		REQUIRE_EQUAL(DestFloatAttr->GetValueFromItemKey(DestMetadataEntryRange[1]), FloatValues[1]);
		REQUIRE_EQUAL(DestFloatAttr->GetValueFromItemKey(DestMetadataEntryRange[2]), FloatValues[0]);
	}

	SECTION("Copy only first element")
	{
		REQUIRE(SrcData.FloatAccessor->CopyTo(*SrcKeys, *DestFloatAccessor, *DestKeys, 0, 1));
		// Entry 0 should have source value (reverse order)
		REQUIRE_EQUAL(DestFloatAttr->GetValueFromItemKey(DestMetadataEntryRange[0]), FloatValues[2]);
		// Entries 1, 2 and 3 should be untouched (default)
		REQUIRE_EQUAL(DestFloatAttr->GetValueFromItemKey(DestMetadataEntryRange[1]), DestDefault);
		REQUIRE_EQUAL(DestFloatAttr->GetValueFromItemKey(DestMetadataEntryRange[2]), DestDefault);
		REQUIRE_EQUAL(DestFloatAttr->GetValueFromItemKey(DestMetadataEntryRange[3]), DestDefault);
	}
}

// Tests that CopyTo with AllowConstructible copies between constructible types (float->double), fails without the flag, and works with larger element counts.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::CopyToConstructibleType", "[PCG][Accessor][Attribute]")
{
	using namespace PCGAttributeAccessorTests;
	
	auto ExecuteTest = [this](int32 Count, EPCGAttributeAccessorFlags Flags, bool bShouldFail)
	{
		// Source: float
		UPCGBasePointData* SrcPointData = CreatePointData();
		FPCGMetadataAttribute<float>* SrcFloatAttr = SrcPointData->MutableMetadata()->CreateAttribute<float>(TEXT("SrcFloat"), 0.0f, true, false);
		SrcPointData->SetNumPoints(Count);
		TPCGValueRange<int64> SrcEntryRange = SrcPointData->GetMetadataEntryValueRange();
		TArray<float> Values;
		Values.SetNumUninitialized(Count);

		FRandomStream RandomStream(42);

		for (int32 i = 0; i < Count; ++i)
		{
			SrcEntryRange[i] = SrcPointData->MutableMetadata()->AddEntry();
			Values[i] = RandomStream.FRand();
			SrcFloatAttr->SetValue(SrcEntryRange[i], Values[i]);
		}
			
		TUniquePtr<IPCGAttributeAccessor> SrcFloatAccessor = PCGAttributeAccessorHelpers::CreateAccessor(SrcFloatAttr, SrcPointData->MutableMetadata());
		TUniquePtr<const IPCGAttributeAccessorKeys> SrcKeys = PCGAttributeAccessorHelpers::CreateConstKeys(SrcPointData, FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("SrcFloat")));
		REQUIRE(SrcKeys);

		// Destination: double
		UPCGBasePointData* DestPointData = CreatePointData();
		FPCGMetadataAttribute<double>* DestDoubleAttr = DestPointData->MutableMetadata()->CreateAttribute<double>(TEXT("DestDouble"), 0.0, true, false);
		DestPointData->SetNumPoints(Count);
		TPCGValueRange<int64> DestEntryRange = DestPointData->GetMetadataEntryValueRange();
			
		TUniquePtr<IPCGAttributeAccessor> DestDoubleAccessor = PCGAttributeAccessorHelpers::CreateAccessor(DestDoubleAttr, DestPointData->MutableMetadata());
		TUniquePtr<IPCGAttributeAccessorKeys> DestKeys = PCGAttributeAccessorHelpers::CreateKeys(DestPointData, FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("DestDouble")));
		REQUIRE(DestKeys);
			
		REQUIRE(SrcFloatAccessor->CopyTo(*SrcKeys, *DestDoubleAccessor, *DestKeys, 0, Count, Flags) != bShouldFail);
		if (!bShouldFail)
		{
			for (int32 i = 0; i < Count; ++i)
			{
				REQUIRE_EQUAL(DestDoubleAttr->GetValueFromItemKey(DestEntryRange[i]), static_cast<double>(Values[i]));
			}
		}
	};
	
	SECTION("CopyTo float->double without AllowConstructible fails")
	{
		PCGTests::FPCGTestsLogOutputDevice LogCapture(/*bSuppressErrors=*/true);
		ExecuteTest(ValuesSize, EPCGAttributeAccessorFlags::StrictType, /*bShouldFail=*/true);
		REQUIRE(LogCapture.NbMessageReceived.Contains(ELogVerbosity::Error));
	}

	SECTION("CopyTo float->double with AllowConstructible succeeds")
	{
		ExecuteTest(ValuesSize, EPCGAttributeAccessorFlags::AllowConstructible, /*bShouldFail=*/false);
	}

	SECTION("CopyTo float->double with large count")
	{
		ExecuteTest(1024, EPCGAttributeAccessorFlags::AllowConstructible, /*bShouldFail=*/false);
	}
}

// Tests that CopyTo with AllowBroadcast copies between broadcastable types (float->FVector), fails without the flag, and works with larger element counts.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::CopyToBroadcastableType", "[PCG][Accessor][Attribute]")
{
	using namespace PCGAttributeAccessorTests;

	auto ExecuteTest = [this](int32 Count, EPCGAttributeAccessorFlags Flags, bool bShouldFail)
	{
		// Source: float
		UPCGBasePointData* SrcPointData = CreatePointData();
		FPCGMetadataAttribute<float>* SrcFloatAttr = SrcPointData->MutableMetadata()->CreateAttribute<float>(TEXT("SrcFloat"), 0.0f, true, false);
		SrcPointData->SetNumPoints(Count);
		TPCGValueRange<int64> SrcEntryRange = SrcPointData->GetMetadataEntryValueRange();
		TArray<float> Values;
		Values.SetNumUninitialized(Count);

		FRandomStream RandomStream(42);

		for (int32 i = 0; i < Count; ++i)
		{
			SrcEntryRange[i] = SrcPointData->MutableMetadata()->AddEntry();
			Values[i] = RandomStream.FRand();
			SrcFloatAttr->SetValue(SrcEntryRange[i], Values[i]);
		}
	
		TUniquePtr<IPCGAttributeAccessor> SrcFloatAccessor = PCGAttributeAccessorHelpers::CreateAccessor(SrcFloatAttr, SrcPointData->MutableMetadata());
		TUniquePtr<const IPCGAttributeAccessorKeys> SrcKeys = PCGAttributeAccessorHelpers::CreateConstKeys(SrcPointData, FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("SrcFloat")));
		REQUIRE(SrcKeys);

		// Destination: FVector
		UPCGBasePointData* DestPointData = CreatePointData();
		FPCGMetadataAttribute<FVector>* DestVectorAttr = DestPointData->MutableMetadata()->CreateAttribute<FVector>(TEXT("DestVector"), FVector::ZeroVector, false, false);
		DestPointData->SetNumPoints(ValuesSize);
		TPCGValueRange<int64> DestEntryRange = DestPointData->GetMetadataEntryValueRange();

		TUniquePtr<IPCGAttributeAccessor> DestVectorAccessor = PCGAttributeAccessorHelpers::CreateAccessor(DestVectorAttr, DestPointData->MutableMetadata());
		TUniquePtr<IPCGAttributeAccessorKeys> DestKeys = PCGAttributeAccessorHelpers::CreateKeys(DestPointData, FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("DestVector")));
		REQUIRE(DestKeys);
			
		REQUIRE(SrcFloatAccessor->CopyTo(*SrcKeys, *DestVectorAccessor, *DestKeys, 0, ValuesSize, Flags) != bShouldFail);
		if (!bShouldFail)
		{
			for (int32 i = 0; i < ValuesSize; ++i)
			{
				REQUIRE_EQUAL(DestVectorAttr->GetValueFromItemKey(DestEntryRange[i]), FVector(Values[i]));
			}
		}
	};

	SECTION("CopyTo float->FVector without AllowBroadcast fails")
	{
		PCGTests::FPCGTestsLogOutputDevice LogCapture(/*bSuppressErrors=*/true);
		ExecuteTest(ValuesSize, EPCGAttributeAccessorFlags::StrictType, /*bShouldFail=*/true);
		REQUIRE(LogCapture.NbMessageReceived.Contains(ELogVerbosity::Error));
	}

	SECTION("CopyTo float->FVector with AllowBroadcast succeeds")
	{
		ExecuteTest(ValuesSize, EPCGAttributeAccessorFlags::AllowBroadcast, /*bShouldFail=*/false);
	}

	SECTION("CopyTo float->FVector with large count")
	{
		ExecuteTest(1024, EPCGAttributeAccessorFlags::AllowBroadcast, /*bShouldFail=*/false);
	}
}

// Tests that CopyTo with AllowConstructible copies between array attributes whose element types are constructible
// (TArray<float>->TArray<double>), fails without the flag, and works with larger element counts. Each entry has a
// different array length so the test also covers the per-entry buffer allocation path.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::CopyToConstructibleType::Array", "[PCG][Accessor][Attribute]")
{
	using namespace PCGAttributeAccessorTests;

	auto ExecuteTest = [this](int32 Count, EPCGAttributeAccessorFlags Flags, bool bShouldFail)
	{
		// Source: TArray<float>
		UPCGBasePointData* SrcPointData = CreatePointData();
		FPCGMetadataAttributeBase* SrcArrayAttr = SrcPointData->MutableMetadata()->CreateAttribute<TArray<float>>(TEXT("SrcArrayFloat"), {}, true, false);
		REQUIRE(SrcArrayAttr);
		SrcPointData->SetNumPoints(Count);
		TPCGValueRange<int64> SrcEntryRange = SrcPointData->GetMetadataEntryValueRange();
		TArray<TArray<float>> Values;
		Values.SetNum(Count);

		FRandomStream RandomStream(42);

		for (int32 i = 0; i < Count; ++i)
		{
			SrcEntryRange[i] = SrcPointData->MutableMetadata()->AddEntry();
			const int32 ArrayNum = RandomStream.RandRange(2, 5);
			Values[i].SetNumUninitialized(ArrayNum);
			for (int32 j = 0; j < ArrayNum; ++j)
			{
				Values[i][j] = RandomStream.FRand();
			}
			SrcArrayAttr->SetValue(SrcEntryRange[i], Values[i]);
		}

		TUniquePtr<IPCGAttributeAccessor> SrcArrayAccessor = PCGAttributeAccessorHelpers::CreateAccessor(SrcArrayAttr, SrcPointData->MutableMetadata());
		TUniquePtr<const IPCGAttributeAccessorKeys> SrcKeys = PCGAttributeAccessorHelpers::CreateConstKeys(SrcPointData, FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("SrcArrayFloat")));
		REQUIRE(SrcKeys);

		// Destination: TArray<double>
		UPCGBasePointData* DestPointData = CreatePointData();
		FPCGMetadataAttributeBase* DestArrayAttr = DestPointData->MutableMetadata()->CreateAttribute<TArray<double>>(TEXT("DestArrayDouble"), {}, true, false);
		REQUIRE(DestArrayAttr);
		DestPointData->SetNumPoints(Count);
		TPCGValueRange<int64> DestEntryRange = DestPointData->GetMetadataEntryValueRange();

		TUniquePtr<IPCGAttributeAccessor> DestArrayAccessor = PCGAttributeAccessorHelpers::CreateAccessor(DestArrayAttr, DestPointData->MutableMetadata());
		TUniquePtr<IPCGAttributeAccessorKeys> DestKeys = PCGAttributeAccessorHelpers::CreateKeys(DestPointData, FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("DestArrayDouble")));
		REQUIRE(DestKeys);

		REQUIRE(SrcArrayAccessor->CopyTo(*SrcKeys, *DestArrayAccessor, *DestKeys, 0, Count, Flags) != bShouldFail);
		if (!bShouldFail)
		{
			for (int32 i = 0; i < Count; ++i)
			{
				TConstArrayView<double> AttributeValues = DestArrayAttr->GetValueFromItemKey<TConstArrayView<double>>(DestEntryRange[i]);
				REQUIRE_EQUAL(AttributeValues.Num(), Values[i].Num());
				CHECK_THAT(Values[i], Catch::Matchers::RangeEquals(AttributeValues, [](const float a, const double b) { return static_cast<double>(a) == b; }));
			}
		}
	};

	SECTION("CopyTo TArray<float>->TArray<double> without AllowConstructible fails")
	{
		PCGTests::FPCGTestsLogOutputDevice LogCapture(/*bSuppressErrors=*/true);
		ExecuteTest(ValuesSize, EPCGAttributeAccessorFlags::StrictType, /*bShouldFail=*/true);
		REQUIRE(LogCapture.NbMessageReceived.Contains(ELogVerbosity::Error));
	}

	SECTION("CopyTo TArray<float>->TArray<double> with AllowConstructible succeeds")
	{
		ExecuteTest(ValuesSize, EPCGAttributeAccessorFlags::AllowConstructible, /*bShouldFail=*/false);
	}

	SECTION("CopyTo TArray<float>->TArray<double> with large count")
	{
		ExecuteTest(1024, EPCGAttributeAccessorFlags::AllowConstructible, /*bShouldFail=*/false);
	}
}

// Tests that CopyTo with AllowBroadcast copies between array attributes whose element types are broadcastable
// (TArray<float>->TArray<FVector>), fails without the flag, and works with larger element counts. Each entry has a
// different array length so the test also covers the per-entry buffer allocation path.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::CopyToBroadcastableType::Array", "[PCG][Accessor][Attribute]")
{
	using namespace PCGAttributeAccessorTests;

	auto ExecuteTest = [this](int32 Count, EPCGAttributeAccessorFlags Flags, bool bShouldFail)
	{
		// Source: TArray<float>
		UPCGBasePointData* SrcPointData = CreatePointData();
		FPCGMetadataAttributeBase* SrcArrayAttr = SrcPointData->MutableMetadata()->CreateAttribute<TArray<float>>(TEXT("SrcArrayFloat"), {}, true, false);
		REQUIRE(SrcArrayAttr);
		SrcPointData->SetNumPoints(Count);
		TPCGValueRange<int64> SrcEntryRange = SrcPointData->GetMetadataEntryValueRange();
		TArray<TArray<float>> Values;
		Values.SetNum(Count);

		FRandomStream RandomStream(42);

		for (int32 i = 0; i < Count; ++i)
		{
			SrcEntryRange[i] = SrcPointData->MutableMetadata()->AddEntry();
			const int32 ArrayNum = RandomStream.RandRange(2, 5);
			Values[i].SetNumUninitialized(ArrayNum);
			for (int32 j = 0; j < ArrayNum; ++j)
			{
				Values[i][j] = RandomStream.FRand();
			}
			SrcArrayAttr->SetValue(SrcEntryRange[i], Values[i]);
		}

		TUniquePtr<IPCGAttributeAccessor> SrcArrayAccessor = PCGAttributeAccessorHelpers::CreateAccessor(SrcArrayAttr, SrcPointData->MutableMetadata());
		TUniquePtr<const IPCGAttributeAccessorKeys> SrcKeys = PCGAttributeAccessorHelpers::CreateConstKeys(SrcPointData, FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("SrcArrayFloat")));
		REQUIRE(SrcKeys);

		// Destination: TArray<FVector>
		UPCGBasePointData* DestPointData = CreatePointData();
		FPCGMetadataAttributeBase* DestArrayAttr = DestPointData->MutableMetadata()->CreateAttribute<TArray<FVector>>(TEXT("DestArrayVector"), {}, false, false);
		REQUIRE(DestArrayAttr);
		DestPointData->SetNumPoints(Count);
		TPCGValueRange<int64> DestEntryRange = DestPointData->GetMetadataEntryValueRange();

		TUniquePtr<IPCGAttributeAccessor> DestArrayAccessor = PCGAttributeAccessorHelpers::CreateAccessor(DestArrayAttr, DestPointData->MutableMetadata());
		TUniquePtr<IPCGAttributeAccessorKeys> DestKeys = PCGAttributeAccessorHelpers::CreateKeys(DestPointData, FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("DestArrayVector")));
		REQUIRE(DestKeys);

		REQUIRE(SrcArrayAccessor->CopyTo(*SrcKeys, *DestArrayAccessor, *DestKeys, 0, Count, Flags) != bShouldFail);
		if (!bShouldFail)
		{
			for (int32 i = 0; i < Count; ++i)
			{
				TConstArrayView<FVector> AttributeValues = DestArrayAttr->GetValueFromItemKey<TConstArrayView<FVector>>(DestEntryRange[i]);
				REQUIRE_EQUAL(AttributeValues.Num(), Values[i].Num());
				CHECK_THAT(Values[i], Catch::Matchers::RangeEquals(AttributeValues, [](const float a, const FVector& b) { return b == FVector(a); }));
			}
		}
	};

	SECTION("CopyTo TArray<float>->TArray<FVector> without AllowBroadcast fails")
	{
		PCGTests::FPCGTestsLogOutputDevice LogCapture(/*bSuppressErrors=*/true);
		ExecuteTest(ValuesSize, EPCGAttributeAccessorFlags::StrictType, /*bShouldFail=*/true);
		REQUIRE(LogCapture.NbMessageReceived.Contains(ELogVerbosity::Error));
	}

	SECTION("CopyTo TArray<float>->TArray<FVector> with AllowBroadcast succeeds")
	{
		ExecuteTest(ValuesSize, EPCGAttributeAccessorFlags::AllowBroadcast, /*bShouldFail=*/false);
	}

	SECTION("CopyTo TArray<float>->TArray<FVector> with large count")
	{
		ExecuteTest(1024, EPCGAttributeAccessorFlags::AllowBroadcast, /*bShouldFail=*/false);
	}
}

// Tests that CopyTo copies between array attributes of the same type (TArray<float>->TArray<float>) over
// multiple entries. The destination must contain every source entry's array, not just the first.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::CopyToSameType::Array", "[PCG][Accessor][Attribute]")
{
	using namespace PCGAttributeAccessorTests;

	auto ExecuteTest = [this](int32 Count)
	{
		UPCGBasePointData* SrcPointData = CreatePointData();
		FPCGMetadataAttributeBase* SrcArrayAttr = SrcPointData->MutableMetadata()->CreateAttribute<TArray<float>>(TEXT("SrcArrayFloat"), {}, true, false);
		REQUIRE(SrcArrayAttr);
		SrcPointData->SetNumPoints(Count);
		TPCGValueRange<int64> SrcEntryRange = SrcPointData->GetMetadataEntryValueRange();
		TArray<TArray<float>> Values;
		Values.SetNum(Count);

		FRandomStream RandomStream(42);
		for (int32 i = 0; i < Count; ++i)
		{
			SrcEntryRange[i] = SrcPointData->MutableMetadata()->AddEntry();
			const int32 ArrayNum = RandomStream.RandRange(2, 5);
			Values[i].SetNumUninitialized(ArrayNum);
			for (int32 j = 0; j < ArrayNum; ++j)
			{
				Values[i][j] = RandomStream.FRand();
			}
			SrcArrayAttr->SetValue(SrcEntryRange[i], Values[i]);
		}

		TUniquePtr<IPCGAttributeAccessor> SrcArrayAccessor = PCGAttributeAccessorHelpers::CreateAccessor(SrcArrayAttr, SrcPointData->MutableMetadata());
		TUniquePtr<const IPCGAttributeAccessorKeys> SrcKeys = PCGAttributeAccessorHelpers::CreateConstKeys(SrcPointData, FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("SrcArrayFloat")));
		REQUIRE(SrcKeys);

		UPCGBasePointData* DestPointData = CreatePointData();
		FPCGMetadataAttributeBase* DestArrayAttr = DestPointData->MutableMetadata()->CreateAttribute<TArray<float>>(TEXT("DestArrayFloat"), {}, true, false);
		REQUIRE(DestArrayAttr);
		DestPointData->SetNumPoints(Count);
		TPCGValueRange<int64> DestEntryRange = DestPointData->GetMetadataEntryValueRange();

		TUniquePtr<IPCGAttributeAccessor> DestArrayAccessor = PCGAttributeAccessorHelpers::CreateAccessor(DestArrayAttr, DestPointData->MutableMetadata());
		TUniquePtr<IPCGAttributeAccessorKeys> DestKeys = PCGAttributeAccessorHelpers::CreateKeys(DestPointData, FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("DestArrayFloat")));
		REQUIRE(DestKeys);

		REQUIRE(SrcArrayAccessor->CopyTo(*SrcKeys, *DestArrayAccessor, *DestKeys, 0, Count));
		for (int32 i = 0; i < Count; ++i)
		{
			TConstArrayView<float> AttributeValues = DestArrayAttr->GetValueFromItemKey<TConstArrayView<float>>(DestEntryRange[i]);
			REQUIRE_EQUAL(AttributeValues.Num(), Values[i].Num());
			CHECK_THAT(Values[i], Catch::Matchers::RangeEquals(AttributeValues));
		}
	};

	SECTION("CopyTo TArray<float>->TArray<float> writes all entries")
	{
		ExecuteTest(ValuesSize);
	}

	SECTION("CopyTo TArray<float>->TArray<float> with large count")
	{
		ExecuteTest(1024);
	}
}

// Tests the special case where CopyTo is asked to write multiple source entries from a single-value
// attribute into a single container entry of an array attribute (DestKeys.GetNum() == 1). The special
// case is gated by EPCGAttributeAccessorFlags::AllowRangeOfValuesIntoSingleContainer.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::CopyToSingleValueIntoArray", "[PCG][Accessor][Attribute]")
{
	using namespace PCGAttributeAccessorTests;

	// Source: float attribute with ValuesSize entries.
	UPCGBasePointData* SrcPointData = CreatePointData();
	FPCGMetadataAttribute<float>* SrcAttr = SrcPointData->MutableMetadata()->CreateAttribute<float>(TEXT("SrcFloat"), 0.0f, true, false);
	SrcPointData->SetNumPoints(ValuesSize);
	TPCGValueRange<int64> SrcEntryRange = SrcPointData->GetMetadataEntryValueRange();
	for (int32 i = 0; i < ValuesSize; ++i)
	{
		SrcEntryRange[i] = SrcPointData->MutableMetadata()->AddEntry();
		SrcAttr->SetValue(SrcEntryRange[i], FloatValues[i]);
	}

	TUniquePtr<IPCGAttributeAccessor> SrcAccessor = PCGAttributeAccessorHelpers::CreateAccessor(SrcAttr, SrcPointData->MutableMetadata());
	TUniquePtr<const IPCGAttributeAccessorKeys> SrcKeys = PCGAttributeAccessorHelpers::CreateConstKeys(SrcPointData, FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("SrcFloat")));
	REQUIRE(SrcKeys);

	SECTION("float -> TArray<float> with single dest key collects all source values")
	{
		UPCGBasePointData* DestPointData = CreatePointData();
		FPCGMetadataAttributeBase* DestArrayAttr = DestPointData->MutableMetadata()->CreateAttribute<TArray<float>>(TEXT("DestArrayFloat"), {}, true, false);
		REQUIRE(DestArrayAttr);
		DestPointData->SetNumPoints(1);
		TPCGValueRange<int64> DestEntryRange = DestPointData->GetMetadataEntryValueRange();

		TUniquePtr<IPCGAttributeAccessor> DestAccessor = PCGAttributeAccessorHelpers::CreateAccessor(DestArrayAttr, DestPointData->MutableMetadata());
		TUniquePtr<IPCGAttributeAccessorKeys> DestKeys = PCGAttributeAccessorHelpers::CreateKeys(DestPointData, FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("DestArrayFloat")));
		REQUIRE(DestKeys);
		REQUIRE_EQUAL(DestKeys->GetNum(), 1);

		REQUIRE(SrcAccessor->CopyTo(*SrcKeys, *DestAccessor, *DestKeys, 0, ValuesSize, EPCGAttributeAccessorFlags::AllowRangeOfValuesIntoSingleContainer));

		TConstArrayView<float> ResultArray = DestArrayAttr->GetValueFromItemKey<TConstArrayView<float>>(DestEntryRange[0]);
		REQUIRE_EQUAL(ResultArray.Num(), ValuesSize);
		for (int32 i = 0; i < ValuesSize; ++i)
		{
			REQUIRE_EQUAL(ResultArray[i], FloatValues[i]);
		}
	}

	SECTION("float -> TArray<double> with single dest key constructs all values")
	{
		UPCGBasePointData* DestPointData = CreatePointData();
		FPCGMetadataAttributeBase* DestArrayAttr = DestPointData->MutableMetadata()->CreateAttribute<TArray<double>>(TEXT("DestArrayDouble"), {}, true, false);
		REQUIRE(DestArrayAttr);
		DestPointData->SetNumPoints(1);
		TPCGValueRange<int64> DestEntryRange = DestPointData->GetMetadataEntryValueRange();

		TUniquePtr<IPCGAttributeAccessor> DestAccessor = PCGAttributeAccessorHelpers::CreateAccessor(DestArrayAttr, DestPointData->MutableMetadata());
		TUniquePtr<IPCGAttributeAccessorKeys> DestKeys = PCGAttributeAccessorHelpers::CreateKeys(DestPointData, FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("DestArrayDouble")));
		REQUIRE(DestKeys);
		REQUIRE_EQUAL(DestKeys->GetNum(), 1);

		REQUIRE(SrcAccessor->CopyTo(*SrcKeys, *DestAccessor, *DestKeys, 0, ValuesSize, EPCGAttributeAccessorFlags::AllowRangeOfValuesIntoSingleContainer | EPCGAttributeAccessorFlags::AllowConstructible));

		TConstArrayView<double> ResultArray = DestArrayAttr->GetValueFromItemKey<TConstArrayView<double>>(DestEntryRange[0]);
		REQUIRE_EQUAL(ResultArray.Num(), ValuesSize);
		for (int32 i = 0; i < ValuesSize; ++i)
		{
			REQUIRE_EQUAL(ResultArray[i], static_cast<double>(FloatValues[i]));
		}
	}

	SECTION("float -> TArray<FVector> with single dest key broadcasts all values")
	{
		UPCGBasePointData* DestPointData = CreatePointData();
		FPCGMetadataAttributeBase* DestArrayAttr = DestPointData->MutableMetadata()->CreateAttribute<TArray<FVector>>(TEXT("DestArrayVector"), {}, false, false);
		REQUIRE(DestArrayAttr);
		DestPointData->SetNumPoints(1);
		TPCGValueRange<int64> DestEntryRange = DestPointData->GetMetadataEntryValueRange();

		TUniquePtr<IPCGAttributeAccessor> DestAccessor = PCGAttributeAccessorHelpers::CreateAccessor(DestArrayAttr, DestPointData->MutableMetadata());
		TUniquePtr<IPCGAttributeAccessorKeys> DestKeys = PCGAttributeAccessorHelpers::CreateKeys(DestPointData, FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("DestArrayVector")));
		REQUIRE(DestKeys);
		REQUIRE_EQUAL(DestKeys->GetNum(), 1);

		REQUIRE(SrcAccessor->CopyTo(*SrcKeys, *DestAccessor, *DestKeys, 0, ValuesSize, EPCGAttributeAccessorFlags::AllowRangeOfValuesIntoSingleContainer | EPCGAttributeAccessorFlags::AllowBroadcast));

		TConstArrayView<FVector> ResultArray = DestArrayAttr->GetValueFromItemKey<TConstArrayView<FVector>>(DestEntryRange[0]);
		REQUIRE_EQUAL(ResultArray.Num(), ValuesSize);
		for (int32 i = 0; i < ValuesSize; ++i)
		{
			REQUIRE_EQUAL(ResultArray[i], FVector(FloatValues[i]));
		}
	}

	SECTION("float -> TArray<float> without AllowRangeOfValuesIntoSingleContainer fails")
	{
		PCGTests::FPCGTestsLogOutputDevice LogCapture(/*bSuppressErrors=*/true);

		UPCGBasePointData* DestPointData = CreatePointData();
		FPCGMetadataAttributeBase* DestArrayAttr = DestPointData->MutableMetadata()->CreateAttribute<TArray<float>>(TEXT("DestArrayFloat"), {}, true, false);
		REQUIRE(DestArrayAttr);
		DestPointData->SetNumPoints(1);

		TUniquePtr<IPCGAttributeAccessor> DestAccessor = PCGAttributeAccessorHelpers::CreateAccessor(DestArrayAttr, DestPointData->MutableMetadata());
		TUniquePtr<IPCGAttributeAccessorKeys> DestKeys = PCGAttributeAccessorHelpers::CreateKeys(DestPointData, FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("DestArrayFloat")));
		REQUIRE(DestKeys);

		REQUIRE_FALSE(SrcAccessor->CopyTo(*SrcKeys, *DestAccessor, *DestKeys, 0, ValuesSize));
		REQUIRE(LogCapture.NbMessageReceived.Contains(ELogVerbosity::Error));
	}

	SECTION("float -> TArray<float> with multi-entry dest fails even with the flag (DestKeys.GetNum() must be 1)")
	{
		PCGTests::FPCGTestsLogOutputDevice LogCapture(/*bSuppressErrors=*/true);

		UPCGBasePointData* DestPointData = CreatePointData();
		FPCGMetadataAttributeBase* DestArrayAttr = DestPointData->MutableMetadata()->CreateAttribute<TArray<float>>(TEXT("DestArrayFloat"), {}, true, false);
		REQUIRE(DestArrayAttr);
		DestPointData->SetNumPoints(ValuesSize);
		TPCGValueRange<int64> DestEntryRange = DestPointData->GetMetadataEntryValueRange();
		for (int32 i = 0; i < ValuesSize; ++i)
		{
			DestEntryRange[i] = DestPointData->MutableMetadata()->AddEntry();
		}

		TUniquePtr<IPCGAttributeAccessor> DestAccessor = PCGAttributeAccessorHelpers::CreateAccessor(DestArrayAttr, DestPointData->MutableMetadata());
		TUniquePtr<IPCGAttributeAccessorKeys> DestKeys = PCGAttributeAccessorHelpers::CreateKeys(DestPointData, FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("DestArrayFloat")));
		REQUIRE(DestKeys);
		REQUIRE_EQUAL(DestKeys->GetNum(), ValuesSize);

		REQUIRE_FALSE(SrcAccessor->CopyTo(*SrcKeys, *DestAccessor, *DestKeys, 0, ValuesSize, EPCGAttributeAccessorFlags::AllowRangeOfValuesIntoSingleContainer));
		REQUIRE(LogCapture.NbMessageReceived.Contains(ELogVerbosity::Error));
	}
}

// Tests that SetRange/GetRange through the attribute accessor round-trips correctly on a
// compressed TArray<FString> attribute, and that the underlying attribute still deduplicates
// duplicate array payloads (entries sharing identical arrays share a single stored value key).
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::CompressedArrayRoundTrip", "[PCG][Accessor][Attribute]")
{
	UPCGBasePointData* PointData = CreatePointData();
	static const FName AttributeName = TEXT("StringArr");
	FPCGMetadataAttributeBase* Attr = PointData->Metadata->CreateAttribute<TArray<FString>>(AttributeName, TArray<FString>{}, true, false);
	REQUIRE(Attr);
	REQUIRE(Attr->DoesCompressData());

	constexpr int32 NumPoints = 5;
	PointData->SetNumPoints(NumPoints);
	TPCGValueRange<int64> MetadataEntryRange = PointData->GetMetadataEntryValueRange();
	for (int32 i = 0; i < NumPoints; ++i)
	{
		MetadataEntryRange[i] = PointData->Metadata->AddEntry();
	}

	TArray<TArray<FString>> Inputs;
	Inputs.Add({TEXT("alpha"), TEXT("beta")});                   // 0
	Inputs.Add({TEXT("gamma")});                                 // 1
	Inputs.Add({TEXT("alpha"), TEXT("beta")});                   // 2 -> same as 0
	Inputs.Add({TEXT("gamma")});                                 // 3 -> same as 1
	Inputs.Add({TEXT("delta"), TEXT("epsilon"), TEXT("zeta")});  // 4

	TUniquePtr<IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateAccessor(PointData, FPCGAttributePropertySelector::CreateAttributeSelector(AttributeName));
	REQUIRE(Accessor);

	TUniquePtr<IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateKeys(PointData, FPCGAttributePropertySelector::CreateAttributeSelector(AttributeName));
	REQUIRE(Keys);
	REQUIRE_EQUAL(Keys->GetNum(), NumPoints);

	REQUIRE(Accessor->SetRange<TArray<FString>>(Inputs, 0, *Keys));

	// Read back via the accessor using the array wrapper
	TArray<PCG::TPCGArrayAccessorWrapper<FString>> Out;
	Out.SetNum(NumPoints);
	TUniquePtr<const IPCGAttributeAccessorKeys> ConstKeys = PCGAttributeAccessorHelpers::CreateConstKeys(PointData, FPCGAttributePropertySelector::CreateAttributeSelector(AttributeName));
	REQUIRE(ConstKeys);
	REQUIRE(Accessor->GetRange(MakeArrayView(Out), 0, *ConstKeys));

	for (int32 i = 0; i < NumPoints; ++i)
	{
		TConstArrayView<FString> View = Out[i].GetView();
		REQUIRE_EQUAL(View.Num(), Inputs[i].Num());
		for (int32 j = 0; j < Inputs[i].Num(); ++j)
		{
			REQUIRE_EQUAL(View[j], Inputs[i][j]);
		}
	}

	// Deduplication is observable through the underlying attribute: entries 0 and 2 share
	// the same value key, 1 and 3 share another, and entry 4 is unique.
	const PCGMetadataValueKey K0 = Attr->GetValueKey(MetadataEntryRange[0]);
	const PCGMetadataValueKey K1 = Attr->GetValueKey(MetadataEntryRange[1]);
	const PCGMetadataValueKey K4 = Attr->GetValueKey(MetadataEntryRange[4]);
	REQUIRE_EQUAL(Attr->GetValueKey(MetadataEntryRange[2]), K0);
	REQUIRE_EQUAL(Attr->GetValueKey(MetadataEntryRange[3]), K1);
	REQUIRE_NOT_EQUAL(K0, K1);
	REQUIRE_NOT_EQUAL(K0, K4);
	REQUIRE_NOT_EQUAL(K1, K4);
}

// Tests the public type-erased IPCGAttributeAccessor::GetRange entry point with FOutValuesByValue — covers SameType passthrough, Constructible (float->double), Broadcast (float->FVector), and confirms StrictType across types fails.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::TypeErasedGetRangeSingleValue", "[PCG][Accessor][Attribute]")
{
	UPCGBasePointData* PointData = CreatePointData();
	FPCGMetadataAttribute<float>* FloatAttribute = PointData->Metadata->CreateAttribute<float>(TEXT("FloatAttr"), 0.0f, true, false);
	TUniquePtr<IPCGAttributeAccessor> FloatAccessor = PCGAttributeAccessorHelpers::CreateAccessor(FloatAttribute, PointData->Metadata);
	REQUIRE(FloatAccessor);

	constexpr int32 NumElements = 3;
	const float SourceValues[NumElements] = { 1.5f, 2.5f, 3.5f };

	PointData->SetNumPoints(NumElements);
	TPCGValueRange<int64> EntryRange = PointData->GetMetadataEntryValueRange();
	for (int32 i = 0; i < NumElements; ++i)
	{
		EntryRange[i] = PointData->Metadata->AddEntry();
		FloatAttribute->SetValue(EntryRange[i], SourceValues[i]);
	}

	TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(PointData, FPCGAttributePropertySelector());
	REQUIRE(Keys);
	REQUIRE_EQUAL(Keys->GetNum(), NumElements);

	const TConstArrayView<float> SourceView(SourceValues, NumElements);

	SECTION("SameType (float -> float) succeeds")
	{
		const FPCGMetadataAttributeDesc& WriteDesc = PCG::Private::GetDefaultAttributeDesc<float>();
		float Out[NumElements] = { 0.0f, 0.0f, 0.0f };

		REQUIRE(FloatAccessor->GetRange(
			PCG::Private::FOutValues{TInPlaceType<PCG::Private::FOutValuesByValue>{}, Out, NumElements, &WriteDesc},
			WriteDesc, NumElements, 0, *Keys, EPCGAttributeAccessorFlags::StrictType));

		CHECK_THAT(TConstArrayView<float>(Out, NumElements), Catch::Matchers::RangeEquals(SourceView));
	}

	SECTION("Constructible (float -> double) with AllowConstructible succeeds")
	{
		const FPCGMetadataAttributeDesc& WriteDesc = PCG::Private::GetDefaultAttributeDesc<double>();
		double Out[NumElements] = { 0.0, 0.0, 0.0 };

		REQUIRE(FloatAccessor->GetRange(
			PCG::Private::FOutValues{TInPlaceType<PCG::Private::FOutValuesByValue>{}, Out, NumElements, &WriteDesc},
			WriteDesc, NumElements, 0, *Keys, EPCGAttributeAccessorFlags::AllowConstructible));

		CHECK_THAT(TConstArrayView<double>(Out, NumElements), Catch::Matchers::RangeEquals(SourceView, [](const double a, const float b) { return a == static_cast<double>(b); }));
	}

	SECTION("Broadcast (float -> FVector) with AllowBroadcast succeeds")
	{
		const FPCGMetadataAttributeDesc& WriteDesc = PCG::Private::GetDefaultAttributeDesc<FVector>();
		FVector Out[NumElements] = { FVector::ZeroVector, FVector::ZeroVector, FVector::ZeroVector };

		REQUIRE(FloatAccessor->GetRange(
			PCG::Private::FOutValues{TInPlaceType<PCG::Private::FOutValuesByValue>{}, Out, NumElements, &WriteDesc},
			WriteDesc, NumElements, 0, *Keys, EPCGAttributeAccessorFlags::AllowBroadcast));

		CHECK_THAT(TConstArrayView<FVector>(Out, NumElements), Catch::Matchers::RangeEquals(SourceView, [](const FVector& a, const float b) { return a == FVector(b); }));
	}

	SECTION("StrictType across different types fails")
	{
		const FPCGMetadataAttributeDesc& WriteDesc = PCG::Private::GetDefaultAttributeDesc<double>();
		double Out[NumElements] = { 0.0, 0.0, 0.0 };

		REQUIRE_FALSE(FloatAccessor->GetRange(
			PCG::Private::FOutValues{TInPlaceType<PCG::Private::FOutValuesByValue>{}, Out, NumElements, &WriteDesc},
			WriteDesc, NumElements, 0, *Keys, EPCGAttributeAccessorFlags::StrictType));
	}

	SECTION("Struct (FPCGPoint -> FPCGPoint) SameType")
	{
		const auto Tester = PCGAttributeTestsCommonHelper::FPCGPointTester();
		FPCGMetadataAttributeBase* PointAttribute = PointData->Metadata->CreateAttribute<FPCGPoint>(TEXT("PointAttr"), Tester.DefaultValue, false, false);
		TUniquePtr<IPCGAttributeAccessor> PointAccessor = PCGAttributeAccessorHelpers::CreateAccessor(PointAttribute, PointData->Metadata);
		REQUIRE(PointAccessor);

		FRandomStream RandomStream(42);
		FPCGPoint SamplePoints[NumElements];
		for (int32 i = 0; i < NumElements; ++i)
		{
			Tester.GenerateRandom(RandomStream, SamplePoints[i]);
			PointAttribute->SetValue<FPCGPoint>(EntryRange[i], SamplePoints[i]);
		}

		const FPCGMetadataAttributeDesc& WriteDesc = PCG::Private::GetDefaultAttributeDesc<FPCGPoint>();
		FPCGPoint Out[NumElements];

		REQUIRE(PointAccessor->GetRange(
			PCG::Private::FOutValues{TInPlaceType<PCG::Private::FOutValuesByValue>{}, Out, NumElements, &WriteDesc},
			WriteDesc, NumElements, 0, *Keys, EPCGAttributeAccessorFlags::StrictType));

		CHECK_THAT(TConstArrayView<FPCGPoint>(Out, NumElements), Catch::Matchers::RangeEquals(TConstArrayView<FPCGPoint>(SamplePoints, NumElements), [&Tester](const FPCGPoint& A, const FPCGPoint& B) { return Tester.Verify(A, &B); }));
	}

	SECTION("Complex conversion (FString -> FSoftObjectPath) with AllowConstructible")
	{
		FPCGMetadataAttribute<FString>* StringAttribute = PointData->Metadata->CreateAttribute<FString>(TEXT("StrAttr"), FString(), false, false);
		TUniquePtr<IPCGAttributeAccessor> StringAccessor = PCGAttributeAccessorHelpers::CreateAccessor(StringAttribute, PointData->Metadata);
		REQUIRE(StringAccessor);

		const FString PathStrings[NumElements] = {
			TEXT("/Game/Path/Alpha.Alpha"),
			TEXT("/Game/Path/Beta.Beta"),
			TEXT("/Game/Path/Gamma.Gamma"),
		};
		for (int32 i = 0; i < NumElements; ++i)
		{
			StringAttribute->SetValue(EntryRange[i], PathStrings[i]);
		}

		const FPCGMetadataAttributeDesc& WriteDesc = PCG::Private::GetDefaultAttributeDesc<FSoftObjectPath>();
		FSoftObjectPath Out[NumElements];

		REQUIRE(StringAccessor->GetRange(
			PCG::Private::FOutValues{TInPlaceType<PCG::Private::FOutValuesByValue>{}, Out, NumElements, &WriteDesc},
			WriteDesc, NumElements, 0, *Keys, EPCGAttributeAccessorFlags::AllowConstructible));

		CHECK_THAT(TConstArrayView<FSoftObjectPath>(Out, NumElements), Catch::Matchers::RangeEquals(TConstArrayView<FString>(PathStrings, NumElements), [](const FSoftObjectPath& a, const FString& b) { return a == FSoftObjectPath(b); }));
	}
}

// Tests the public type-erased IPCGAttributeAccessor::GetRange entry point with FOutValuesAsArray — covers SameType (no buffer allocation), Constructible (TArray<float>->TArray<double>), Broadcast (TArray<float>->TArray<FVector>), and a Buffers/OutValues count mismatch failure. Regression guard for the missing-constexpr / ConvertToSingleValue / TransformFunc arg-order fixes.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::TypeErasedGetRangeArray", "[PCG][Accessor][Attribute]")
{
	UPCGBasePointData* PointData = CreatePointData();
	static const FName ArrayAttrName = "FloatArrAttr";
	FPCGMetadataAttributeBase* ArrayAttribute = PointData->Metadata->CreateAttribute<TArray<float>>(ArrayAttrName, {}, true, false);
	TUniquePtr<IPCGAttributeAccessor> ArrayAccessor = PCGAttributeAccessorHelpers::CreateAccessor(ArrayAttribute, PointData->Metadata);
	REQUIRE(ArrayAttribute);
	REQUIRE(ArrayAccessor);

	constexpr int32 NumElements = 3;
	TArray<TArray<float>> SourceArrays;
	SourceArrays.SetNum(NumElements);
	SourceArrays[0] = { 1.0f, 2.0f, 3.0f };
	SourceArrays[1] = { 4.0f };
	SourceArrays[2] = { 5.0f, 6.0f };

	PointData->SetNumPoints(NumElements);
	TPCGValueRange<int64> EntryRange = PointData->GetMetadataEntryValueRange();
	for (int32 i = 0; i < NumElements; ++i)
	{
		EntryRange[i] = PointData->Metadata->AddEntry();
		ArrayAttribute->SetValue<TArray<float>>(EntryRange[i], SourceArrays[i]);
	}

	TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(PointData, FPCGAttributePropertySelector::CreateAttributeSelector(ArrayAttrName));
	REQUIRE(Keys);
	REQUIRE_EQUAL(Keys->GetNum(), NumElements);

	SECTION("SameType (TArray<float> -> TArray<float>) succeeds without using Buffers")
	{
		const FPCGMetadataAttributeDesc& WriteDesc = PCG::Private::GetDefaultAttributeDesc<TArray<float>>();
		TArray<TTuple<const void*, int32>> OutTuples;
		OutTuples.SetNum(NumElements);
		TArray<PCG::FPCGAccessorBuffer> OutBuffers;
		OutBuffers.SetNum(NumElements);

		REQUIRE(ArrayAccessor->GetRange(
			PCG::Private::FOutValues{TInPlaceType<PCG::Private::FOutValuesAsArray>{}, TArrayView<TTuple<const void*, int32>>(OutTuples), TArrayView<PCG::FPCGAccessorBuffer>(OutBuffers), &WriteDesc},
			WriteDesc, NumElements, 0, *Keys, EPCGAttributeAccessorFlags::StrictType));

		for (int32 i = 0; i < NumElements; ++i)
		{
			const auto [DataPtr, ArrayNum] = OutTuples[i];
			// SameType goes through GetRangeVirtual directly — buffers must remain unallocated.
			REQUIRE_FALSE(OutBuffers[i].IsOwningMemory());

			const TConstArrayView<float> OutView(static_cast<const float*>(DataPtr), ArrayNum);
			CHECK_THAT(OutView, Catch::Matchers::RangeEquals(SourceArrays[i]));
		}
	}

	SECTION("Constructible (TArray<float> -> TArray<double>) with AllowConstructible succeeds and repoints DataPtrs to Buffers")
	{
		const FPCGMetadataAttributeDesc& WriteDesc = PCG::Private::GetDefaultAttributeDesc<TArray<double>>();
		TArray<TTuple<const void*, int32>> OutTuples;
		OutTuples.SetNum(NumElements);
		TArray<PCG::FPCGAccessorBuffer> OutBuffers;
		OutBuffers.SetNum(NumElements);

		REQUIRE(ArrayAccessor->GetRange(
			PCG::Private::FOutValues{TInPlaceType<PCG::Private::FOutValuesAsArray>{}, TArrayView<TTuple<const void*, int32>>(OutTuples), TArrayView<PCG::FPCGAccessorBuffer>(OutBuffers), &WriteDesc},
			WriteDesc, NumElements, 0, *Keys, EPCGAttributeAccessorFlags::AllowConstructible));

		for (int32 i = 0; i < NumElements; ++i)
		{
			const auto [DataPtr, ArrayNum] = OutTuples[i];
			REQUIRE(OutBuffers[i].IsOwningMemory());
			REQUIRE_EQUAL(DataPtr, OutBuffers[i].GetOwnedMemoryPtr());

			const TConstArrayView<double> OutView(static_cast<const double*>(DataPtr), ArrayNum);
			CHECK_THAT(OutView, Catch::Matchers::RangeEquals(SourceArrays[i], [](const double a, const float b) { return a == static_cast<double>(b); }));
		}
	}

	SECTION("Broadcast (TArray<float> -> TArray<FVector>) with AllowBroadcast succeeds")
	{
		const FPCGMetadataAttributeDesc& WriteDesc = PCG::Private::GetDefaultAttributeDesc<TArray<FVector>>();
		TArray<TTuple<const void*, int32>> OutTuples;
		OutTuples.SetNum(NumElements);
		TArray<PCG::FPCGAccessorBuffer> OutBuffers;
		OutBuffers.SetNum(NumElements);

		REQUIRE(ArrayAccessor->GetRange(
			PCG::Private::FOutValues{TInPlaceType<PCG::Private::FOutValuesAsArray>{}, TArrayView<TTuple<const void*, int32>>(OutTuples), TArrayView<PCG::FPCGAccessorBuffer>(OutBuffers), &WriteDesc},
			WriteDesc, NumElements, 0, *Keys, EPCGAttributeAccessorFlags::AllowBroadcast));

		for (int32 i = 0; i < NumElements; ++i)
		{
			const auto [DataPtr, ArrayNum] = OutTuples[i];
			REQUIRE(OutBuffers[i].IsOwningMemory());

			const TConstArrayView<FVector> OutView(static_cast<const FVector*>(DataPtr), ArrayNum);
			CHECK_THAT(OutView, Catch::Matchers::RangeEquals(SourceArrays[i], [](const FVector& a, const float b) { return a == FVector(b); }));
		}
	}

	SECTION("Buffers count not matching OutValues count returns false")
	{
		const FPCGMetadataAttributeDesc& WriteDesc = PCG::Private::GetDefaultAttributeDesc<TArray<double>>();
		TArray<TTuple<const void*, int32>> OutTuples;
		OutTuples.SetNum(NumElements);
		TArray<PCG::FPCGAccessorBuffer> OutBuffers; // empty — mismatched count

		REQUIRE_FALSE(ArrayAccessor->GetRange(
			PCG::Private::FOutValues{TInPlaceType<PCG::Private::FOutValuesAsArray>{}, TArrayView<TTuple<const void*, int32>>(OutTuples), TArrayView<PCG::FPCGAccessorBuffer>(OutBuffers), &WriteDesc},
			WriteDesc, NumElements, 0, *Keys, EPCGAttributeAccessorFlags::AllowConstructible));
	}

	SECTION("Struct (TArray<FPCGPoint>) SameType")
	{
		const auto Tester = PCGAttributeTestsCommonHelper::ArrayFPCGPointTester();
		static const FName PointArrName = "PointArr";
		FPCGMetadataAttributeBase* PointArrayAttribute = PointData->Metadata->CreateAttribute<TArray<FPCGPoint>>(PointArrName, Tester.DefaultValue, false, false);
		TUniquePtr<IPCGAttributeAccessor> PointArrayAccessor = PCGAttributeAccessorHelpers::CreateAccessor(PointArrayAttribute, PointData->Metadata);
		REQUIRE(PointArrayAccessor);

		FRandomStream RandomStream(42);
		TArray<TArray<FPCGPoint>> PointSourceArrays;
		PointSourceArrays.SetNum(NumElements);
		for (int32 i = 0; i < NumElements; ++i)
		{
			Tester.GenerateRandom(RandomStream, PointSourceArrays[i]);
			PointArrayAttribute->SetValue<TArray<FPCGPoint>>(EntryRange[i], PointSourceArrays[i]);
		}

		TUniquePtr<const IPCGAttributeAccessorKeys> PointKeys = PCGAttributeAccessorHelpers::CreateConstKeys(PointData, FPCGAttributePropertySelector::CreateAttributeSelector(PointArrName));
		REQUIRE(PointKeys);
		REQUIRE_EQUAL(PointKeys->GetNum(), NumElements);

		const FPCGMetadataAttributeDesc& WriteDesc = PCG::Private::GetDefaultAttributeDesc<TArray<FPCGPoint>>();
		TArray<TTuple<const void*, int32>> OutTuples;
		OutTuples.SetNum(NumElements);
		TArray<PCG::FPCGAccessorBuffer> OutBuffers;
		OutBuffers.SetNum(NumElements);

		REQUIRE(PointArrayAccessor->GetRange(
			PCG::Private::FOutValues{TInPlaceType<PCG::Private::FOutValuesAsArray>{}, TArrayView<TTuple<const void*, int32>>(OutTuples), TArrayView<PCG::FPCGAccessorBuffer>(OutBuffers), &WriteDesc},
			WriteDesc, NumElements, 0, *PointKeys, EPCGAttributeAccessorFlags::StrictType));

		for (int32 i = 0; i < NumElements; ++i)
		{
			const auto [DataPtr, ArrayNum] = OutTuples[i];
			REQUIRE_FALSE(OutBuffers[i].IsOwningMemory());

			const TConstArrayView<FPCGPoint> OutView(static_cast<const FPCGPoint*>(DataPtr), ArrayNum);
			CHECK(Tester.Verify(PointSourceArrays[i], OutView));
		}
	}

	SECTION("Complex conversion (TArray<FString> -> TArray<FSoftObjectPath>) with AllowConstructible")
	{
		static const FName StringArrName = "StringArr";
		FPCGMetadataAttributeBase* StringArrayAttribute = PointData->Metadata->CreateAttribute<TArray<FString>>(StringArrName, {}, false, false);
		TUniquePtr<IPCGAttributeAccessor> StringArrayAccessor = PCGAttributeAccessorHelpers::CreateAccessor(StringArrayAttribute, PointData->Metadata);
		REQUIRE(StringArrayAccessor);

		TArray<TArray<FString>> PathSourceArrays;
		PathSourceArrays.SetNum(NumElements);
		PathSourceArrays[0] = { TEXT("/Game/A.A"), TEXT("/Game/B.B") };
		PathSourceArrays[1] = { TEXT("/Game/C.C") };
		PathSourceArrays[2] = { TEXT("/Game/D.D"), TEXT("/Game/E.E"), TEXT("/Game/F.F") };
		for (int32 i = 0; i < NumElements; ++i)
		{
			StringArrayAttribute->SetValue<TArray<FString>>(EntryRange[i], PathSourceArrays[i]);
		}

		TUniquePtr<const IPCGAttributeAccessorKeys> StringKeys = PCGAttributeAccessorHelpers::CreateConstKeys(PointData, FPCGAttributePropertySelector::CreateAttributeSelector(StringArrName));
		REQUIRE(StringKeys);
		REQUIRE_EQUAL(StringKeys->GetNum(), NumElements);

		const FPCGMetadataAttributeDesc& WriteDesc = PCG::Private::GetDefaultAttributeDesc<TArray<FSoftObjectPath>>();
		TArray<TTuple<const void*, int32>> OutTuples;
		OutTuples.SetNum(NumElements);
		TArray<PCG::FPCGAccessorBuffer> OutBuffers;
		OutBuffers.SetNum(NumElements);

		REQUIRE(StringArrayAccessor->GetRange(
			PCG::Private::FOutValues{TInPlaceType<PCG::Private::FOutValuesAsArray>{}, TArrayView<TTuple<const void*, int32>>(OutTuples), TArrayView<PCG::FPCGAccessorBuffer>(OutBuffers), &WriteDesc},
			WriteDesc, NumElements, 0, *StringKeys, EPCGAttributeAccessorFlags::AllowConstructible));

		for (int32 i = 0; i < NumElements; ++i)
		{
			const auto [DataPtr, ArrayNum] = OutTuples[i];
			REQUIRE(OutBuffers[i].IsOwningMemory());
			REQUIRE_EQUAL(DataPtr, OutBuffers[i].GetOwnedMemoryPtr());

			const TConstArrayView<FSoftObjectPath> OutView(static_cast<const FSoftObjectPath*>(DataPtr), ArrayNum);
			CHECK_THAT(OutView, Catch::Matchers::RangeEquals(PathSourceArrays[i], [](const FSoftObjectPath& a, const FString& b) { return a == FSoftObjectPath(b); }));
		}
	}
}

// Tests that the public type-erased IPCGAttributeAccessor::GetRange returns false when given FOutValuesAsSet with a non-SameType WriteDesc — exercises the explicit "Not supported" branch in the Visit lambda.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::TypeErasedGetRangeSet", "[PCG][Accessor][Attribute]")
{
	UPCGBasePointData* PointData = CreatePointData();
	FPCGMetadataAttributeBase* SetAttribute = PointData->Metadata->CreateAttribute<TSet<float>>(TEXT("SetFloatAttr"), TSet<float>{1.0f, 2.0f, 3.0f});
	TUniquePtr<IPCGAttributeAccessor> SetAccessor = PCGAttributeAccessorHelpers::CreateAccessor(SetAttribute, PointData->Metadata);
	REQUIRE(SetAccessor);

	PointData->SetNumPoints(1);
	TPCGValueRange<int64> EntryRange = PointData->GetMetadataEntryValueRange();
	EntryRange[0] = PointData->Metadata->AddEntry();

	TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(PointData, FPCGAttributePropertySelector());
	REQUIRE(Keys);

	SECTION("Non-SameType conversion (TSet<float> -> TSet<double>) returns false")
	{
		const FPCGMetadataAttributeDesc& WriteDesc = PCG::Private::GetDefaultAttributeDesc<TSet<double>>();
		// Empty view — the unsupported branch returns before dereferencing.
		TArrayView<FScriptSetHelper*> EmptyHelpers;

		REQUIRE_FALSE(SetAccessor->GetRange(
			PCG::Private::FOutValues{TInPlaceType<PCG::Private::FOutValuesAsSet>{}, EmptyHelpers, &WriteDesc},
			WriteDesc, 1, 0, *Keys, EPCGAttributeAccessorFlags::AllowConstructible));
	}
}

// Tests that the public type-erased IPCGAttributeAccessor::GetRange returns false when given FOutValuesAsMap with a non-SameType WriteDesc — exercises the explicit "Not supported" branch in the Visit lambda.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::TypeErasedGetRangeMap", "[PCG][Accessor][Attribute]")
{
	UPCGBasePointData* PointData = CreatePointData();
	FPCGMetadataAttributeBase* MapAttribute = PointData->Metadata->CreateAttribute<TMap<FString, int32>>(TEXT("MapAttr"), TMap<FString, int32>{});
	TUniquePtr<IPCGAttributeAccessor> MapAccessor = PCGAttributeAccessorHelpers::CreateAccessor(MapAttribute, PointData->Metadata);
	REQUIRE(MapAccessor);

	PointData->SetNumPoints(1);
	TPCGValueRange<int64> EntryRange = PointData->GetMetadataEntryValueRange();
	EntryRange[0] = PointData->Metadata->AddEntry();

	TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(PointData, FPCGAttributePropertySelector());
	REQUIRE(Keys);

	SECTION("Non-SameType conversion (TMap<FString,int32> -> TMap<FString,double>) returns false")
	{
		const FPCGMetadataAttributeDesc& WriteDesc = PCG::Private::GetDefaultAttributeDesc<TMap<FString, double>>();
		TArrayView<FScriptMapHelper*> EmptyHelpers;

		REQUIRE_FALSE(MapAccessor->GetRange(
			PCG::Private::FOutValues{TInPlaceType<PCG::Private::FOutValuesAsMap>{}, EmptyHelpers, &WriteDesc},
			WriteDesc, 1, 0, *Keys, EPCGAttributeAccessorFlags::AllowConstructible));
	}
}

// Tests the public type-erased IPCGAttributeAccessor::SetRange entry point with FInValuesByValue — covers SameType, Constructible (float -> double), and Broadcast (float -> FVector) writes.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::TypeErasedSetRangeSingleValue", "[PCG][Accessor][Attribute]")
{
	UPCGBasePointData* PointData = CreatePointData();

	constexpr int32 NumElements = 3;
	const float SourceValues[NumElements] = { 1.5f, 2.5f, 3.5f };

	PointData->SetNumPoints(NumElements);
	TPCGValueRange<int64> EntryRange = PointData->GetMetadataEntryValueRange();
	for (int32 i = 0; i < NumElements; ++i)
	{
		EntryRange[i] = PointData->Metadata->AddEntry();
	}

	TUniquePtr<IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateKeys(PointData, FPCGAttributePropertySelector());
	REQUIRE(Keys);
	REQUIRE_EQUAL(Keys->GetNum(), NumElements);

	SECTION("SameType (write float into float attribute)")
	{
		FPCGMetadataAttribute<float>* FloatAttribute = PointData->Metadata->CreateAttribute<float>(TEXT("FloatAttr"), 0.0f, false, false);
		TUniquePtr<IPCGAttributeAccessor> FloatAccessor = PCGAttributeAccessorHelpers::CreateAccessor(FloatAttribute, PointData->Metadata);
		REQUIRE(FloatAccessor);

		const FPCGMetadataAttributeDesc& ReadDesc = PCG::Private::GetDefaultAttributeDesc<float>();

		REQUIRE(FloatAccessor->SetRange(
			PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesByValue>{}, SourceValues, NumElements, &ReadDesc},
			ReadDesc, NumElements, 0, *Keys, EPCGAttributeAccessorFlags::StrictType));

		for (int32 i = 0; i < NumElements; ++i)
		{
			REQUIRE_EQUAL(FloatAttribute->GetValueFromItemKey(EntryRange[i]), SourceValues[i]);
		}
	}

	SECTION("Constructible (write float into double attribute) with AllowConstructible")
	{
		FPCGMetadataAttribute<double>* DoubleAttribute = PointData->Metadata->CreateAttribute<double>(TEXT("DoubleAttr"), 0.0, false, false);
		TUniquePtr<IPCGAttributeAccessor> DoubleAccessor = PCGAttributeAccessorHelpers::CreateAccessor(DoubleAttribute, PointData->Metadata);
		REQUIRE(DoubleAccessor);

		const FPCGMetadataAttributeDesc& ReadDesc = PCG::Private::GetDefaultAttributeDesc<float>();

		REQUIRE(DoubleAccessor->SetRange(
			PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesByValue>{}, SourceValues, NumElements, &ReadDesc},
			ReadDesc, NumElements, 0, *Keys, EPCGAttributeAccessorFlags::AllowConstructible));

		for (int32 i = 0; i < NumElements; ++i)
		{
			REQUIRE_EQUAL(DoubleAttribute->GetValueFromItemKey(EntryRange[i]), static_cast<double>(SourceValues[i]));
		}
	}

	SECTION("Broadcast (write float into FVector attribute) with AllowBroadcast")
	{
		FPCGMetadataAttribute<FVector>* VectorAttribute = PointData->Metadata->CreateAttribute<FVector>(TEXT("VectorAttr"), FVector::ZeroVector, false, false);
		TUniquePtr<IPCGAttributeAccessor> VectorAccessor = PCGAttributeAccessorHelpers::CreateAccessor(VectorAttribute, PointData->Metadata);
		REQUIRE(VectorAccessor);

		const FPCGMetadataAttributeDesc& ReadDesc = PCG::Private::GetDefaultAttributeDesc<float>();

		REQUIRE(VectorAccessor->SetRange(
			PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesByValue>{}, SourceValues, NumElements, &ReadDesc},
			ReadDesc, NumElements, 0, *Keys, EPCGAttributeAccessorFlags::AllowBroadcast));

		for (int32 i = 0; i < NumElements; ++i)
		{
			REQUIRE_EQUAL(VectorAttribute->GetValueFromItemKey(EntryRange[i]), FVector(SourceValues[i]));
		}
	}

	SECTION("Struct (write FPCGPoint into FPCGPoint attribute) SameType")
	{
		const auto Tester = PCGAttributeTestsCommonHelper::FPCGPointTester();
		FPCGMetadataAttributeBase* PointAttribute = PointData->Metadata->CreateAttribute<FPCGPoint>(TEXT("PointAttr"), Tester.DefaultValue, false, false);
		TUniquePtr<IPCGAttributeAccessor> PointAccessor = PCGAttributeAccessorHelpers::CreateAccessor(PointAttribute, PointData->Metadata);
		REQUIRE(PointAccessor);

		FRandomStream RandomStream(42);
		FPCGPoint SamplePoints[NumElements];
		for (int32 i = 0; i < NumElements; ++i)
		{
			Tester.GenerateRandom(RandomStream, SamplePoints[i]);
		}
		const FPCGMetadataAttributeDesc& ReadDesc = PCG::Private::GetDefaultAttributeDesc<FPCGPoint>();

		REQUIRE(PointAccessor->SetRange(
			PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesByValue>{}, SamplePoints, NumElements, &ReadDesc},
			ReadDesc, NumElements, 0, *Keys, EPCGAttributeAccessorFlags::StrictType));

		for (int32 i = 0; i < NumElements; ++i)
		{
			const FPCGPoint Stored = PointAttribute->GetValueFromItemKey<FPCGPoint>(EntryRange[i]);
			CHECK(Tester.Verify(SamplePoints[i], &Stored));
		}
	}

	SECTION("Complex conversion (write FString into FSoftObjectPath attribute) with AllowConstructible")
	{
		FPCGMetadataAttribute<FSoftObjectPath>* SoftPathAttribute = PointData->Metadata->CreateAttribute<FSoftObjectPath>(TEXT("SoftPathAttr"), FSoftObjectPath(), false, false);
		TUniquePtr<IPCGAttributeAccessor> SoftPathAccessor = PCGAttributeAccessorHelpers::CreateAccessor(SoftPathAttribute, PointData->Metadata);
		REQUIRE(SoftPathAccessor);

		const FString PathStrings[NumElements] = {
			TEXT("/Game/Path/Alpha.Alpha"),
			TEXT("/Game/Path/Beta.Beta"),
			TEXT("/Game/Path/Gamma.Gamma"),
		};
		const FPCGMetadataAttributeDesc& ReadDesc = PCG::Private::GetDefaultAttributeDesc<FString>();

		REQUIRE(SoftPathAccessor->SetRange(
			PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesByValue>{}, PathStrings, NumElements, &ReadDesc},
			ReadDesc, NumElements, 0, *Keys, EPCGAttributeAccessorFlags::AllowConstructible));

		for (int32 i = 0; i < NumElements; ++i)
		{
			REQUIRE_EQUAL(SoftPathAttribute->GetValueFromItemKey(EntryRange[i]), FSoftObjectPath(PathStrings[i]));
		}
	}
}

// Tests the public type-erased IPCGAttributeAccessor::SetRange entry point with FInValuesAsArray — covers SameType, Constructible (TArray<float> -> TArray<double> inner), and Broadcast (TArray<float> -> TArray<FVector> inner) writes.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::TypeErasedSetRangeArray", "[PCG][Accessor][Attribute]")
{
	UPCGBasePointData* PointData = CreatePointData();

	constexpr int32 NumElements = 3;
	TArray<TArray<float>> SourceArrays;
	SourceArrays.SetNum(NumElements);
	SourceArrays[0] = { 1.0f, 2.0f, 3.0f };
	SourceArrays[1] = { 4.0f };
	SourceArrays[2] = { 5.0f, 6.0f };

	PointData->SetNumPoints(NumElements);
	TPCGValueRange<int64> EntryRange = PointData->GetMetadataEntryValueRange();
	for (int32 i = 0; i < NumElements; ++i)
	{
		EntryRange[i] = PointData->Metadata->AddEntry();
	}

	// Build the per-entry tuples shared across sections.
	TArray<TTuple<const void*, int32>> InTuples;
	InTuples.SetNum(NumElements);
	for (int32 i = 0; i < NumElements; ++i)
	{
		InTuples[i] = { SourceArrays[i].GetData(), SourceArrays[i].Num() };
	}

	TUniquePtr<IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateKeys(PointData, FPCGAttributePropertySelector());
	REQUIRE(Keys);
	REQUIRE_EQUAL(Keys->GetNum(), NumElements);

	SECTION("SameType (TArray<float> -> TArray<float>)")
	{
		static const FName AttrName = "FloatArr";
		FPCGMetadataAttributeBase* FloatArrayAttribute = PointData->Metadata->CreateAttribute<TArray<float>>(AttrName, {}, false, false);
		TUniquePtr<IPCGAttributeAccessor> FloatArrayAccessor = PCGAttributeAccessorHelpers::CreateAccessor(FloatArrayAttribute, PointData->Metadata);
		REQUIRE(FloatArrayAccessor);

		const FPCGMetadataAttributeDesc& ReadDesc = PCG::Private::GetDefaultAttributeDesc<TArray<float>>();

		REQUIRE(FloatArrayAccessor->SetRange(
			PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesAsArray>{}, TArrayView<TTuple<const void*, int32>>(InTuples), &ReadDesc},
			ReadDesc, NumElements, 0, *Keys, EPCGAttributeAccessorFlags::StrictType));

		for (int32 i = 0; i < NumElements; ++i)
		{
			TArray<float> Stored = FloatArrayAttribute->GetValueFromItemKey<TArray<float>>(EntryRange[i]);
			CHECK_THAT(Stored, Catch::Matchers::RangeEquals(SourceArrays[i]));
		}
	}

	SECTION("Constructible (TArray<float> -> TArray<double>) with AllowConstructible")
	{
		static const FName AttrName = "DoubleArr";
		FPCGMetadataAttributeBase* DoubleArrayAttribute = PointData->Metadata->CreateAttribute<TArray<double>>(AttrName, {}, false, false);
		TUniquePtr<IPCGAttributeAccessor> DoubleArrayAccessor = PCGAttributeAccessorHelpers::CreateAccessor(DoubleArrayAttribute, PointData->Metadata);
		REQUIRE(DoubleArrayAccessor);

		const FPCGMetadataAttributeDesc& ReadDesc = PCG::Private::GetDefaultAttributeDesc<TArray<float>>();

		REQUIRE(DoubleArrayAccessor->SetRange(
			PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesAsArray>{}, TArrayView<TTuple<const void*, int32>>(InTuples), &ReadDesc},
			ReadDesc, NumElements, 0, *Keys, EPCGAttributeAccessorFlags::AllowConstructible));

		for (int32 i = 0; i < NumElements; ++i)
		{
			TArray<double> Stored = DoubleArrayAttribute->GetValueFromItemKey<TArray<double>>(EntryRange[i]);
			CHECK_THAT(Stored, Catch::Matchers::RangeEquals(SourceArrays[i], [](const double a, const float b) { return a == static_cast<double>(b); }));
		}
	}

	SECTION("Broadcast (TArray<float> -> TArray<FVector>) with AllowBroadcast")
	{
		static const FName AttrName = "VectorArr";
		FPCGMetadataAttributeBase* VectorArrayAttribute = PointData->Metadata->CreateAttribute<TArray<FVector>>(AttrName, {}, false, false);
		TUniquePtr<IPCGAttributeAccessor> VectorArrayAccessor = PCGAttributeAccessorHelpers::CreateAccessor(VectorArrayAttribute, PointData->Metadata);
		REQUIRE(VectorArrayAccessor);

		const FPCGMetadataAttributeDesc& ReadDesc = PCG::Private::GetDefaultAttributeDesc<TArray<float>>();

		REQUIRE(VectorArrayAccessor->SetRange(
			PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesAsArray>{}, TArrayView<TTuple<const void*, int32>>(InTuples), &ReadDesc},
			ReadDesc, NumElements, 0, *Keys, EPCGAttributeAccessorFlags::AllowBroadcast));

		for (int32 i = 0; i < NumElements; ++i)
		{
			TArray<FVector> Stored = VectorArrayAttribute->GetValueFromItemKey<TArray<FVector>>(EntryRange[i]);
			CHECK_THAT(Stored, Catch::Matchers::RangeEquals(SourceArrays[i], [](const FVector& a, const float b) { return a == FVector(b); }));
		}
	}

	SECTION("Struct (TArray<FPCGPoint>) SameType")
	{
		const auto Tester = PCGAttributeTestsCommonHelper::ArrayFPCGPointTester();
		static const FName AttrName = "PointArr";
		FPCGMetadataAttributeBase* PointArrayAttribute = PointData->Metadata->CreateAttribute<TArray<FPCGPoint>>(AttrName, Tester.DefaultValue, false, false);
		TUniquePtr<IPCGAttributeAccessor> PointArrayAccessor = PCGAttributeAccessorHelpers::CreateAccessor(PointArrayAttribute, PointData->Metadata);
		REQUIRE(PointArrayAccessor);

		FRandomStream RandomStream(42);
		TArray<TArray<FPCGPoint>> PointSourceArrays;
		PointSourceArrays.SetNum(NumElements);
		for (int32 i = 0; i < NumElements; ++i)
		{
			Tester.GenerateRandom(RandomStream, PointSourceArrays[i]);
		}

		TArray<TTuple<const void*, int32>> PointInTuples;
		PointInTuples.SetNum(NumElements);
		for (int32 i = 0; i < NumElements; ++i)
		{
			PointInTuples[i] = { PointSourceArrays[i].GetData(), PointSourceArrays[i].Num() };
		}

		const FPCGMetadataAttributeDesc& ReadDesc = PCG::Private::GetDefaultAttributeDesc<TArray<FPCGPoint>>();

		REQUIRE(PointArrayAccessor->SetRange(
			PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesAsArray>{}, TArrayView<TTuple<const void*, int32>>(PointInTuples), &ReadDesc},
			ReadDesc, NumElements, 0, *Keys, EPCGAttributeAccessorFlags::StrictType));

		for (int32 i = 0; i < NumElements; ++i)
		{
			TArray<FPCGPoint> Stored = PointArrayAttribute->GetValueFromItemKey<TArray<FPCGPoint>>(EntryRange[i]);
			CHECK(Tester.Verify(PointSourceArrays[i], Stored));
		}
	}

	SECTION("Complex conversion (TArray<FString> -> TArray<FSoftObjectPath>) with AllowConstructible")
	{
		static const FName AttrName = "SoftPathArr";
		FPCGMetadataAttributeBase* SoftPathArrayAttribute = PointData->Metadata->CreateAttribute<TArray<FSoftObjectPath>>(AttrName, {}, false, false);
		TUniquePtr<IPCGAttributeAccessor> SoftPathArrayAccessor = PCGAttributeAccessorHelpers::CreateAccessor(SoftPathArrayAttribute, PointData->Metadata);
		REQUIRE(SoftPathArrayAccessor);

		TArray<TArray<FString>> PathSourceArrays;
		PathSourceArrays.SetNum(NumElements);
		PathSourceArrays[0] = { TEXT("/Game/A.A"), TEXT("/Game/B.B") };
		PathSourceArrays[1] = { TEXT("/Game/C.C") };
		PathSourceArrays[2] = { TEXT("/Game/D.D"), TEXT("/Game/E.E"), TEXT("/Game/F.F") };

		TArray<TTuple<const void*, int32>> StringInTuples;
		StringInTuples.SetNum(NumElements);
		for (int32 i = 0; i < NumElements; ++i)
		{
			StringInTuples[i] = { PathSourceArrays[i].GetData(), PathSourceArrays[i].Num() };
		}

		const FPCGMetadataAttributeDesc& ReadDesc = PCG::Private::GetDefaultAttributeDesc<TArray<FString>>();

		REQUIRE(SoftPathArrayAccessor->SetRange(
			PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesAsArray>{}, TArrayView<TTuple<const void*, int32>>(StringInTuples), &ReadDesc},
			ReadDesc, NumElements, 0, *Keys, EPCGAttributeAccessorFlags::AllowConstructible));

		for (int32 i = 0; i < NumElements; ++i)
		{
			TArray<FSoftObjectPath> Stored = SoftPathArrayAttribute->GetValueFromItemKey<TArray<FSoftObjectPath>>(EntryRange[i]);
			CHECK_THAT(Stored, Catch::Matchers::RangeEquals(PathSourceArrays[i], [](const FSoftObjectPath& a, const FString& b) { return a == FSoftObjectPath(b); }));
		}
	}
}

// Tests the public type-erased IPCGAttributeAccessor::SetRange entry point with FInValuesAsSet — only SameType is supported by the new code; verifies that path round-trips through the attribute storage.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::TypeErasedSetRangeSet", "[PCG][Accessor][Attribute]")
{
	UPCGBasePointData* PointData = CreatePointData();
	FPCGMetadataAttributeBase* SetAttribute = PointData->Metadata->CreateAttribute<TSet<int32>>(TEXT("SetIntAttr"), TSet<int32>{});
	TUniquePtr<IPCGAttributeAccessor> SetAccessor = PCGAttributeAccessorHelpers::CreateAccessor(SetAttribute, PointData->Metadata);
	REQUIRE(SetAttribute);
	REQUIRE(SetAccessor);

	PointData->SetNumPoints(1);
	TPCGValueRange<int64> EntryRange = PointData->GetMetadataEntryValueRange();
	EntryRange[0] = PointData->Metadata->AddEntry();

	TUniquePtr<IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateKeys(PointData, FPCGAttributePropertySelector());
	REQUIRE(Keys);
	REQUIRE_EQUAL(Keys->GetNum(), 1);

	const TArray<int32> SourceValues = { 7, 11, 13, 17, 19 };
	TArray<const void*> ElementPtrs;
	ElementPtrs.Reserve(SourceValues.Num());
	for (const int32& V : SourceValues)
	{
		ElementPtrs.Add(&V);
	}
	TArray<TArray<const void*>> SetView;
	SetView.Add(MoveTemp(ElementPtrs));

	const FPCGMetadataAttributeDesc& ReadDesc = PCG::Private::GetDefaultAttributeDesc<TSet<int32>>();

	REQUIRE(SetAccessor->SetRange(
		PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesAsSet>{}, TArrayView<TArray<const void*>>(SetView), &ReadDesc},
		ReadDesc, 1, 0, *Keys, EPCGAttributeAccessorFlags::StrictType));

	const TSet<int32> Stored = SetAttribute->GetValueFromItemKey<TSet<int32>>(EntryRange[0]);
	REQUIRE_EQUAL(Stored.Num(), SourceValues.Num());
	for (int32 V : SourceValues)
	{
		REQUIRE(Stored.Contains(V));
	}
}

// Tests the public type-erased IPCGAttributeAccessor::SetRange entry point with FInValuesAsMap — only SameType is supported by the new code; verifies that path round-trips through the attribute storage.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::TypeErasedSetRangeMap", "[PCG][Accessor][Attribute]")
{
	UPCGBasePointData* PointData = CreatePointData();
	FPCGMetadataAttributeBase* MapAttribute = PointData->Metadata->CreateAttribute<TMap<FString, int32>>(TEXT("MapAttr"), TMap<FString, int32>{});
	TUniquePtr<IPCGAttributeAccessor> MapAccessor = PCGAttributeAccessorHelpers::CreateAccessor(MapAttribute, PointData->Metadata);
	REQUIRE(MapAttribute);
	REQUIRE(MapAccessor);

	PointData->SetNumPoints(1);
	TPCGValueRange<int64> EntryRange = PointData->GetMetadataEntryValueRange();
	EntryRange[0] = PointData->Metadata->AddEntry();

	TUniquePtr<IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateKeys(PointData, FPCGAttributePropertySelector());
	REQUIRE(Keys);
	REQUIRE_EQUAL(Keys->GetNum(), 1);

	// Source pairs — keys and values are owned by these arrays.
	const TArray<TPair<FString, int32>> SourcePairs = {
		{ FString(TEXT("alpha")), 1 },
		{ FString(TEXT("beta")), 2 },
		{ FString(TEXT("gamma")), 3 },
	};
	TArray<TPair<const void*, const void*>> Pairs;
	Pairs.Reserve(SourcePairs.Num());
	for (const auto& KV : SourcePairs)
	{
		Pairs.Add({ &KV.Key, &KV.Value });
	}
	TArray<TArray<TPair<const void*, const void*>>> MapView;
	MapView.Add(MoveTemp(Pairs));

	const FPCGMetadataAttributeDesc& ReadDesc = PCG::Private::GetDefaultAttributeDesc<TMap<FString, int32>>();

	REQUIRE(MapAccessor->SetRange(
		PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesAsMap>{}, TArrayView<TArray<TPair<const void*, const void*>>>(MapView), &ReadDesc},
		ReadDesc, 1, 0, *Keys, EPCGAttributeAccessorFlags::StrictType));

	const TMap<FString, int32> Stored = MapAttribute->GetValueFromItemKey<TMap<FString, int32>>(EntryRange[0]);
	REQUIRE_EQUAL(Stored.Num(), SourcePairs.Num());
	for (const auto& KV : SourcePairs)
	{
		const int32* Found = Stored.Find(KV.Key);
		REQUIRE(Found != nullptr);
		REQUIRE_EQUAL(*Found, KV.Value);
	}
}

// Tests the public type-erased IPCGAttributeAccessor::SetRange's AllowRangeOfValuesIntoSingleContainer special case — scalars into Array (SameType + Constructible) and into Set with int32/FVector elements (regression guard for the inner-element-size pointer arithmetic, which previously used FArrayProperty::GetElementSize and produced sizeof(FScriptArray)=16). Also verifies the explicit no-fall-through guard for non-SameType set conversion.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Accessor::Attribute::TypeErasedSetRangeIntoSingleContainer", "[PCG][Accessor][Attribute]")
{
	UPCGBasePointData* PointData = CreatePointData();
	PointData->SetNumPoints(1);
	TPCGValueRange<int64> EntryRange = PointData->GetMetadataEntryValueRange();
	EntryRange[0] = PointData->Metadata->AddEntry();

	TUniquePtr<IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateKeys(PointData, FPCGAttributePropertySelector());
	REQUIRE(Keys);
	REQUIRE_EQUAL(Keys->GetNum(), 1);

	SECTION("Scalars into TArray<float> SameType")
	{
		FPCGMetadataAttributeBase* ArrayAttribute = PointData->Metadata->CreateAttribute<TArray<float>>(TEXT("FloatArr"), TArray<float>{});
		TUniquePtr<IPCGAttributeAccessor> ArrayAccessor = PCGAttributeAccessorHelpers::CreateAccessor(ArrayAttribute, PointData->Metadata);
		REQUIRE(ArrayAccessor);

		const TArray<float> InValues = { 1.0f, 2.0f, 3.0f, 4.0f };
		const FPCGMetadataAttributeDesc& ReadDesc = PCG::Private::GetDefaultAttributeDesc<float>();

		REQUIRE(ArrayAccessor->SetRange(
			PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesByValue>{}, InValues.GetData(), InValues.Num(), &ReadDesc},
			ReadDesc, InValues.Num(), 0, *Keys, EPCGAttributeAccessorFlags::AllowRangeOfValuesIntoSingleContainer));

		const TArray<float> Stored = ArrayAttribute->GetValueFromItemKey<TArray<float>>(EntryRange[0]);
		CHECK_THAT(Stored, Catch::Matchers::RangeEquals(InValues));
	}

	SECTION("Scalars into TArray<double> Constructible")
	{
		FPCGMetadataAttributeBase* ArrayAttribute = PointData->Metadata->CreateAttribute<TArray<double>>(TEXT("DoubleArr"), TArray<double>{});
		TUniquePtr<IPCGAttributeAccessor> ArrayAccessor = PCGAttributeAccessorHelpers::CreateAccessor(ArrayAttribute, PointData->Metadata);
		REQUIRE(ArrayAccessor);

		const TArray<float> InValues = { 1.5f, 2.5f, 3.5f };
		const FPCGMetadataAttributeDesc& ReadDesc = PCG::Private::GetDefaultAttributeDesc<float>();

		REQUIRE(ArrayAccessor->SetRange(
			PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesByValue>{}, InValues.GetData(), InValues.Num(), &ReadDesc},
			ReadDesc, InValues.Num(), 0, *Keys,
			EPCGAttributeAccessorFlags::AllowRangeOfValuesIntoSingleContainer | EPCGAttributeAccessorFlags::AllowConstructible));

		const TArray<double> Stored = ArrayAttribute->GetValueFromItemKey<TArray<double>>(EntryRange[0]);
		CHECK_THAT(Stored, Catch::Matchers::RangeEquals(InValues, [](const double a, const float b) { return a == static_cast<double>(b); }));
	}

	SECTION("Scalars into TSet<int32> SameType (4-byte element)")
	{
		FPCGMetadataAttributeBase* SetAttribute = PointData->Metadata->CreateAttribute<TSet<int32>>(TEXT("SetIntAttr"), TSet<int32>{});
		TUniquePtr<IPCGAttributeAccessor> SetAccessor = PCGAttributeAccessorHelpers::CreateAccessor(SetAttribute, PointData->Metadata);
		REQUIRE(SetAccessor);

		const TArray<int32> InValues = { 7, 11, 13, 17, 19 };
		const FPCGMetadataAttributeDesc& ReadDesc = PCG::Private::GetDefaultAttributeDesc<int32>();

		REQUIRE(SetAccessor->SetRange(
			PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesByValue>{}, InValues.GetData(), InValues.Num(), &ReadDesc},
			ReadDesc, InValues.Num(), 0, *Keys, EPCGAttributeAccessorFlags::AllowRangeOfValuesIntoSingleContainer));

		const TSet<int32> Stored = SetAttribute->GetValueFromItemKey<TSet<int32>>(EntryRange[0]);
		REQUIRE_EQUAL(Stored.Num(), InValues.Num());
		for (int32 V : InValues)
		{
			REQUIRE(Stored.Contains(V));
		}
	}

	SECTION("Scalars into TSet<FVector> SameType (24-byte element)")
	{
		FPCGMetadataAttributeBase* SetAttribute = PointData->Metadata->CreateAttribute<TSet<FVector>>(TEXT("SetVecAttr"), TSet<FVector>{});
		TUniquePtr<IPCGAttributeAccessor> SetAccessor = PCGAttributeAccessorHelpers::CreateAccessor(SetAttribute, PointData->Metadata);
		REQUIRE(SetAccessor);

		const TArray<FVector> InValues = {
			FVector(1.0, 2.0, 3.0),
			FVector(4.0, 5.0, 6.0),
			FVector(7.0, 8.0, 9.0),
		};
		const FPCGMetadataAttributeDesc& ReadDesc = PCG::Private::GetDefaultAttributeDesc<FVector>();

		REQUIRE(SetAccessor->SetRange(
			PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesByValue>{}, InValues.GetData(), InValues.Num(), &ReadDesc},
			ReadDesc, InValues.Num(), 0, *Keys, EPCGAttributeAccessorFlags::AllowRangeOfValuesIntoSingleContainer));

		const TSet<FVector> Stored = SetAttribute->GetValueFromItemKey<TSet<FVector>>(EntryRange[0]);
		REQUIRE_EQUAL(Stored.Num(), InValues.Num());
		for (const FVector& V : InValues)
		{
			REQUIRE(Stored.Contains(V));
		}
	}

	SECTION("Scalars into TSet<double> non-SameType returns false (no fall-through)")
	{
		// Source float values, destination TSet<double> — the special case only supports SameType
		// for sets, so the explicit `return false;` guard at the end of the special-case block
		// must catch this path instead of falling through with mismatched OpType / ReadDesc.
		FPCGMetadataAttributeBase* SetAttribute = PointData->Metadata->CreateAttribute<TSet<double>>(TEXT("SetDoubleAttr"), TSet<double>{});
		TUniquePtr<IPCGAttributeAccessor> SetAccessor = PCGAttributeAccessorHelpers::CreateAccessor(SetAttribute, PointData->Metadata);
		REQUIRE(SetAccessor);

		const TArray<float> InValues = { 1.0f, 2.0f, 3.0f };
		const FPCGMetadataAttributeDesc& ReadDesc = PCG::Private::GetDefaultAttributeDesc<float>();

		REQUIRE_FALSE(SetAccessor->SetRange(
			PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesByValue>{}, InValues.GetData(), InValues.Num(), &ReadDesc},
			ReadDesc, InValues.Num(), 0, *Keys,
			EPCGAttributeAccessorFlags::AllowRangeOfValuesIntoSingleContainer | EPCGAttributeAccessorFlags::AllowConstructible));
	}
}
