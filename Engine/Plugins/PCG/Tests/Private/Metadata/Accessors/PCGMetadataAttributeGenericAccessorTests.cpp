// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Tests for FPCGAttributeGenericAccessor -- the type-erased accessor for generic attributes.
 * These accessors bypass the CRTP typed path and instead use the virtual FInValues/FOutValues
 * variant dispatch to read/write data of any type supported by the attribute system.
 *
 * Test cases:
 *   Basic          - Round-trip write/read for all supported types (scalars, containers, structs, enums).
 *   NotExactType   - Type broadcast and construction: writing int32 into a double attribute,
 *                    then reading back as int32 and FString.
 *   CompressedData - Value deduplication for compressed attributes (e.g. FString), verifying that
 *                    repeated values share storage and new values are correctly appended.
 */

#include "CoreMinimal.h"
#include "TestHarness.h"

#include "PCGTestsCommon.h"
#include "Metadata/PCGMetadataAttributeTestsCommonHelper.h"

#include <catch2/generators/catch_generators.hpp>

#include "Data/PCGPointArrayData.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"


/**
 * Validates that FPCGAttributeGenericAccessor can write and read back values for every supported type.
 * Each type is tested via a TypedAttributeTester that generates random values, writes them through
 * SetRange, and reads them back through GetRange, comparing the results. The write is split into two
 * halves to exercise offset-based range access. Types covered: Float, Double, Vector (basic and struct
 * representations), String, FPCGPoint (struct), Array<double>, Array<FPCGPoint>, Set<double>,
 * Map<FString, double>, and Enum.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Metadata::Accessors::GenericAttribute::Basic", "[PCG][AttributeGeneric]")
{
	constexpr int32 Seed = 42;
	
	auto Tester = [this]<typename T, typename ReadType>(PCGAttributeTestsCommonHelper::TypedAttributeTester<T, ReadType> AttributeTester)
	{
		// Preparing the data
		constexpr int32 NumPoints = 100;
		constexpr int32 HalfNumPoints = NumPoints / 2;
		UPCGPointArrayData* PointArrayData = CreateData<UPCGPointArrayData>();

		PointArrayData->SetNumPoints(NumPoints);

		PointArrayData->AllocateProperties(EPCGPointNativeProperties::MetadataEntry);
		TPCGValueRange<int64> PointArrayDataMetadataEntryRange = PointArrayData->GetMetadataEntryValueRange(/*bAllocate=*/false);
		
		FPCGMetadataDomain* Domain = PointArrayData->MutableMetadata()->GetDefaultMetadataDomain();
		
		FRandomStream RandomStream(Seed);
		
		FPCGMetadataAttributeBase* GenericAttribute = nullptr;
	
		if (AttributeTester.OverrideDesc.IsSet())
		{
			GenericAttribute = Domain->CreateAttribute(*AttributeTester.OverrideDesc);
			REQUIRE_NOT_EQUAL(GenericAttribute, nullptr);
			GenericAttribute->SetDefaultValue<T>(AttributeTester.DefaultValue);
		}
		else
		{
			GenericAttribute = Domain->CreateAttribute<T>(PCGAttributeTestsCommonHelper::AttributeName, AttributeTester.DefaultValue);
			REQUIRE_NOT_EQUAL(GenericAttribute, nullptr);
		}
		
		SECTION("Expected attribute descriptor")
		{
			REQUIRE_EQUAL(GenericAttribute->GetAttributeDesc(), AttributeTester.ExpectedDesc);
		}
		
		TUniquePtr<IPCGAttributeAccessor> AttributeAccessor = PCGAttributeAccessorHelpers::CreateAccessor(PointArrayData, FPCGAttributePropertySelector::CreateAttributeSelector(PCGAttributeTestsCommonHelper::AttributeName));
		TUniquePtr<IPCGAttributeAccessorKeys> AttributeKeys = PCGAttributeAccessorHelpers::CreateKeys(PointArrayData, FPCGAttributePropertySelector::CreateAttributeSelector(PCGAttributeTestsCommonHelper::AttributeName));

		REQUIRE(AttributeAccessor.IsValid());
		REQUIRE(AttributeKeys.IsValid());
		
		TStaticArray<T, NumPoints> Values{};
		TStaticArray<ReadType, NumPoints> ReadValues;
		
		for (int32 i = 0; i < NumPoints; ++i)
		{
			AttributeTester.GenerateRandom(RandomStream, Values[i]);
		}

		// Do it twice, on half the points
		REQUIRE(AttributeAccessor->SetRange<T>(MakeArrayView(Values.GetData(), HalfNumPoints), 0, *AttributeKeys));
		REQUIRE_EQUAL(GenericAttribute->GetNumberOfEntries(), HalfNumPoints);

		REQUIRE(AttributeAccessor->SetRange<T>(MakeArrayView(Values.GetData() + HalfNumPoints, HalfNumPoints), HalfNumPoints, *AttributeKeys));
		REQUIRE_EQUAL(GenericAttribute->GetNumberOfEntries(), NumPoints);
		
		SECTION("Read back and verify values")
		{
			// Do it twice, on half the points
			REQUIRE(AttributeAccessor->GetRange<ReadType>(MakeArrayView(ReadValues.GetData(), HalfNumPoints), 0, *AttributeKeys));
			REQUIRE(AttributeAccessor->GetRange<ReadType>(MakeArrayView(ReadValues.GetData() + HalfNumPoints, HalfNumPoints), HalfNumPoints, *AttributeKeys));

			CHECK_THAT(Values, Catch::Matchers::RangeEquals(ReadValues, [&AttributeTester](const auto& a, const auto& b){ return AttributeTester.Verify(a, b); } ));
		}
	};
	
	SECTION("Float Attribute")
	{
		Tester(PCGAttributeTestsCommonHelper::FloatTester());
	}
	
	SECTION("Double Attribute")
	{
		Tester(PCGAttributeTestsCommonHelper::DoubleTester());
	}
	
	SECTION("Vector Attribute")
	{
		SECTION("As Basic Type")
		{
			Tester(PCGAttributeTestsCommonHelper::VectorTester_Basic());
		}
		
		SECTION("As Struct")
		{
			Tester(PCGAttributeTestsCommonHelper::VectorTester_Struct());
		}
	}
	
	SECTION("String Attribute")
	{
		Tester(PCGAttributeTestsCommonHelper::StringTester());
	}
	
	SECTION("FPCGPoint Attribute")
	{		
		Tester(PCGAttributeTestsCommonHelper::FPCGPointTester());
	}
	
	SECTION("Array of Double Attribute")
	{		
		Tester(PCGAttributeTestsCommonHelper::ArrayAccessorDoubleTester());
	}
	
	SECTION("Array of FPCGPoint Attribute")
	{	
		Tester(PCGAttributeTestsCommonHelper::ArrayAccessorFPCGPointTester());
	}
	
	SECTION("Set of Double Attribute")
	{		
		Tester(PCGAttributeTestsCommonHelper::SetDoubleTester());
	}
	
	SECTION("Map of String -> Double Attribute")
	{
		Tester(PCGAttributeTestsCommonHelper::MapStringDoubleTester());
	}
	
	SECTION("Enum attribute")
	{
		Tester(PCGAttributeTestsCommonHelper::EnumTester());
	}
}

/**
 * Verifies that the generic accessor supports type broadcast and construction when the caller's type
 * does not match the attribute's underlying type. A double attribute is created, then written to with
 * int32 values (requiring construction from int32 -> double). The values are read back as int32
 * (broadcast double -> int32) and as FString (broadcast double -> string) to confirm the type
 * conversion path works end-to-end through SetRange_Internal/GetRange_Internal.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Metadata::Accessors::GenericAttribute::NotExactType", "[PCG][Accessors][GenericAttribute]")
{
	const FName DoubleAttributeName = "Double";
	
	// Preparing the data
	constexpr int32 NumPoints = 5;
	constexpr int32 Seed = 42;
	UPCGPointArrayData* PointArrayData = CreateData<UPCGPointArrayData>();

	PointArrayData->SetNumPoints(NumPoints);

	PointArrayData->AllocateProperties(EPCGPointNativeProperties::MetadataEntry);
	TPCGValueRange<int64> PointArrayDataMetadataEntryRange = PointArrayData->GetMetadataEntryValueRange(/*bAllocate=*/false);

	FPCGMetadataDomain* Domain = PointArrayData->MutableMetadata()->GetDefaultMetadataDomain();
		
	FRandomStream RandomStream(Seed);
		
	FPCGMetadataAttributeBase* GenericAttribute = Domain->CreateAttribute<double>(DoubleAttributeName, -1.0);
	REQUIRE_NOT_EQUAL(GenericAttribute, nullptr);
	
	TUniquePtr<IPCGAttributeAccessor> DoubleAttributeAccessor = PCGAttributeAccessorHelpers::CreateAccessor(PointArrayData, FPCGAttributePropertySelector::CreateAttributeSelector(DoubleAttributeName));
	TUniquePtr<IPCGAttributeAccessorKeys> DoubleAttributeKeys = PCGAttributeAccessorHelpers::CreateKeys(PointArrayData, FPCGAttributePropertySelector::CreateAttributeSelector(DoubleAttributeName));

	REQUIRE(DoubleAttributeAccessor.IsValid());
	REQUIRE(DoubleAttributeKeys.IsValid());
	
	REQUIRE_EQUAL(PointArrayData->MutableMetadata()->GetItemCountForChild(), 0);
	REQUIRE_EQUAL(DoubleAttributeKeys->GetNum(), 5);

	TStaticArray<int32, NumPoints> Values{0, 1, 2, 3, 4};
	REQUIRE(DoubleAttributeAccessor->SetRange<int32>(Values, 0, *DoubleAttributeKeys, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible));
	REQUIRE_EQUAL(PointArrayData->MutableMetadata()->GetItemCountForChild(), 5);
	
	TStaticArray<int32, NumPoints> ReadValues{};
	REQUIRE(DoubleAttributeAccessor->GetRange<int32>(ReadValues, 0, *DoubleAttributeKeys, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible));
	CHECK_THAT(Values, Catch::Matchers::RangeEquals(ReadValues, [](const auto& a, const auto& b){ return a == b; } ));
	
	TStaticArray<FString, NumPoints> ExpectedValuesString{};
	TStaticArray<FString, NumPoints> ReadValuesString{};
	for (int32 i = 0; i < NumPoints; ++i)
	{
		ExpectedValuesString[i] = PCG::Private::MetadataTraits<double>::ToString(Values[i]);	
	}
	
	REQUIRE(DoubleAttributeAccessor->GetRange<FString>(ReadValuesString, 0, *DoubleAttributeKeys, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible));
	CHECK_THAT(ExpectedValuesString, Catch::Matchers::RangeEquals(ReadValuesString, [](const auto& a, const auto& b){ return a == b; } ));
}

/**
 * Tests value deduplication for compressed generic attributes. FString attributes use compressed storage
 * where duplicate values share the same value key. This test writes 6 values containing 3 unique strings,
 * verifies that only 3 value keys are allocated, then appends 4 more values (2 existing + 2 new) and
 * verifies the value key count grows to 5. Finally reads back all 10 values via pointer-based GetRange
 * to confirm correctness. Exercises the AddCompressedValues and FInValuesSubset code paths.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Metadata::Accessors::GenericAttribute::CompressedData", "[PCG][AttributeGeneric]")
{
	constexpr int32 Seed = 42;
	
	// Preparing the data
	constexpr int32 NumPoints = 10;
	UPCGPointArrayData* PointArrayData = CreateData<UPCGPointArrayData>();

	PointArrayData->SetNumPoints(NumPoints);

	PointArrayData->AllocateProperties(EPCGPointNativeProperties::MetadataEntry);
	TPCGValueRange<int64> PointArrayDataMetadataEntryRange = PointArrayData->GetMetadataEntryValueRange(/*bAllocate=*/false);
		
	FPCGMetadataDomain* Domain = PointArrayData->MutableMetadata()->GetDefaultMetadataDomain();
		
	FRandomStream RandomStream(Seed);
		
	FPCGMetadataAttributeBase* GenericAttribute = Domain->CreateAttribute<FString>(PCGAttributeTestsCommonHelper::AttributeName, FString{});
	REQUIRE_NOT_EQUAL(GenericAttribute, nullptr);
		
	TUniquePtr<IPCGAttributeAccessor> AttributeAccessor = PCGAttributeAccessorHelpers::CreateAccessor(PointArrayData, FPCGAttributePropertySelector::CreateAttributeSelector(PCGAttributeTestsCommonHelper::AttributeName));
	TUniquePtr<IPCGAttributeAccessorKeys> AttributeKeys = PCGAttributeAccessorHelpers::CreateKeys(PointArrayData, FPCGAttributePropertySelector::CreateAttributeSelector(PCGAttributeTestsCommonHelper::AttributeName));

	REQUIRE(AttributeAccessor.IsValid());
	REQUIRE(AttributeKeys.IsValid());
	
	// First write 6 values, that are 3 unique values
	TStaticArray<FString, NumPoints> Values{};
	TStaticArray<const FString*, NumPoints> ReadValues{};
	
	Values[0] = TEXT("Value 0");
	Values[1] = TEXT("Value 0");
	Values[2] = TEXT("Value 1");
	Values[3] = TEXT("Value 2");
	Values[4] = TEXT("Value 2");
	Values[5] = TEXT("Value 1");
	
	REQUIRE(AttributeAccessor->SetRange<FString>(MakeArrayView(Values.GetData(), 6), 0, *AttributeKeys));
	REQUIRE_EQUAL(GenericAttribute->GetNumberOfEntries(), 6);
	REQUIRE_EQUAL(GenericAttribute->GetValueKeyOffsetForChild(), 3);
	
	// Then add new values, some existing, some new
	Values[6] = TEXT("Value 1");
	Values[7] = TEXT("Value 2");
	Values[8] = TEXT("Value 3");
	Values[9] = TEXT("Value 4");
	
	REQUIRE(AttributeAccessor->SetRange<FString>(MakeArrayView(Values.GetData() + 6, 4), 6, *AttributeKeys));
	REQUIRE_EQUAL(GenericAttribute->GetNumberOfEntries(), 10);
	REQUIRE_EQUAL(GenericAttribute->GetValueKeyOffsetForChild(), 5);
	
	//  Read back to verify we have all our values
	REQUIRE(AttributeAccessor->GetRange<const FString*>(ReadValues, 0, *AttributeKeys));
	CHECK_THAT(Values, Catch::Matchers::RangeEquals(ReadValues, [](const FString& a, const FString* b){ return a == *b; } ));
}