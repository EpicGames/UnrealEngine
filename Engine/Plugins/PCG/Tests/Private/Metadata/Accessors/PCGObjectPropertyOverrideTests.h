// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/MathFwd.h"
#include "Templates/SubclassOf.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/SoftObjectPtr.h"

#include "PCGObjectPropertyOverrideTests.generated.h"

/** Empty UObject used as the value carried by object/class/soft-object/soft-class properties in the testbed. */
UCLASS(BlueprintInternalUseOnly)
class UPCGObjectPropertyOverrideTestsObject : public UObject
{
	GENERATED_BODY()
};

USTRUCT()
struct FPCGObjectPropertyOverrideTestsSubStruct
{
	GENERATED_BODY()

	UPROPERTY()
	double Inner = 0.0;
	
	bool operator==(const FPCGObjectPropertyOverrideTestsSubStruct& Other) const { return Inner == Other.Inner; }
	bool operator!=(const FPCGObjectPropertyOverrideTestsSubStruct& Other) const { return Inner != Other.Inner; }
};

UENUM()
enum class EPCGObjectPropertyOverrideTestsEnum : uint8
{
	A,
	B,
	C,
	D
};

/**
 * One UPROPERTY per supported type, plus its TArray<> counterpart.
 * Used by FPCGObjectOverrides<T> as the template object — the override path must be able to read/write each property.
 */
USTRUCT()
struct FPCGObjectPropertyOverrideTestsStruct
{
	GENERATED_BODY()

	// Numeric scalars
	UPROPERTY()
	double DoubleValue = 0.0;
	
	UPROPERTY()
	float FloatValue = 0.f;
	
	UPROPERTY()
	int32 Int32Value = 0;
	
	UPROPERTY()
	int64 Int64Value = 0;
	
	// Native bool
	UPROPERTY()
	bool BoolValue = false;
	
	// Bitflags
	UPROPERTY()
	uint8 bBitfieldFlag1 : 1 = 0;
	
	UPROPERTY()
	uint8 bBitfieldFlag2 : 1 = 0;
	
	// Math structs (predefined types in the descriptor system)
	UPROPERTY()
	FVector2D Vector2DValue = FVector2D::ZeroVector;
	
	UPROPERTY()
	FVector VectorValue = FVector::ZeroVector;
	
	UPROPERTY()
	FVector4 Vector4Value = FVector4::Zero();
	
	UPROPERTY()
	FQuat QuatValue = FQuat::Identity;
	
	UPROPERTY()
	FRotator RotatorValue = FRotator::ZeroRotator;
	
	UPROPERTY()
	FTransform TransformValue = FTransform::Identity;
	
	// Strings / names
	UPROPERTY()
	FString StringValue;
	
	UPROPERTY()
	FName NameValue = NAME_None;
	
	// Object / class references
	UPROPERTY()
	TObjectPtr<UObject> ObjectValue;
	
	UPROPERTY()
	TSoftObjectPtr<UObject> SoftObjectValue;
	
	UPROPERTY()
	TSubclassOf<UObject> ClassValue;
	
	UPROPERTY()
	TSoftClassPtr<UObject> SoftClassValue;
	
	UPROPERTY()
	FSoftObjectPath SoftObjectPathValue;
	
	UPROPERTY()
	FSoftClassPath SoftClassPathValue;
	
	// Enums
	UPROPERTY()
	EPCGObjectPropertyOverrideTestsEnum EnumValue = EPCGObjectPropertyOverrideTestsEnum::A;
	
	// Structs
	UPROPERTY()
	FPCGObjectPropertyOverrideTestsSubStruct StructValue;

	
	// Array variants
	UPROPERTY()
	TArray<double> DoubleArray;
	
	UPROPERTY()
	TArray<float> FloatArray;
	
	UPROPERTY()
	TArray<int32> Int32Array;
	
	UPROPERTY()
	TArray<int64> Int64Array;
	
	UPROPERTY()
	TArray<bool> BoolArray;
	
	UPROPERTY()
	TArray<FVector2D> Vector2DArray;
	
	UPROPERTY()
	TArray<FVector> VectorArray;
	
	UPROPERTY()
	TArray<FVector4> Vector4Array;
	
	UPROPERTY()
	TArray<FQuat> QuatArray;
	
	UPROPERTY()
	TArray<FRotator> RotatorArray;
	
	UPROPERTY()
	TArray<FTransform> TransformArray;
	
	UPROPERTY()
	TArray<FString> StringArray;
	
	UPROPERTY()
	TArray<FName> NameArray;
	
	UPROPERTY()
	TArray<TObjectPtr<UObject>> ObjectArray;
	
	UPROPERTY()
	TArray<TSoftObjectPtr<UObject>> SoftObjectArray;
	
	UPROPERTY()
	TArray<FSoftObjectPath> SoftObjectPathArray;
	
	UPROPERTY()
	TArray<FSoftClassPath> SoftClassPathArray;
	
	UPROPERTY() 
	TArray<EPCGObjectPropertyOverrideTestsEnum> EnumArray;
	
	UPROPERTY() 
	TArray<FPCGObjectPropertyOverrideTestsSubStruct> StructArray;
};
