// Copyright Epic Games, Inc. All Rights Reserved.

// =============================================================================
// PCG Property Accessor Tests
//
// Tests for FPCGPropertyGenericAccessor Get/Set operations on Unreal properties.
// Validates generic read/write across multiple value types, container
// types (Array, Set, Map), property chains (struct-in-struct), non-zero property
// offsets, custom getter/setter support, and broadcast conversions between the
// object descriptor types and their string/path counterparts.
//
// Test summary:
//  - Basic::DirectContainer:              Set/Get values through a single PropertyBag property for each type
//  - Basic::StructInStruct:               Set/Get values through a nested struct property chain
//  - Basic::WithOffset:                   Set/Get on the 3rd property in a bag, verifying the 1st and 2nd are untouched
//  - GetterSetter:                        Set/Get through a UObject property with custom Setter (divides by 2) and Getter (multiplies by 2)
//  - ObjectBroadcast::Range:              SetRange/GetRange round-trips with AllowBroadcast between Object/Class/SoftObject/SoftClass
//                                         scalar slots and FString/FSoftObjectPath/FSoftClassPath/TSoftObjectPtr/TSoftClassPtr targets
//  - ObjectBroadcast::Array:              Same broadcast pairs but on array-container slots, exercising element-wise conversion
//  - ObjectBadRelation:                   IsCompatible gate rejects Set/Get when the read class is not a subclass of the write class
//                                         (UObject into AActor, Base into Derived, Class metaclass mismatch)
//  - NullObjectInChain:                   FPCGPropertyGenericAccessor rejects Get/SetRange when the property chain traverses
//                                         a null TObjectPtr hop (all-null and one-null-in-the-middle), and round-trips correctly
//                                         when every hop resolves
//
// Types covered by Basic:
//  Double, FVector, FString, FPCGPoint, TArray<double>, TArray<FPCGPoint>,
//  TSet<double>, TMap<FString, double>, EPCGMetadataTypes (enum)
//
// Broadcast pairs covered by ObjectBroadcast:
//  Object        <-> FString, FSoftObjectPath, TSoftObjectPtr<>
//  Class         <-> FString, FSoftClassPath,  TSoftClassPtr<>
//  SoftObject    <-> FString, FSoftObjectPath
//  SoftClass     <-> FString, FSoftClassPath
//  TArray<Object>     <-> TArray<FString>, TArray<FSoftObjectPath>
//  TArray<Class>      <-> TArray<FString>
//  TArray<SoftObject> <-> TArray<FSoftObjectPath>
//  TArray<SoftClass>  <-> TArray<FSoftClassPath>
// =============================================================================

#include "PCGPropertyAccessorTests.h"

#include "CoreMinimal.h"
#include "TestHarness.h"

#include "PCGTestsCommon.h"
#include "Metadata/PCGMetadataAttributeTestsCommonHelper.h"

#include <catch2/generators/catch_generators.hpp>

#include "Data/PCGPointArrayData.h"
#include "Engine/StaticMesh.h"
#include "Helpers/PCGPropertyHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGPropertyAccessor.h"

#include "GameFramework/Actor.h"

// Tests that FPCGPropertyGenericAccessor can Set and Get values for various types, through direct containers, nested struct chains, and properties at non-zero offsets within a PropertyBag.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Metadata::Accessors::Properties::Basic", "[PCG][Properties]")
{
	constexpr int32 Seed = 42;
	constexpr int32 NumInstances = 128;
	
	auto Tester = [this]<typename T, typename ReadType>(PCGAttributeTestsCommonHelper::TypedAttributeTester<T, ReadType> AttributeTester)
	{
		static const FName PropertyName = "NewProperty";

		const FPropertyBagPropertyDesc Desc = PCGPropertyHelpers::CreatePropertyBagDescWithMetadataType(PropertyName, AttributeTester.ExpectedDesc);
		
		// @hack: Add a dummy "complex" property to bypass the ZeroConstructor flag set on the property bag if all the properties are "simple".
		// At the moment, having a property bag into a property bag doesn't increment the property bag ref count correctly if the property bag is marked "ZeroConstructor".
		const FPropertyBagPropertyDesc ComplexDesc("Dummy", EPropertyBagContainerType::None, EPropertyBagPropertyType::Text);
		
		const UPropertyBag* PropertyBag = UPropertyBag::GetOrCreateFromDescs({Desc, ComplexDesc});
		REQUIRE_NOT_EQUAL(PropertyBag, nullptr);
		
		const FPropertyBagPropertyDesc* AddedDesc = PropertyBag->FindPropertyDescByName(PropertyName);
		REQUIRE_NOT_EQUAL(AddedDesc, nullptr);
		
		SECTION("Direct Container")
		{
			TArray<FInstancedPropertyBag> Instances;
			TArray<void*> InstancesMemory;
			Instances.Reserve(NumInstances);
			InstancesMemory.Reserve(NumInstances);
			
			for (int32 i = 0; i < NumInstances; ++i)
			{
				Instances.Emplace_GetRef().InitializeFromBagStruct(PropertyBag);
				REQUIRE(Instances[i].IsValid());
				InstancesMemory.Add(Instances[i].GetMutableValue().GetMemory());
			}
			
			FRandomStream RandomStream(Seed);
			TArray<T> ValuesToAdd{};
			ValuesToAdd.Reserve(NumInstances);
			
			for (int32 i = 0; i < NumInstances; ++i)
			{
				AttributeTester.GenerateRandom(RandomStream, ValuesToAdd.Emplace_GetRef());
			}
			
			FPCGPropertyGenericAccessor Accessor(AddedDesc->CachedProperty);
			FPCGAttributeAccessorKeysGenericPtrs Keys(InstancesMemory);
		
			REQUIRE(Accessor.GetUnderlyingDesc().IsSameType(AttributeTester.ExpectedDesc));
		
			REQUIRE(Accessor.SetRange<T>(ValuesToAdd, /*Index=*/0, Keys));
		
			TArray<ReadType> ReadValues;
			ReadValues.SetNum(NumInstances);
			REQUIRE(Accessor.GetRange<ReadType>(ReadValues,  /*Index=*/0, Keys));
		
			CHECK_THAT(ValuesToAdd, Catch::Matchers::RangeEquals(ReadValues, [&AttributeTester](const auto& a, const auto& b){ return AttributeTester.Verify(a, b); } ));
		}
		
		SECTION("Struct in Struct")
		{
			static const FName InnerStructName = "Inner";
			TArray<FInstancedPropertyBag> OuterInstances;
			TArray<void*> OuterInstancesMemory;
			OuterInstances.Reserve(NumInstances);
			OuterInstancesMemory.Reserve(NumInstances);
			
			FPropertyBagPropertyDesc InnerDesc(InnerStructName, EPropertyBagPropertyType::Struct, PropertyBag);
			const UPropertyBag* OuterPropertyBag = UPropertyBag::GetOrCreateFromDescs({InnerDesc});
			
			for (int32 i = 0; i < NumInstances; ++i)
			{
				OuterInstances.Emplace_GetRef().InitializeFromBagStruct(OuterPropertyBag);
				REQUIRE(OuterInstances[i].IsValid());
				OuterInstancesMemory.Add(OuterInstances[i].GetMutableValue().GetMemory());
			}
			
			const FPropertyBagPropertyDesc* InnerAddedDesc = OuterPropertyBag->FindPropertyDescByName(InnerStructName);
			REQUIRE_NOT_EQUAL(InnerAddedDesc, nullptr);
			
			// Offset seed to get another value
			FRandomStream RandomStream(Seed+1);
			TArray<T> ValuesToAdd{};
			ValuesToAdd.Reserve(NumInstances);
			
			for (int32 i = 0; i < NumInstances; ++i)
			{
				AttributeTester.GenerateRandom(RandomStream, ValuesToAdd.Emplace_GetRef());
			}
			
			FPCGPropertyGenericAccessor Accessor(AddedDesc->CachedProperty, {InnerAddedDesc->CachedProperty});
			FPCGAttributeAccessorKeysGenericPtrs Keys(OuterInstancesMemory);
		
			REQUIRE(Accessor.GetUnderlyingDesc().IsSameType(AttributeTester.ExpectedDesc));
		
			REQUIRE(Accessor.SetRange<T>(ValuesToAdd, /*Index=*/0, Keys));
			
			TArray<ReadType> ReadValues;
			ReadValues.SetNum(NumInstances);
			REQUIRE(Accessor.GetRange<ReadType>(ReadValues,  /*Index=*/0, Keys));
			
			CHECK_THAT(ValuesToAdd, Catch::Matchers::RangeEquals(ReadValues, [&AttributeTester](const auto& a, const auto& b){ return AttributeTester.Verify(a, b); } ));
		}

		 SECTION("With offset")
		 {
		 	// Add the same property thrice, and try to access the last one
			// @hack: Add a dummy "complex" property to bypass the ZeroConstructor flag set on the property bag if all the properties are "simple".
			// At the moment, having a property bag into a property bag doesn't increment the property bag ref count correctly if the property bag is marked "ZeroConstructor".
			const FPropertyBagPropertyDesc Desc1("Dummy", EPropertyBagContainerType::None, EPropertyBagPropertyType::Text);
		 	const FPropertyBagPropertyDesc Desc2 = PCGPropertyHelpers::CreatePropertyBagDescWithMetadataType("SecondProp", AttributeTester.ExpectedDesc);
		 	const FPropertyBagPropertyDesc Desc3 = PCGPropertyHelpers::CreatePropertyBagDescWithMetadataType(PropertyName, AttributeTester.ExpectedDesc);
		 	const UPropertyBag* BulkyPropertyBag = UPropertyBag::GetOrCreateFromDescs({Desc1, Desc2, Desc3});
		 	REQUIRE_NOT_EQUAL(BulkyPropertyBag, nullptr);
		
		 	TArray<FInstancedPropertyBag> BulkyInstances;
		 	TArray<void*> BulkyInstancesMemory;
		 	BulkyInstances.Reserve(NumInstances);
		 	BulkyInstancesMemory.Reserve(NumInstances);
		
		 	for (int32 i = 0; i < NumInstances; ++i)
		 	{
		 		BulkyInstances.Emplace_GetRef().InitializeFromBagStruct(BulkyPropertyBag);
		 		REQUIRE(BulkyInstances[i].IsValid());
		 		BulkyInstancesMemory.Add(BulkyInstances[i].GetMutableValue().GetMemory());
		 	}
		
		 	const FPropertyBagPropertyDesc* BulkyAddedDesc1 = BulkyPropertyBag->FindPropertyDescByName("Dummy");
		 	const FPropertyBagPropertyDesc* BulkyAddedDesc2 = BulkyPropertyBag->FindPropertyDescByName("SecondProp");
		 	const FPropertyBagPropertyDesc* BulkyAddedDesc3 = BulkyPropertyBag->FindPropertyDescByName(PropertyName);
		 	REQUIRE_NOT_EQUAL(BulkyAddedDesc1, nullptr);
		 	REQUIRE_NOT_EQUAL(BulkyAddedDesc2, nullptr);
		 	REQUIRE_NOT_EQUAL(BulkyAddedDesc3, nullptr);
		 	
		 	// Offset seed to get another value
		 	FRandomStream RandomStream(Seed+2);
		 	TArray<T> ValuesToAdd{};
		 	ValuesToAdd.Reserve(NumInstances);
		
		 	for (int32 i = 0; i < NumInstances; ++i)
		 	{
		 		AttributeTester.GenerateRandom(RandomStream, ValuesToAdd.Emplace_GetRef());
		 	}
		
		 	FPCGPropertyGenericAccessor Accessor(BulkyAddedDesc3->CachedProperty);
		 	FPCGAttributeAccessorKeysGenericPtrs Keys(BulkyInstancesMemory);
		
		 	REQUIRE(Accessor.GetUnderlyingDesc().IsSameType(AttributeTester.ExpectedDesc));
		
		 	REQUIRE(Accessor.SetRange<T>(ValuesToAdd, /*Index=*/0, Keys));
		 	
		 	// Verify that the values are set correctly
		 	FInstancedPropertyBag EmptyInstance;
		 	EmptyInstance.InitializeFromBagStruct(BulkyPropertyBag);
		 	for (int32 i = 0; i < NumInstances; ++i)
		 	{
		 		REQUIRE(BulkyAddedDesc1->CachedProperty->Identical_InContainer(BulkyInstances[i].GetValue().GetMemory(), EmptyInstance.GetValue().GetMemory()));
		 		REQUIRE(BulkyAddedDesc2->CachedProperty->Identical_InContainer(BulkyInstances[i].GetValue().GetMemory(), EmptyInstance.GetValue().GetMemory()));
		 		REQUIRE_FALSE(BulkyAddedDesc3->CachedProperty->Identical_InContainer(BulkyInstances[i].GetValue().GetMemory(), EmptyInstance.GetValue().GetMemory()));
		 	}
		
		 	TArray<ReadType> ReadValues;
		 	ReadValues.SetNum(NumInstances);
		 	REQUIRE(Accessor.GetRange<ReadType>(ReadValues,  /*Index=*/0, Keys));
		
		 	CHECK_THAT(ValuesToAdd, Catch::Matchers::RangeEquals(ReadValues, [&AttributeTester](const auto& a, const auto& b){ return AttributeTester.Verify(a, b); } ));
		 }
	};
	
	SECTION("Double Attribute")
	{
		Tester(PCGAttributeTestsCommonHelper::DoubleTester());
	}
	
	SECTION("Vector Attribute")
	{
		Tester(PCGAttributeTestsCommonHelper::VectorTester_Basic());
	}
	
	SECTION("String Attribute")
	{
		Tester(PCGAttributeTestsCommonHelper::StringTester());
	}
	
	SECTION("FPCGPoint Attribute")
	{		
		Tester(PCGAttributeTestsCommonHelper::FPCGPointTester());
	}
	
	SECTION("Array of Double Attribute")
	{		
		Tester(PCGAttributeTestsCommonHelper::ArrayAccessorDoubleTester());
	}
	
	SECTION("Array of FPCGPoint Attribute")
	{	
		Tester(PCGAttributeTestsCommonHelper::ArrayAccessorFPCGPointTester());
	}
	
	SECTION("Set of Double Attribute")
	{		
		Tester(PCGAttributeTestsCommonHelper::SetDoubleTester());
	}
	
	SECTION("Map of String -> Double Attribute")
	{
		Tester(PCGAttributeTestsCommonHelper::MapStringDoubleTester());
	}
	
	SECTION("Enum attribute")
	{
		Tester(PCGAttributeTestsCommonHelper::EnumTester());
	}
}

// Tests that FPCGPropertyGenericAccessor correctly invokes custom UObject Setter/Getter when writing and reading a property, verifying the stored value is transformed (divided by 2) and the read value round-trips back to the original.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Metadata::Accessors::Properties::GetterSetter", "[PCG][Properties]")
{
	constexpr int32 Seed = 42;
	constexpr int32 NumInstances = 128;
	
	TArray<UPCGPropertyAccessorTestsGetterSetter*> Instances;
	TArray<void*> InstancesMemory;
	Instances.Reserve(NumInstances);
	InstancesMemory.Reserve(NumInstances);
			
	for (int32 i = 0; i < NumInstances; ++i)
	{
		Instances.Add(FPCGContext::NewObject_AnyThread<UPCGPropertyAccessorTestsGetterSetter>(GetContext()));
		InstancesMemory.Add(Instances[i]);
	}
			
	// Offset seed to get another value
	FRandomStream RandomStream(Seed+1);
	TArray<double> ValuesToAdd{};
	ValuesToAdd.Reserve(NumInstances);
			
	for (int32 i = 0; i < NumInstances; ++i)
	{
		ValuesToAdd.Add(RandomStream.FRand());
	}
	
	const FProperty* Property = UPCGPropertyAccessorTestsGetterSetter::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UPCGPropertyAccessorTestsGetterSetter, Value));
	REQUIRE(Property);
	REQUIRE(Property->HasSetter());
	REQUIRE(Property->HasGetter());
	
	FPCGPropertyGenericAccessor Accessor(Property);
	FPCGAttributeAccessorKeysGenericPtrs Keys(InstancesMemory);
		
	REQUIRE(Accessor.GetUnderlyingDesc().IsSameType(PCG::Private::GetDefaultAttributeDesc<double>()));
	
	REQUIRE(Accessor.SetRange<double>(ValuesToAdd, /*Index=*/0, Keys));
	
	// Verify that the value is stored as expected
	TArray<double> StoredValues;
	StoredValues.Reserve(NumInstances);
	Algo::Transform(Instances, StoredValues, [](const UPCGPropertyAccessorTestsGetterSetter* Obj) { return Obj->Value; });
	REQUIRE_THAT(ValuesToAdd, Catch::Matchers::RangeEquals(StoredValues, [](const double a, const double b){ return FMath::IsNearlyEqual(a / 2.0, b); }));
		
	TArray<double> ReadValues;
	ReadValues.SetNum(NumInstances);
	REQUIRE(Accessor.GetRange<double>(ReadValues,  /*Index=*/0, Keys));

	CHECK_THAT(ValuesToAdd, Catch::Matchers::RangeEquals(ReadValues, [](const double a, const double b){ return FMath::IsNearlyEqual(a, b); }));
}

// =============================================================================
// Object broadcast tests
//
// Verifies the broadcast conversions between the
// object descriptor types (Object, Class, SoftObject, SoftClass) and the
// string/path descriptor types (FString, FSoftObjectPath, FSoftClassPath) and
// their TSoftObjectPtr/TSoftClassPtr counterparts.
//
// The slot-and-broadcast helper builds a property bag with a single field of
// the slot descriptor type, wraps it in a FPCGPropertyGenericAccessor and runs
// a round-trip: Set<BroadcastType>(Sample) with AllowBroadcast, then
// Get<BroadcastType>(...) with AllowBroadcast. The stored value is also
// probed directly in its native slot type so that both conversion directions
// are exercised.
// =============================================================================

namespace PCGPropertyAccessorTests_ObjectBroadcast
{
	constexpr int32 NumInstances = 2;
	static const FName PropertyName = "ObjectSlot";

	struct FPropertyBagAccessorFixture
	{
		TArray<FInstancedPropertyBag> Bags;
		TArray<void*> InstancesMemory;
		TUniquePtr<FPCGPropertyGenericAccessor> Accessor;
		TUniquePtr<FPCGAttributeAccessorKeysGenericPtrs> Keys;
	};

	// Build a property bag with a single field of SlotDesc type, instantiated InNumInstances times so that
	// SetRange/GetRange can operate on a contiguous range of keys.
	inline TUniquePtr<FPropertyBagAccessorFixture> MakeFixture(const FPCGMetadataAttributeDesc& SlotDesc, int32 InNumInstances = NumInstances)
	{
		const FPropertyBagPropertyDesc FieldDesc = PCGPropertyHelpers::CreatePropertyBagDescWithMetadataType(PropertyName, SlotDesc);

		const UPropertyBag* PropertyBag = UPropertyBag::GetOrCreateFromDescs({FieldDesc});
		if (!PropertyBag)
		{
			return nullptr;
		}

		const FPropertyBagPropertyDesc* AddedDesc = PropertyBag->FindPropertyDescByName(PropertyName);
		if (!AddedDesc || !AddedDesc->CachedProperty)
		{
			return nullptr;
		}

		TUniquePtr<FPropertyBagAccessorFixture> Fixture = MakeUnique<FPropertyBagAccessorFixture>();

		// Reserve up-front so that Emplace_GetRef never reallocates and the captured value-memory pointers stay stable.
		Fixture->Bags.Reserve(InNumInstances);
		Fixture->InstancesMemory.Reserve(InNumInstances);
		for (int32 i = 0; i < InNumInstances; ++i)
		{
			Fixture->Bags.Emplace_GetRef().InitializeFromBagStruct(PropertyBag);
			if (!Fixture->Bags[i].IsValid())
			{
				return nullptr;
			}
			Fixture->InstancesMemory.Add(Fixture->Bags[i].GetMutableValue().GetMemory());
		}

		Fixture->Accessor = MakeUnique<FPCGPropertyGenericAccessor>(AddedDesc->CachedProperty);
		Fixture->Keys = MakeUnique<FPCGAttributeAccessorKeysGenericPtrs>(Fixture->InstancesMemory);
		return Fixture;
	}

	// Writes a range of BroadcastType values into the slot via SetRange<BroadcastType> with AllowBroadcast, then reads them
	// back via GetRange<BroadcastType> with AllowBroadcast. Both calls must succeed and every read-back value must match the
	// corresponding input according to Verify. The read-back step implicitly exercises the reverse conversion Slot -> Broadcast.
	template <typename BroadcastType>
	void TestRoundTripBroadcastRange(const FPCGMetadataAttributeDesc& SlotDesc, TConstArrayView<BroadcastType> Samples, TFunctionRef<bool(const BroadcastType& /*Written*/, const BroadcastType& /*ReadBack*/)> Verify)
	{
		TUniquePtr<FPropertyBagAccessorFixture> Fixture = MakeFixture(SlotDesc, Samples.Num());
		REQUIRE(Fixture);

		REQUIRE(Fixture->Accessor->SetRange<BroadcastType>(Samples, /*Index=*/0, *Fixture->Keys, EPCGAttributeAccessorFlags::AllowBroadcast));

		TArray<BroadcastType> ReadBack;
		ReadBack.SetNum(Samples.Num());
		REQUIRE(Fixture->Accessor->GetRange<BroadcastType>(ReadBack, /*Index=*/0, *Fixture->Keys, EPCGAttributeAccessorFlags::AllowBroadcast));

		for (int32 i = 0; i < Samples.Num(); ++i)
		{
			REQUIRE(Verify(Samples[i], ReadBack[i]));
		}
	}

	// Array variant of the round-trip helper. The slot is an array container of the scalar type; each sample is itself a
	// TArray<BroadcastType> that is written to one instance via SetRange<TArray<BroadcastType>> and read back via
	// GetRange<TPCGArrayAccessorWrapper<BroadcastType>>, both with AllowBroadcast.
	template <typename BroadcastType>
	void TestRoundTripBroadcastArray(const FPCGMetadataAttributeDesc& SlotDesc, TConstArrayView<TArray<BroadcastType>> Samples, TFunctionRef<bool(const BroadcastType& /*Written*/, const BroadcastType& /*ReadBack*/)> Verify)
	{
		TUniquePtr<FPropertyBagAccessorFixture> Fixture = MakeFixture(SlotDesc, Samples.Num());
		REQUIRE(Fixture);

		REQUIRE(Fixture->Accessor->SetRange<TArray<BroadcastType>>(Samples, /*Index=*/0, *Fixture->Keys, EPCGAttributeAccessorFlags::AllowBroadcast));

		TArray<PCG::TPCGArrayAccessorWrapper<BroadcastType>> ReadBack;
		ReadBack.SetNum(Samples.Num());
		REQUIRE(Fixture->Accessor->GetRange<PCG::TPCGArrayAccessorWrapper<BroadcastType>>(ReadBack, /*Index=*/0, *Fixture->Keys, EPCGAttributeAccessorFlags::AllowBroadcast));

		for (int32 i = 0; i < Samples.Num(); ++i)
		{
			const TConstArrayView<BroadcastType> ReadBackView = ReadBack[i].GetView();
			REQUIRE_EQUAL(Samples[i].Num(), ReadBackView.Num());
			for (int32 j = 0; j < Samples[i].Num(); ++j)
			{
				REQUIRE(Verify(Samples[i][j], ReadBackView[j]));
			}
		}
	}

	inline FPCGMetadataAttributeDesc MakeDesc(EPCGMetadataTypes InType, const UClass* InClass, bool bArray = false)
	{
		FPCGMetadataAttributeDesc Desc;
		Desc.Name = "ObjectSlot";
		Desc.ValueType = InType;
		Desc.ValueTypeObject = InClass;
		if (bArray)
		{
			Desc.ContainerTypes.Add(EPCGMetadataAttributeContainerTypes::Array);
		}
		return Desc;
	}
}

// Tests all Object/Class/SoftObject/SoftClass <-> FString/FSoftObjectPath/FSoftClassPath/TSoftObjectPtr/TSoftClassPtr
// broadcast conversions introduced in CL 53124582. For each slot type we pick two representative object values, then
// exercise every registered broadcast target by round-tripping a range of samples through SetRange/GetRange with
// AllowBroadcast. Array-container slots are exercised separately through the ContainerTypes=[Array] variant.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Metadata::Accessors::Properties::ObjectBroadcast", "[PCG][Properties]")
{
	using namespace PCGPropertyAccessorTests_ObjectBroadcast;

	// Two distinct resolvable instances to guarantee that SetRange/GetRange really propagate each element to its own key.
	UPCGPropertyAccessorTestsDerivedObject* TestObjectA = FPCGContext::NewObject_AnyThread<UPCGPropertyAccessorTestsDerivedObject>(GetContext());
	UPCGPropertyAccessorTestsDerivedObject* TestObjectB = FPCGContext::NewObject_AnyThread<UPCGPropertyAccessorTestsDerivedObject>(GetContext());
	REQUIRE(TestObjectA);
	REQUIRE(TestObjectB);

	// For class tests we use two distinct UClass values so the two range entries differ.
	UClass* TestClassA = UPCGPropertyAccessorTestsDerivedObject::StaticClass();
	UClass* TestClassB = UPCGPropertyAccessorTestsBaseObject::StaticClass();
	REQUIRE(TestClassA);
	REQUIRE(TestClassB);

	const TArray<FSoftObjectPath> TestObjectPaths = { FSoftObjectPath(TestObjectA), FSoftObjectPath(TestObjectB) };
	const TArray<FSoftClassPath> TestClassPaths = { FSoftClassPath(TestClassA), FSoftClassPath(TestClassB) };

	TArray<FString> TestObjectStrings;
	TArray<FString> TestClassStrings;
	Algo::Transform(TestObjectPaths, TestObjectStrings, [](const FSoftObjectPath& P) { return P.ToString(); });
	Algo::Transform(TestClassPaths, TestClassStrings, [](const FSoftClassPath& P) { return P.ToString(); });

	const TArray<TSoftObjectPtr<UPCGPropertyAccessorTestsBaseObject>> TestSoftObjects = {
		TSoftObjectPtr<UPCGPropertyAccessorTestsBaseObject>(TestObjectA),
		TSoftObjectPtr<UPCGPropertyAccessorTestsBaseObject>(TestObjectB)
	};
	const TArray<TSoftClassPtr<UPCGPropertyAccessorTestsBaseObject>> TestSoftClasses = {
		TSoftClassPtr<UPCGPropertyAccessorTestsBaseObject>(TestClassA),
		TSoftClassPtr<UPCGPropertyAccessorTestsBaseObject>(TestClassB)
	};

	const auto StringEqual = [](const FString& A, const FString& B) { return A == B; };
	const auto PathEqual = [](const auto& A, const auto& B) { return A == B; };
	const auto SoftPtrEqual = [](const auto& A, const auto& B) { return A.ToSoftObjectPath() == B.ToSoftObjectPath(); };

	// ---- Scalar ranges ----
	SECTION("Object slot <-> FString range")
	{
		const FPCGMetadataAttributeDesc SlotDesc = MakeDesc(EPCGMetadataTypes::Object, UPCGPropertyAccessorTestsBaseObject::StaticClass());
		TestRoundTripBroadcastRange<FString>(SlotDesc, TestObjectStrings, StringEqual);
	}

	SECTION("Object slot <-> FSoftObjectPath range")
	{
		const FPCGMetadataAttributeDesc SlotDesc = MakeDesc(EPCGMetadataTypes::Object, UPCGPropertyAccessorTestsBaseObject::StaticClass());
		TestRoundTripBroadcastRange<FSoftObjectPath>(SlotDesc, TestObjectPaths, PathEqual);
	}

	SECTION("Object slot <-> TSoftObjectPtr range")
	{
		const FPCGMetadataAttributeDesc SlotDesc = MakeDesc(EPCGMetadataTypes::Object, UPCGPropertyAccessorTestsBaseObject::StaticClass());
		TestRoundTripBroadcastRange<TSoftObjectPtr<UPCGPropertyAccessorTestsBaseObject>>(SlotDesc, TestSoftObjects, SoftPtrEqual);
	}

	SECTION("Class slot <-> FString range")
	{
		const FPCGMetadataAttributeDesc SlotDesc = MakeDesc(EPCGMetadataTypes::Class, UPCGPropertyAccessorTestsBaseObject::StaticClass());
		TestRoundTripBroadcastRange<FString>(SlotDesc, TestClassStrings, StringEqual);
	}

	SECTION("Class slot <-> FSoftClassPath range")
	{
		const FPCGMetadataAttributeDesc SlotDesc = MakeDesc(EPCGMetadataTypes::Class, UPCGPropertyAccessorTestsBaseObject::StaticClass());
		TestRoundTripBroadcastRange<FSoftClassPath>(SlotDesc, TestClassPaths, PathEqual);
	}

	SECTION("Class slot <-> TSoftClassPtr range")
	{
		const FPCGMetadataAttributeDesc SlotDesc = MakeDesc(EPCGMetadataTypes::Class, UPCGPropertyAccessorTestsBaseObject::StaticClass());
		TestRoundTripBroadcastRange<TSoftClassPtr<UPCGPropertyAccessorTestsBaseObject>>(SlotDesc, TestSoftClasses, SoftPtrEqual);
	}

	SECTION("SoftObject slot <-> FString range")
	{
		const FPCGMetadataAttributeDesc SlotDesc = MakeDesc(EPCGMetadataTypes::SoftObject, UPCGPropertyAccessorTestsBaseObject::StaticClass());
		TestRoundTripBroadcastRange<FString>(SlotDesc, TestObjectStrings, StringEqual);
	}

	SECTION("SoftObject slot <-> FSoftObjectPath range")
	{
		const FPCGMetadataAttributeDesc SlotDesc = MakeDesc(EPCGMetadataTypes::SoftObject, UPCGPropertyAccessorTestsBaseObject::StaticClass());
		TestRoundTripBroadcastRange<FSoftObjectPath>(SlotDesc, TestObjectPaths, PathEqual);
	}

	SECTION("SoftClass slot <-> FString range")
	{
		const FPCGMetadataAttributeDesc SlotDesc = MakeDesc(EPCGMetadataTypes::SoftClass, UPCGPropertyAccessorTestsBaseObject::StaticClass());
		TestRoundTripBroadcastRange<FString>(SlotDesc, TestClassStrings, StringEqual);
	}

	SECTION("SoftClass slot <-> FSoftClassPath range")
	{
		const FPCGMetadataAttributeDesc SlotDesc = MakeDesc(EPCGMetadataTypes::SoftClass, UPCGPropertyAccessorTestsBaseObject::StaticClass());
		TestRoundTripBroadcastRange<FSoftClassPath>(SlotDesc, TestClassPaths, PathEqual);
	}

	// ---- Array containers ----
	// One sample array per instance. Each inner array holds the same two values so we also verify that the broadcast
	// applies element-wise inside the container.
	SECTION("Object array slot <-> FString array")
	{
		const FPCGMetadataAttributeDesc SlotDesc = MakeDesc(EPCGMetadataTypes::Object, UPCGPropertyAccessorTestsBaseObject::StaticClass(), /*bArray=*/true);
		const TArray<TArray<FString>> Samples = { TestObjectStrings, TestObjectStrings };
		TestRoundTripBroadcastArray<FString>(SlotDesc, Samples, StringEqual);
	}

	SECTION("Object array slot <-> FSoftObjectPath array")
	{
		const FPCGMetadataAttributeDesc SlotDesc = MakeDesc(EPCGMetadataTypes::Object, UPCGPropertyAccessorTestsBaseObject::StaticClass(), /*bArray=*/true);
		const TArray<TArray<FSoftObjectPath>> Samples = { TestObjectPaths, TestObjectPaths };
		TestRoundTripBroadcastArray<FSoftObjectPath>(SlotDesc, Samples, PathEqual);
	}

	SECTION("Class array slot <-> FString array")
	{
		const FPCGMetadataAttributeDesc SlotDesc = MakeDesc(EPCGMetadataTypes::Class, UPCGPropertyAccessorTestsBaseObject::StaticClass(), /*bArray=*/true);
		const TArray<TArray<FString>> Samples = { TestClassStrings, TestClassStrings };
		TestRoundTripBroadcastArray<FString>(SlotDesc, Samples, StringEqual);
	}

	SECTION("SoftObject array slot <-> FSoftObjectPath array")
	{
		const FPCGMetadataAttributeDesc SlotDesc = MakeDesc(EPCGMetadataTypes::SoftObject, UPCGPropertyAccessorTestsBaseObject::StaticClass(), /*bArray=*/true);
		const TArray<TArray<FSoftObjectPath>> Samples = { TestObjectPaths, TestObjectPaths };
		TestRoundTripBroadcastArray<FSoftObjectPath>(SlotDesc, Samples, PathEqual);
	}

	SECTION("SoftClass array slot <-> FSoftClassPath array")
	{
		const FPCGMetadataAttributeDesc SlotDesc = MakeDesc(EPCGMetadataTypes::SoftClass, UPCGPropertyAccessorTestsBaseObject::StaticClass(), /*bArray=*/true);
		const TArray<TArray<FSoftClassPath>> Samples = { TestClassPaths, TestClassPaths };
		TestRoundTripBroadcastArray<FSoftClassPath>(SlotDesc, Samples, PathEqual);
	}
}

// Tests the IsCompatible gate added in CL 53124582: when the read and write descriptors both contain an object type
// (Object/Class/SoftObject/SoftClass), the read class must be a subclass of the write class. Writing a UObject into
// a slot whose metaclass is a more-specific type (AActor, or our derived test class) must fail, while writing the
// more-specific type into a permissive slot must still succeed.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Metadata::Accessors::Properties::ObjectBadRelation", "[PCG][Properties]")
{
	using namespace PCGPropertyAccessorTests_ObjectBroadcast;

	UPCGPropertyAccessorTestsDerivedObject* DerivedInstance = FPCGContext::NewObject_AnyThread<UPCGPropertyAccessorTestsDerivedObject>(GetContext());
	UPCGPropertyAccessorTestsBaseObject* BaseInstance = FPCGContext::NewObject_AnyThread<UPCGPropertyAccessorTestsBaseObject>(GetContext());
	REQUIRE(DerivedInstance);
	REQUIRE(BaseInstance);

	SECTION("Setting a UObject into an AActor slot fails")
	{
		// Slot expects AActor; reading descriptor is Object(UObject). UObject is not a child of AActor -> Invalid.
		const FPCGMetadataAttributeDesc SlotDesc = MakeDesc(EPCGMetadataTypes::Object, AActor::StaticClass());
		TUniquePtr<FPropertyBagAccessorFixture> Fixture = MakeFixture(SlotDesc);
		REQUIRE(Fixture);

		TObjectPtr<UObject> Value = BaseInstance;
		REQUIRE_FALSE(Fixture->Accessor->Set<TObjectPtr<UObject>>(Value, *Fixture->Keys, EPCGAttributeAccessorFlags::AllowBroadcast));
	}

	SECTION("Reading a UObject slot as TObjectPtr<AActor> fails")
	{
		// For Get, ReadDesc is the slot (Object(UObject)) and WriteDesc is the target (Object(AActor)).
		// UObject is not a child of AActor, so IsCompatible returns false and the call must fail.
		const FPCGMetadataAttributeDesc SlotDesc = MakeDesc(EPCGMetadataTypes::Object, UObject::StaticClass());
		TUniquePtr<FPropertyBagAccessorFixture> Fixture = MakeFixture(SlotDesc);
		REQUIRE(Fixture);

		TObjectPtr<AActor> ReadBack = nullptr;
		REQUIRE_FALSE(Fixture->Accessor->Get<TObjectPtr<AActor>>(ReadBack, *Fixture->Keys, EPCGAttributeAccessorFlags::AllowBroadcast));
	}

	SECTION("Setting a Base into a Derived slot fails")
	{
		// Slot expects Derived; reading descriptor is Object(Base). Base is not a child of Derived -> Invalid.
		const FPCGMetadataAttributeDesc SlotDesc = MakeDesc(EPCGMetadataTypes::Object, UPCGPropertyAccessorTestsDerivedObject::StaticClass());
		TUniquePtr<FPropertyBagAccessorFixture> Fixture = MakeFixture(SlotDesc);
		REQUIRE(Fixture);

		TObjectPtr<UPCGPropertyAccessorTestsBaseObject> Value = BaseInstance;
		REQUIRE_FALSE(Fixture->Accessor->Set<TObjectPtr<UPCGPropertyAccessorTestsBaseObject>>(Value, *Fixture->Keys, EPCGAttributeAccessorFlags::AllowBroadcast));
	}

	SECTION("IsCompatible gate also applies to Class slots")
	{
		// Writing a generic UObject class into a slot filtered to AActor-or-derived must fail.
		const FPCGMetadataAttributeDesc SlotDesc = MakeDesc(EPCGMetadataTypes::Class, AActor::StaticClass());
		TUniquePtr<FPropertyBagAccessorFixture> Fixture = MakeFixture(SlotDesc);
		REQUIRE(Fixture);

		TObjectPtr<UClass> Value = UObject::StaticClass();
		REQUIRE_FALSE(Fixture->Accessor->Set<TObjectPtr<UClass>>(Value, *Fixture->Keys, EPCGAttributeAccessorFlags::AllowBroadcast));
	}
}

// Tests the null-pointer guards added for property chains that traverse a null TObjectPtr.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Metadata::Accessors::Properties::NullObjectInChain", "[PCG][Properties]")
{
	constexpr int32 NumInstances = 8;

	const FProperty* InnerObjectProperty = UPCGPropertyAccessorTestsOuterObject::StaticClass()->FindPropertyByName(
		GET_MEMBER_NAME_CHECKED(UPCGPropertyAccessorTestsOuterObject, InnerObject));
	REQUIRE(InnerObjectProperty);
	REQUIRE(CastField<FObjectProperty>(InnerObjectProperty) != nullptr);

	const FProperty* ValueProperty = UPCGPropertyAccessorTestsInnerObject::StaticClass()->FindPropertyByName(
		GET_MEMBER_NAME_CHECKED(UPCGPropertyAccessorTestsInnerObject, Value));
	REQUIRE(ValueProperty);

	TArray<UPCGPropertyAccessorTestsOuterObject*> Instances;
	TArray<void*> InstancesMemory;
	Instances.Reserve(NumInstances);
	InstancesMemory.Reserve(NumInstances);

	for (int32 i = 0; i < NumInstances; ++i)
	{
		UPCGPropertyAccessorTestsOuterObject* Outer = FPCGContext::NewObject_AnyThread<UPCGPropertyAccessorTestsOuterObject>(GetContext());
		REQUIRE(Outer);
		// Sanity: the outer starts with a null inner so resolving the chain produces a null container address.
		REQUIRE(Outer->InnerObject == nullptr);
		Instances.Add(Outer);
		InstancesMemory.Add(Outer);
	}

	// Property chain order is [outer-hop, ..., leaf]: first hop is OuterObject::InnerObject (FObjectProperty),
	// leaf is InnerObject::Value. Constructing through FPCGPropertyGenericAccessor exercises the GetRangeVirtual/
	// SetRangeVirtual code path that contains the new null-pointer guard.
	FPCGPropertyGenericAccessor Accessor(ValueProperty, {InnerObjectProperty});
	FPCGAttributeAccessorKeysGenericPtrs Keys(InstancesMemory);

	SECTION("GetRange with all-null inner objects fails")
	{
		PCGTests::FPCGTestsLogOutputDevice LogCapture(/*bInSuppressErrors=*/true);
		TArray<double> ReadValues;
		ReadValues.SetNum(NumInstances);
		REQUIRE_FALSE(Accessor.GetRange<double>(ReadValues, /*Index=*/0, Keys));
		REQUIRE(LogCapture.NbMessageReceived.Contains(ELogVerbosity::Error));
		REQUIRE(LogCapture.NbMessageReceived[ELogVerbosity::Error] >= 1);
	}

	SECTION("SetRange with all-null inner objects fails")
	{
		PCGTests::FPCGTestsLogOutputDevice LogCapture(/*bInSuppressErrors=*/true);
		TArray<double> ValuesToWrite;
		ValuesToWrite.Init(1.0, NumInstances);
		REQUIRE_FALSE(Accessor.SetRange<double>(ValuesToWrite, /*Index=*/0, Keys));
		REQUIRE(LogCapture.NbMessageReceived.Contains(ELogVerbosity::Error));
		REQUIRE(LogCapture.NbMessageReceived[ELogVerbosity::Error] >= 1);
	}

	SECTION("GetRange with a single null inner object in the middle fails")
	{
		PCGTests::FPCGTestsLogOutputDevice LogCapture(/*bInSuppressErrors=*/true);

		// Populate every instance except one so the range is otherwise fully resolvable.
		constexpr int32 NullIndex = NumInstances / 2;
		for (int32 i = 0; i < NumInstances; ++i)
		{
			if (i != NullIndex)
			{
				Instances[i]->InnerObject = FPCGContext::NewObject_AnyThread<UPCGPropertyAccessorTestsInnerObject>(GetContext());
				REQUIRE(Instances[i]->InnerObject);
			}
		}

		TArray<double> ReadValues;
		ReadValues.SetNum(NumInstances);
		REQUIRE_FALSE(Accessor.GetRange<double>(ReadValues, /*Index=*/0, Keys));
		
		REQUIRE(LogCapture.NbMessageReceived.Contains(ELogVerbosity::Error));
		REQUIRE(LogCapture.NbMessageReceived[ELogVerbosity::Error] >= 1);
	}

	SECTION("SetRange with a single null inner object in the middle fails")
	{
		PCGTests::FPCGTestsLogOutputDevice LogCapture(/*bInSuppressErrors=*/true);
		
		constexpr int32 NullIndex = NumInstances / 2;
		for (int32 i = 0; i < NumInstances; ++i)
		{
			if (i != NullIndex)
			{
				Instances[i]->InnerObject = FPCGContext::NewObject_AnyThread<UPCGPropertyAccessorTestsInnerObject>(GetContext());
				REQUIRE(Instances[i]->InnerObject);
			}
		}

		TArray<double> ValuesToWrite;
		ValuesToWrite.Init(1.0, NumInstances);
		REQUIRE_FALSE(Accessor.SetRange<double>(ValuesToWrite, /*Index=*/0, Keys));
		
		REQUIRE(LogCapture.NbMessageReceived.Contains(ELogVerbosity::Error));
		REQUIRE(LogCapture.NbMessageReceived[ELogVerbosity::Error] >= 1);
	}

	SECTION("Round-trip on a fully resolved chain still works (baseline)")
	{
		PCGTests::FPCGTestsLogOutputDevice LogCapture(/*bInSuppressErrors=*/true);
		
		// Confirms the new guard does not interfere when every chain hop resolves successfully.
		for (int32 i = 0; i < NumInstances; ++i)
		{
			Instances[i]->InnerObject = FPCGContext::NewObject_AnyThread<UPCGPropertyAccessorTestsInnerObject>(GetContext());
			REQUIRE(Instances[i]->InnerObject);
		}

		TArray<double> ValuesToWrite;
		ValuesToWrite.Reserve(NumInstances);
		for (int32 i = 0; i < NumInstances; ++i)
		{
			ValuesToWrite.Add(static_cast<double>(i) + 0.5);
		}

		REQUIRE(Accessor.SetRange<double>(ValuesToWrite, /*Index=*/0, Keys));

		TArray<double> ReadValues;
		ReadValues.SetNum(NumInstances);
		REQUIRE(Accessor.GetRange<double>(ReadValues, /*Index=*/0, Keys));

		CHECK_THAT(ValuesToWrite, Catch::Matchers::RangeEquals(ReadValues, [](const double a, const double b) { return FMath::IsNearlyEqual(a, b); }));
		
		REQUIRE_FALSE(LogCapture.NbMessageReceived.Contains(ELogVerbosity::Error));
	}
}
