// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"

#include "PCGPropertyAccessorTests.generated.h"

UCLASS(BlueprintInternalUseOnly)
class UPCGPropertyAccessorTestsGetterSetter : public UObject
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, BlueprintInternalUseOnly, Category="Test")
	void SetValue(double InValue)
	{
		Value = InValue / 2.0;
	}

	UFUNCTION(BlueprintPure, BlueprintInternalUseOnly, Category="Test")
	double GetValue() const
	{
		return Value * 2.0;
	}

	UPROPERTY(BlueprintInternalUseOnly, BlueprintReadWrite, EditAnywhere, Category="Test", Setter=SetValue, Getter=GetValue)
	double Value = 0.0;
};

/** Base UObject class used by the Object-broadcast tests to exercise same-type and compatible subclass relations. */
UCLASS(BlueprintInternalUseOnly)
class UPCGPropertyAccessorTestsBaseObject : public UObject
{
	GENERATED_BODY()
};

/** Derived from UPCGPropertyAccessorTestsBaseObject: writing a Derived into a Base slot is compatible; the reverse is not. */
UCLASS(BlueprintInternalUseOnly)
class UPCGPropertyAccessorTestsDerivedObject : public UPCGPropertyAccessorTestsBaseObject
{
	GENERATED_BODY()
};

/** Leaf object used by null-object-in-chain tests: the chain reads/writes this Value through the outer's TObjectPtr hop. */
UCLASS(BlueprintInternalUseOnly)
class UPCGPropertyAccessorTestsInnerObject : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	double Value = 0.0;
};

/** Outer object whose InnerObject is intentionally left null in the null-object-in-chain tests, so resolving the chain to Value yields a null container address. */
UCLASS(BlueprintInternalUseOnly)
class UPCGPropertyAccessorTestsOuterObject : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TObjectPtr<UPCGPropertyAccessorTestsInnerObject> InnerObject = nullptr;
};