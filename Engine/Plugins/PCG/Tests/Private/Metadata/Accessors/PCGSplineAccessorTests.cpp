// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "TestHarness.h"

#include <catch2/generators/catch_generators.hpp>

#include "PCGContext.h"
#include "Data/PCGSplineData.h"
#include "Tests/PCGTestsCommon.h"
#include "PCGTestsCommon.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"


TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Metadata::Accessors::SplineAccessors", "[PCG][SplineAccessors]")
{
	const FName DoubleAttributeName = "Double";
	
	constexpr int32 NumPoints = 5;

	TArray<FSplinePoint> SplinePoints;
	for (int32 i = 0; i < NumPoints; ++i)
	{
		SplinePoints.Emplace_GetRef(i, FVector::OneVector * i * 100).Type = ESplinePointType::Linear;
	}

	UPCGSplineData* InputData = CreateData<UPCGSplineData>();
	InputData->Initialize(SplinePoints, /*bClosedLoop=*/false, FTransform::Identity);


	SECTION("Double attribute")
	{
		FPCGMetadataAttribute<double>* DoubleAttribute = InputData->MutableMetadata()->CreateAttribute<double>(DoubleAttributeName, 0.0, false, false);
		
		TUniquePtr<IPCGAttributeAccessor> DoubleAttributeAccessor = PCGAttributeAccessorHelpers::CreateAccessor(InputData, FPCGAttributePropertySelector::CreateAttributeSelector(DoubleAttributeName));
		TUniquePtr<IPCGAttributeAccessorKeys> DoubleAttributeKeys = PCGAttributeAccessorHelpers::CreateKeys(InputData, FPCGAttributePropertySelector::CreateAttributeSelector(DoubleAttributeName));

		REQUIRE(DoubleAttributeAccessor.IsValid());
		REQUIRE(DoubleAttributeKeys.IsValid());

		REQUIRE_EQUAL(InputData->MutableMetadata()->GetItemCountForChild(), 0);
		REQUIRE_EQUAL(DoubleAttributeKeys->GetNum(), 5);

		TStaticArray<double, NumPoints> Values{};
		REQUIRE(DoubleAttributeAccessor->GetRange<double>(Values, 0, *DoubleAttributeKeys));

		REQUIRE(Algo::AllOf(Values, [](const double Value) { return Value == 0.0; }));

		Values = {0, 1, 2, 3, 4};
		REQUIRE(DoubleAttributeAccessor->SetRange<double>(Values, 0, *DoubleAttributeKeys));
		REQUIRE_EQUAL(InputData->MutableMetadata()->GetItemCountForChild(), 5);

		TArray<PCGMetadataEntryKey> EntryKeys = InputData->GetMetadataEntryKeysForSplinePoints();
		for (int32 i = 0; i < NumPoints; ++i)
		{
			REQUIRE_EQUAL(DoubleAttribute->GetValueFromItemKey(EntryKeys[i]), i);
		}
	}

	SECTION("Double data attribute")
	{
		FPCGMetadataAttribute<double>* DoubleDataAttribute = InputData->MutableMetadata()->CreateAttribute<double>(FPCGAttributeIdentifier{DoubleAttributeName, PCGMetadataDomainID::Data}, 50.0, false, false);
		
		TUniquePtr<IPCGAttributeAccessor> DoubleDataAttributeAccessor = PCGAttributeAccessorHelpers::CreateAccessor(InputData, FPCGAttributePropertySelector::CreateAttributeSelector(DoubleAttributeName, TEXT("Data")));
		TUniquePtr<IPCGAttributeAccessorKeys> DoubleDataAttributeKeys = PCGAttributeAccessorHelpers::CreateKeys(InputData, FPCGAttributePropertySelector::CreateAttributeSelector(DoubleAttributeName, TEXT("Data")));
		
		CHECK(DoubleDataAttributeAccessor.IsValid());
		CHECK(DoubleDataAttributeKeys.IsValid());

		REQUIRE_EQUAL(InputData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Data)->GetItemCountForChild(), 0);
		REQUIRE_EQUAL(DoubleDataAttributeKeys->GetNum(), 1);

		double Value{};
		REQUIRE(DoubleDataAttributeAccessor->Get<double>(Value, *DoubleDataAttributeKeys));

		REQUIRE_EQUAL(Value, 50.0);
		
		REQUIRE(DoubleDataAttributeAccessor->Set<double>(150.0, *DoubleDataAttributeKeys));
		REQUIRE_EQUAL(InputData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Data)->GetItemCountForChild(), 1);

		REQUIRE_EQUAL(DoubleDataAttribute->GetValueFromItemKey(PCGFirstEntryKey), 150.0);
	}

	SECTION("Length accessor")
	{
		// Length is read only, accessor cannot be created.
		TUniquePtr<IPCGAttributeAccessor> LengthNonConstAccessor = PCGAttributeAccessorHelpers::CreateAccessor(InputData, FPCGAttributePropertySelector::CreatePropertySelector(TEXT("Length"), TEXT("Data")));
		CHECK(!LengthNonConstAccessor.IsValid());
		
		TUniquePtr<const IPCGAttributeAccessor> LengthAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InputData, FPCGAttributePropertySelector::CreatePropertySelector(TEXT("Length"), TEXT("Data")));
		TUniquePtr<const IPCGAttributeAccessorKeys> LengthKeys = PCGAttributeAccessorHelpers::CreateConstKeys(InputData, FPCGAttributePropertySelector::CreatePropertySelector(TEXT("Length"), TEXT("Data")));
		
		CHECK(LengthAccessor.IsValid());
		CHECK(LengthKeys.IsValid());

		double Value{};
		REQUIRE(LengthAccessor->Get(Value, *LengthKeys));
		// Each point is located at i * 100, in X,Y,Z so each segment is sqrt(3) * 100. There are (NumPoints - 1) segments
		static const double SegmentLength = FMath::Sqrt(3.0) * 100.0;
		static const double ExpectedLength = SegmentLength * (NumPoints - 1);
		REQUIRE(FMath::IsNearlyEqual(Value, ExpectedLength, /*ErrorTolerance=*/1e-4));
	}
	
	SECTION("IsClosed accessor")
    {
    	// IsClosed is read only, accessor cannot be created.
    	TUniquePtr<IPCGAttributeAccessor> IsClosedNonConstAccessor = PCGAttributeAccessorHelpers::CreateAccessor(InputData, FPCGAttributePropertySelector::CreatePropertySelector(TEXT("IsClosed"), TEXT("Data")));
    	CHECK(!IsClosedNonConstAccessor.IsValid());
    	
    	TUniquePtr<const IPCGAttributeAccessor> IsClosedAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InputData, FPCGAttributePropertySelector::CreatePropertySelector(TEXT("IsClosed"), TEXT("Data")));
    	TUniquePtr<const IPCGAttributeAccessorKeys> IsClosedKeys = PCGAttributeAccessorHelpers::CreateConstKeys(InputData, FPCGAttributePropertySelector::CreatePropertySelector(TEXT("IsClosed"), TEXT("Data")));
    	
    	CHECK(IsClosedAccessor.IsValid());
    	CHECK(IsClosedKeys.IsValid());

    	bool Value = true;
    	REQUIRE(IsClosedAccessor->Get(Value, *IsClosedKeys));
    	REQUIRE_FALSE(Value);
    }

	SECTION("Arrive tangent accessor")
	{
		TUniquePtr<IPCGAttributeAccessor> ArriveTangentAccessor = PCGAttributeAccessorHelpers::CreateAccessor(InputData, FPCGAttributePropertySelector::CreatePropertySelector(TEXT("ArriveTangent")));
		TUniquePtr<IPCGAttributeAccessorKeys> ArriveTangentKeys = PCGAttributeAccessorHelpers::CreateKeys(InputData, FPCGAttributePropertySelector::CreatePropertySelector(TEXT("ArriveTangent")));
		
		CHECK(ArriveTangentAccessor.IsValid());
		CHECK(ArriveTangentKeys.IsValid());

		TStaticArray<FVector, NumPoints> Values{};
		REQUIRE(ArriveTangentAccessor->GetRange<FVector>(Values, 0, *ArriveTangentKeys));

		for (int32 i = 0; i < NumPoints; ++i)
		{
			if (i == 0)
			{
				// No arrive tangent on first point
				REQUIRE_EQUAL(Values[0], FVector::ZeroVector);
			}
			else
			{
				REQUIRE_EQUAL(Values[i], FVector::OneVector * 100);
			}
		}

		for (int32 i = 0; i < NumPoints; ++i)
		{
			Values[i] = FVector::OneVector * (i + 1);
		}

		REQUIRE(ArriveTangentAccessor->SetRange<FVector>(Values, 0, *ArriveTangentKeys));

		TArray<FSplinePoint> Points = InputData->GetSplinePoints();
		for (int32 i = 0; i < NumPoints - 1; ++i)
		{
			REQUIRE_EQUAL(Points[i].ArriveTangent , FVector::OneVector * (i + 1));
		}
	}
}
