// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGAsyncWrapper.h"
#include "PCGTestsCommon.h"
#include "Metadata/PCGMetadataAttributeTestsCommonHelper.h"

#include "PCGParamData.h"
#include "Data/PCGPointArrayData.h"
#include "Data/PCGPointData.h"

#include "Containers/StaticArray.h"

#include <catch2/catch_test_macros.hpp>
#include <catch2/benchmark/catch_benchmark.hpp>
#include <catch2/generators/catch_generators_range.hpp>

#include "TestHarness.h"

/**
 * Test suite to verify basic functionality with generic attributes
 * Rely on the TypedAttributeTester that handles anything that is type specific, the rest is templated in
 * the Tester lambda.
 * Some attribute don't have the same initial type with the type we read, so we distinguish between the Attribute type T
 * and the ReadType.
 * 
 * Test routine is pretty standard.
 *   - A point data is created and a generic attribute is added to it.
 *   - Verify that the descriptor of the generic attribute is the one we expect.
 *   - Verify that the default value is set and retrieved correctly
 *   - Generate random values for the generic attribute, set them on the attribute and read them back.
 *   - Verify that the values read back are the same as the one we set.
 *   - Do it again with a child point array data inheriting from the initial one, while modifying even indices
 *     to verify that we can still read old values (odd indices, unchanged), but also the new ones.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeGeneric::Basic", "[PCG][AttributeGeneric]")
{
	constexpr int32 Seed = 42;
	
	auto Tester = [this]<typename T, typename ReadType>(PCGAttributeTestsCommonHelper::TypedAttributeTester<T, ReadType> AttributeTester)
	{
		// Preparing the data
		constexpr int32 NumPoints = 100;
		UPCGPointArrayData* PointArrayData = CreateData<UPCGPointArrayData>();

		PointArrayData->SetNumPoints(NumPoints);

		PointArrayData->AllocateProperties(EPCGPointNativeProperties::MetadataEntry);
		TPCGValueRange<int64> PointArrayDataMetadataEntryRange = PointArrayData->GetMetadataEntryValueRange(/*bAllocate=*/false);

		FPCGMetadataDomain* Domain = PointArrayData->MutableMetadata()->GetDefaultMetadataDomain();
		
		FRandomStream RandomStream(Seed);
		
		FPCGMetadataAttributeBase* GenericAttribute = nullptr;
	
		if (AttributeTester.OverrideDesc.IsSet())
		{
			GenericAttribute = Domain->CreateAttribute(*AttributeTester.OverrideDesc);
			REQUIRE_NOT_EQUAL(GenericAttribute, nullptr);
			GenericAttribute->SetDefaultValue<T>(AttributeTester.DefaultValue);
		}
		else
		{
			GenericAttribute = Domain->CreateAttribute<T>(PCGAttributeTestsCommonHelper::AttributeName, AttributeTester.DefaultValue);
			REQUIRE_NOT_EQUAL(GenericAttribute, nullptr);
		}
		
		SECTION("Expected attribute descriptor")
		{
			REQUIRE_EQUAL(GenericAttribute->GetAttributeDesc(), AttributeTester.ExpectedDesc);
		}
		
		SECTION("Default value is set")
		{
			ReadType ReadDefaultValue{};
			GenericAttribute->GetValuesFromItemKeys<ReadType>({PCGInvalidEntryKey}, MakeArrayView(&ReadDefaultValue, 1));
			REQUIRE(AttributeTester.Verify(AttributeTester.DefaultValue, ReadDefaultValue));
		}
		
		TStaticArray<T, NumPoints> Values{};
		TStaticArray<ReadType, NumPoints> ReadValues;
		
		for (int32 i = 0; i < NumPoints; ++i)
		{
			PointArrayDataMetadataEntryRange[i] = Domain->AddEntry(PointArrayDataMetadataEntryRange[i]);
			AttributeTester.GenerateRandom(RandomStream, Values[i]);
		}

		GenericAttribute->SetValues<T>(PointArrayDataMetadataEntryRange, Values);
		REQUIRE_EQUAL(GenericAttribute->GetNumberOfEntries(), Values.Num());

		SECTION("Read back and verify values")
		{
			GenericAttribute->GetValuesFromItemKeys<ReadType>(PCGValueRangeHelpers::MakeConstValueRange(PointArrayDataMetadataEntryRange), ReadValues);
			CHECK_THAT(Values, Catch::Matchers::RangeEquals(ReadValues, [&AttributeTester](const auto& a, const auto& b){ return AttributeTester.Verify(a, b); } ));
		}

		SECTION("Parenting")
		{
			UPCGPointArrayData* ChildPointArrayData = CreateData<UPCGPointArrayData>();
			ChildPointArrayData->InitializeFromDataWithParams(FPCGInitializeFromDataParams{ PointArrayData });

			FPCGMetadataDomain* ChildDomain = ChildPointArrayData->MutableMetadata()->GetDefaultMetadataDomain();

			TPCGValueRange<int64> ChildPointArrayDataMetadataEntryRange = ChildPointArrayData->GetMetadataEntryValueRange(/*bAllocate=*/false);

			FPCGMetadataAttributeBase* ChildGenericAttribute = ChildDomain->GetMutableAttribute(PCGAttributeTestsCommonHelper::AttributeName);
			REQUIRE_NOT_EQUAL(ChildGenericAttribute, nullptr);
			
			SECTION("Expected attribute descriptor")
			{
				REQUIRE_EQUAL(ChildGenericAttribute->GetAttributeDesc(), AttributeTester.ExpectedDesc);
			}
			
			SECTION("Default value is set")
			{
				ReadType ReadDefaultValue{};
				ChildGenericAttribute->GetValuesFromItemKeys<ReadType>({PCGInvalidEntryKey}, MakeArrayView(&ReadDefaultValue, 1));
				REQUIRE(AttributeTester.Verify(AttributeTester.DefaultValue, ReadDefaultValue));
			}

			// Change values of even indices.
			TStaticArray<T, NumPoints / 2> ChildValues{};
			TStaticArray<PCGMetadataEntryKey, NumPoints / 2> ChildEntryKeys{};
			for (int32 i = 0; i < NumPoints; ++i)
			{
				if (i % 2 == 0)
				{
					AttributeTester.GenerateRandom(RandomStream, Values[i]);

					ChildDomain->InitializeOnSet(ChildPointArrayDataMetadataEntryRange[i]);
					ChildValues[i / 2] = Values[i];
					ChildEntryKeys[i / 2] = ChildPointArrayDataMetadataEntryRange[i];
				}
			}

			ChildGenericAttribute->SetValues<T>(ChildEntryKeys, ChildValues);
			REQUIRE_EQUAL(ChildGenericAttribute->GetNumberOfEntries(), ChildValues.Num());
			REQUIRE_EQUAL(ChildGenericAttribute->GetNumberOfEntriesWithParents(), Values.Num() + ChildValues.Num());

			SECTION("Read back and verify values")
			{
				ChildGenericAttribute->GetValuesFromItemKeys<ReadType>(PCGValueRangeHelpers::MakeConstValueRange(ChildPointArrayDataMetadataEntryRange), ReadValues);
				CHECK_THAT(Values, Catch::Matchers::RangeEquals(ReadValues, [&AttributeTester](const auto& a, const auto& b){ return AttributeTester.Verify(a, b); } ));
			}
		}
	};
	
	SECTION("Float Attribute")
	{
		Tester(PCGAttributeTestsCommonHelper::FloatTester());
	}
	
	SECTION("Double Attribute")
	{
		Tester(PCGAttributeTestsCommonHelper::DoubleTester());
	}
	
	SECTION("Vector Attribute")
	{
		SECTION("As Basic Type")
		{
			Tester(PCGAttributeTestsCommonHelper::VectorTester_Basic());
		}
		
		SECTION("As Struct")
		{
			Tester(PCGAttributeTestsCommonHelper::VectorTester_Struct());
		}
	}
	
	SECTION("String Attribute")
	{
		Tester(PCGAttributeTestsCommonHelper::StringTester());
	}
	
	SECTION("FPCGPoint Attribute")
	{		
		Tester(PCGAttributeTestsCommonHelper::FPCGPointTester());
	}
	
	SECTION("Array of Double Attribute")
	{		
		Tester(PCGAttributeTestsCommonHelper::ArrayDoubleTester());
	}
	
	SECTION("Array of FPCGPoint Attribute")
	{	
		Tester(PCGAttributeTestsCommonHelper::ArrayFPCGPointTester());
	}
	
	SECTION("Set of Double Attribute")
	{		
		Tester(PCGAttributeTestsCommonHelper::SetDoubleTester());
	}
	
	SECTION("Map of String -> Double Attribute")
	{
		Tester(PCGAttributeTestsCommonHelper::MapStringDoubleTester());
	}
	
	SECTION("Enum attribute")
	{
		Tester(PCGAttributeTestsCommonHelper::EnumTester());
	}
}

/**
 * Test that inheritance with Copy also produces the right values.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeGeneric::Copy", "[PCG][AttributeGeneric]")
{
	constexpr int32 Seed = 42;
	
	auto Tester = [this]<typename T, typename ReadType>(PCGAttributeTestsCommonHelper::TypedAttributeTester<T, ReadType> AttributeTester)
	{
		// Preparing the data
		constexpr int32 NumEntries = 10;
		UPCGParamData* ParamData = CreateData<UPCGParamData>();
		FPCGMetadataDomain* Domain = ParamData->MutableMetadata()->GetDefaultMetadataDomain();
		
		FRandomStream RandomStream(Seed);
		
		FPCGMetadataAttributeBase* GenericAttribute = nullptr;
	
		if (AttributeTester.OverrideDesc.IsSet())
		{
			GenericAttribute = Domain->CreateAttribute(*AttributeTester.OverrideDesc);
			REQUIRE_NOT_EQUAL(GenericAttribute, nullptr);
			GenericAttribute->SetDefaultValue<T>(AttributeTester.DefaultValue);
		}
		else
		{
			GenericAttribute = Domain->CreateAttribute<T>(PCGAttributeTestsCommonHelper::AttributeName, AttributeTester.DefaultValue);
			REQUIRE_NOT_EQUAL(GenericAttribute, nullptr);
		}
		
		TStaticArray<T, NumEntries> Values{};
		TStaticArray<ReadType, NumEntries> ReadValues;
		TStaticArray<PCGMetadataEntryKey, NumEntries> EntryKeys;
		
		for (int32 i = 0; i < NumEntries; ++i)
		{
			EntryKeys[i] = Domain->AddEntry();
			AttributeTester.GenerateRandom(RandomStream, Values[i]);
		}

		GenericAttribute->SetValues<T>(EntryKeys, Values);
		REQUIRE_EQUAL(GenericAttribute->GetNumberOfEntries(), Values.Num());

		SECTION("Read back and verify values")
		{
			GenericAttribute->GetValuesFromItemKeys<ReadType>(EntryKeys, ReadValues);
			CHECK_THAT(Values, Catch::Matchers::RangeEquals(ReadValues, [&AttributeTester](const auto& a, const auto& b){ return AttributeTester.Verify(a, b); } ));
		}

		SECTION("Parenting")
		{
			UPCGParamData* ChildParamData = CreateData<UPCGParamData>();
			ChildParamData->MutableMetadata()->Initialize(ParamData->ConstMetadata());

			FPCGMetadataDomain* ChildDomain = ChildParamData->MutableMetadata()->GetDefaultMetadataDomain();

			FPCGMetadataAttributeBase* ChildGenericAttribute = ChildDomain->GetMutableAttribute(PCGAttributeTestsCommonHelper::AttributeName);
			REQUIRE_NOT_EQUAL(ChildGenericAttribute, nullptr);
			
			SECTION("Expected attribute descriptor")
			{
				REQUIRE_EQUAL(ChildGenericAttribute->GetAttributeDesc(), AttributeTester.ExpectedDesc);
			}
			
			SECTION("Default value is set")
			{
				ReadType ReadDefaultValue{};
				ChildGenericAttribute->GetValuesFromItemKeys<ReadType>({PCGInvalidEntryKey}, MakeArrayView(&ReadDefaultValue, 1));
				REQUIRE(AttributeTester.Verify(AttributeTester.DefaultValue, ReadDefaultValue));
			}

			// Change values of even indices.
			TStaticArray<T, NumEntries / 2> ChildValues{};
			TStaticArray<PCGMetadataEntryKey, NumEntries / 2> ChildEntryKeys{};
			for (int32 i = 0; i < NumEntries; ++i)
			{
				if (i % 2 == 0)
				{
					AttributeTester.GenerateRandom(RandomStream, Values[i]);

					ChildDomain->InitializeOnSet(EntryKeys[i]);
					ChildValues[i / 2] = Values[i];
					ChildEntryKeys[i / 2] = EntryKeys[i];
				}
			}

			ChildGenericAttribute->SetValues<T>(ChildEntryKeys, ChildValues);
			REQUIRE_EQUAL(ChildGenericAttribute->GetNumberOfEntries(), Values.Num());
			REQUIRE_EQUAL(ChildGenericAttribute->GetNumberOfEntriesWithParents(), Values.Num());

			SECTION("Read back and verify values")
			{
				ChildGenericAttribute->GetValuesFromItemKeys<ReadType>(EntryKeys, ReadValues);
				CHECK_THAT(Values, Catch::Matchers::RangeEquals(ReadValues, [&AttributeTester](const auto& a, const auto& b){ return AttributeTester.Verify(a, b); } ));
			}
		}
	};
	
	SECTION("Float Attribute")
	{
		Tester(PCGAttributeTestsCommonHelper::FloatTester());
	}
	
	SECTION("Double Attribute")
	{
		Tester(PCGAttributeTestsCommonHelper::DoubleTester());
	}
	
	SECTION("Vector Attribute")
	{
		SECTION("As Basic Type")
		{
			Tester(PCGAttributeTestsCommonHelper::VectorTester_Basic());
		}
		
		SECTION("As Struct")
		{
			Tester(PCGAttributeTestsCommonHelper::VectorTester_Struct());
		}
	}
	
	SECTION("String Attribute")
	{
		Tester(PCGAttributeTestsCommonHelper::StringTester());
	}
	
	SECTION("FPCGPoint Attribute")
	{		
		Tester(PCGAttributeTestsCommonHelper::FPCGPointTester());
	}
	
	SECTION("Array of Double Attribute")
	{		
		Tester(PCGAttributeTestsCommonHelper::ArrayDoubleTester());
	}
	
	SECTION("Array of FPCGPoint Attribute")
	{	
		Tester(PCGAttributeTestsCommonHelper::ArrayFPCGPointTester());
	}
	
	SECTION("Set of Double Attribute")
	{		
		Tester(PCGAttributeTestsCommonHelper::SetDoubleTester());
	}
	
	SECTION("Map of String -> Double Attribute")
	{
		Tester(PCGAttributeTestsCommonHelper::MapStringDoubleTester());
	}
	
	SECTION("Enum attribute")
	{
		Tester(PCGAttributeTestsCommonHelper::EnumTester());
	}
}

/**
 * Test that getting/setting values with the wrong type is not corrupting the data and fail gracefully.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeGeneric::TypeChecking", "[PCG][AttributeGeneric]")
{
	auto Tester = [this]<typename RealType, typename ReadType, typename WriteType>(const RealType& InValue, ReadType DummyRead, WriteType DummyWrite)
	{
		UPCGParamData* ParamData = CreateData<UPCGParamData>();
		FPCGMetadataDomain* Domain = ParamData->MutableMetadata()->GetDefaultMetadataDomain();
		
		FPCGMetadataAttributeBase* GenericAttribute = Domain->CreateAttribute<RealType>(PCGAttributeTestsCommonHelper::AttributeName, InValue);
		
		SECTION("Get Value with wrong type")
		{
			PCGTests::FPCGTestsLogOutputDevice LogCapture(/*bSuppressErrors=*/true);
			GenericAttribute->GetValuesFromItemKeys<ReadType>({PCGInvalidEntryKey}, MakeArrayView(&DummyRead, 1));
			REQUIRE(LogCapture.NbMessageReceived.Contains(ELogVerbosity::Error));
		}
		
		SECTION("Set Value with wrong type")
		{
			PCGTests::FPCGTestsLogOutputDevice LogCapture(/*bSuppressErrors=*/true);
			PCGMetadataEntryKey EntryKey = Domain->AddEntry();
			GenericAttribute->SetValues<WriteType>({EntryKey}, MakeConstArrayView(&DummyWrite, 1));
			REQUIRE(LogCapture.NbMessageReceived.Contains(ELogVerbosity::Error));
			REQUIRE_EQUAL(GenericAttribute->GetNumberOfEntries(), 0);
		}
	};
	
	SECTION("Double with float")
	{
		Tester(double{}, float{}, float{});
	}
	
	SECTION("String with name")
	{
		Tester(FString{}, FName{}, FName{});
	}
	
	SECTION("Array of double with double")
	{
		Tester(TArray<double>{}, double{}, double{});
	}
	
	SECTION("Array of double with array of float")
	{
		Tester(TArray<double>{}, TConstArrayView<float>{}, TArray<float>{});
	}
	
	SECTION("Set of double with double")
	{
		Tester(TSet<double>{}, double{}, double{});
	}
	
	SECTION("Set of double with Set of float")
	{
		Tester(TSet<double>{}, PCG::TScriptSetWrapper<float>{}, TSet<float>{});
	}
	
	SECTION("Map of String/double with String")
	{
		Tester(TMap<FString, double>{}, FString{}, FString{});
	}
	
	SECTION("Map of String/double with Map of String/String")
	{
		Tester(TMap<FString, double>{}, PCG::TScriptMapWrapper<FString, FString>{}, TMap<FString, FString>{});
	}
	
	SECTION("Map of String/double with Map of double/double")
	{
		Tester(TMap<FString, double>{}, PCG::TScriptMapWrapper<double, double>{}, TMap<double, double>{});
	}
	
	SECTION("Map of String/double with Map of double/String")
	{
		Tester(TMap<FString, double>{}, PCG::TScriptMapWrapper<double, FString>{}, TMap<double, FString>{});
	}
	
	SECTION("Map of String/double with Map of Name/float")
	{
		Tester(TMap<FString, double>{}, PCG::TScriptMapWrapper<FName, float>{}, TMap<FName, float>{});
	}
	
	SECTION("PCGPoint with PCGPinProperties")
	{
		Tester(FPCGPoint{}, static_cast<const FPCGPinProperties*>(nullptr), FPCGPinProperties{});
	}
	
	SECTION("Array of PCGPoint with array of PCGPinProperties")
	{
		Tester(TArray<FPCGPoint>{}, TConstArrayView<FPCGPinProperties>{}, TArray<FPCGPinProperties>{});
	}
	
	SECTION("Set of PCGPoint with set of PCGPinProperties")
	{
		Tester(TSet<FPCGPoint>{}, PCG::TScriptSetWrapper<FPCGPinProperties>{}, TSet<FPCGPinProperties>{});
	}
	
	SECTION("Map of String/PCGPoint with map of String/PCGPinProperties")
	{
		Tester(TMap<FString, FPCGPoint>{}, PCG::TScriptMapWrapper<FString, FPCGPinProperties>{}, TMap<FString, FPCGPinProperties>{});
	}
	
	SECTION("EPCGMetadataTypes with EPCGMetadataAttributeContainerTypes")
	{
		Tester(EPCGMetadataTypes{}, EPCGMetadataAttributeContainerTypes{}, EPCGMetadataAttributeContainerTypes{});
	}
	
	SECTION("Array of EPCGMetadataTypes with array of EPCGMetadataAttributeContainerTypes")
	{
		Tester(TArray<EPCGMetadataTypes>{}, TConstArrayView<EPCGMetadataAttributeContainerTypes>{}, TArray<EPCGMetadataAttributeContainerTypes>{});
	}
	
	SECTION("Set of EPCGMetadataTypes with set of EPCGMetadataAttributeContainerTypes")
	{
		Tester(TSet<EPCGMetadataTypes>{}, PCG::TScriptSetWrapper<EPCGMetadataAttributeContainerTypes>{}, TSet<EPCGMetadataAttributeContainerTypes>{});
	}
	
	SECTION("Map of FString/EPCGMetadataTypes with map of FString/EPCGMetadataAttributeContainerTypes")
	{
		Tester(TMap<FString, EPCGMetadataTypes>{}, PCG::TScriptMapWrapper<FString, EPCGMetadataAttributeContainerTypes>{}, TMap<FString, EPCGMetadataAttributeContainerTypes>{});
	}
	
	SECTION("Map of EPCGMetadataTypes/FString with map of EPCGMetadataAttributeContainerTypes/FString")
	{
		Tester(TMap<EPCGMetadataTypes, FString>{}, PCG::TScriptMapWrapper<EPCGMetadataAttributeContainerTypes, FString>{}, TMap<EPCGMetadataAttributeContainerTypes, FString>{});
	}
}

TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeGeneric::CompressedData", "[PCG][AttributeGeneric]")
{
	// Preparing the data
	constexpr int32 NumPoints = 100;
	UPCGPointArrayData* PointArrayData = CreateData<UPCGPointArrayData>();

	PointArrayData->SetNumPoints(NumPoints);

	PointArrayData->AllocateProperties(EPCGPointNativeProperties::MetadataEntry);
	TPCGValueRange<int64> PointArrayDataMetadataEntryRange = PointArrayData->GetMetadataEntryValueRange(/*bAllocate=*/false);

	FPCGMetadataDomain* Domain = PointArrayData->MutableMetadata()->GetDefaultMetadataDomain();

	// Have base values for SoftPath and SoftClass
	const FString BasePathFormat = TEXT("/PCGTests/Item_{0}.Item_{0}");
	TStaticArray<const UClass*, 10> UClassesPool =
	{
		UPCGData::StaticClass(),
		UPCGParamData::StaticClass(),
		UPCGBasePointData::StaticClass(),
		UPCGPointData::StaticClass(),
		UPCGPointArrayData::StaticClass(),
		UObject::StaticClass(),
		UClass::StaticClass(),
		UPCGSettings::StaticClass(),
		UPCGSettingsInterface::StaticClass(),
		UPCGSettingsInstance::StaticClass(),
	};
	
	// Initialize the default value to a value we will add later on.
	FPCGMetadataAttributeBase* GenericAttribute = Domain->CreateAttribute<FString>(PCGAttributeTestsCommonHelper::AttributeName, TEXT("Value: 0"));
	FPCGMetadataAttributeBase* GenericAttributePath = Domain->CreateAttribute<FSoftObjectPath>(PCGAttributeTestsCommonHelper::AttributeName2, FSoftObjectPath(FString::Format(*BasePathFormat, { 0 })));
	FPCGMetadataAttributeBase* GenericAttributeClass = Domain->CreateAttribute<FSoftClassPath>(PCGAttributeTestsCommonHelper::AttributeName3, UClassesPool[0]);
	REQUIRE_NOT_EQUAL(GenericAttribute, nullptr);
	REQUIRE_NOT_EQUAL(GenericAttributePath, nullptr);
	REQUIRE_NOT_EQUAL(GenericAttributeClass, nullptr);
	
	// First set half the points with different values
	TStaticArray<FString, NumPoints> Values{};
	TStaticArray<FSoftObjectPath, NumPoints> PathValues{};
	TStaticArray<FSoftClassPath, NumPoints> ClassValues{};
	TStaticArray<PCGMetadataEntryKey, NumPoints/2> EntryKeys{};


	for (int32 i = 0; i < NumPoints/2; ++i)
	{
		Domain->InitializeOnSet(PointArrayDataMetadataEntryRange[i]);
		EntryKeys[i] = PointArrayDataMetadataEntryRange[i];
		Values[i] = FString::Printf(TEXT("Value: %d"), i % 5);
		PathValues[i] = FSoftObjectPath(FString::Format(*BasePathFormat, { i % 5 }));
		ClassValues[i] = UClassesPool[i % 5];
	}
	
	// Set the values
	GenericAttribute->SetValues<FString>(EntryKeys, MakeArrayView(Values.GetData(), NumPoints / 2));
	GenericAttributePath->SetValues<FSoftObjectPath>(EntryKeys, MakeArrayView(PathValues.GetData(), NumPoints / 2));
	GenericAttributeClass->SetValues<FSoftClassPath>(EntryKeys, MakeArrayView(ClassValues.GetData(), NumPoints / 2));
	
	// Verify that we have only 4 Value keys (the fifth one is the default value).
	REQUIRE_EQUAL(GenericAttribute->GetValueKeyOffsetForChild(), 4);
	REQUIRE_EQUAL(GenericAttributePath->GetValueKeyOffsetForChild(), 4);
	REQUIRE_EQUAL(GenericAttributeClass->GetValueKeyOffsetForChild(), 4);
	
	// Then add a mix of existing strings and new strings, to test that we only add what does not exist
	for (int32 i = NumPoints/2; i < NumPoints; ++i)
	{
		const int32 EntryKeyIndex = i - (NumPoints/2);
		Domain->InitializeOnSet(PointArrayDataMetadataEntryRange[i]);
		EntryKeys[EntryKeyIndex] = PointArrayDataMetadataEntryRange[i];
		if (EntryKeyIndex < 5)
		{
			Values[i] = FString::Printf(TEXT("Value: %d"), EntryKeyIndex % 5);
			PathValues[i] = FSoftObjectPath(FString::Format(*BasePathFormat, { i % 5 }));
			ClassValues[i] = UClassesPool[i % 5];
		}
		else
		{
			Values[i] = FString::Printf(TEXT("New Value: %d"), EntryKeyIndex % 5);
			PathValues[i] = FSoftObjectPath(FString::Format(*BasePathFormat, { 5 + i % 5 }));
			ClassValues[i] = UClassesPool[5 + i % 5];
		}
	}
	
	// Set the values
	GenericAttribute->SetValues<FString>(EntryKeys, MakeArrayView(Values.GetData() + (NumPoints / 2), NumPoints / 2));
	GenericAttributePath->SetValues<FSoftObjectPath>(EntryKeys, MakeArrayView(PathValues.GetData() + (NumPoints / 2), NumPoints / 2));
	GenericAttributeClass->SetValues<FSoftClassPath>(EntryKeys, MakeArrayView(ClassValues.GetData() + (NumPoints / 2), NumPoints / 2));
	
	// Verify that we have only 9 Value keys (the tenth one is the default value)
	REQUIRE_EQUAL(GenericAttribute->GetValueKeyOffsetForChild(), 9);
	REQUIRE_EQUAL(GenericAttributePath->GetValueKeyOffsetForChild(), 9);
	REQUIRE_EQUAL(GenericAttributeClass->GetValueKeyOffsetForChild(), 9);
	
	// Verify that all the values can be retrieved.
	TStaticArray<const FString*, NumPoints> ReadValues{};
	TStaticArray<const FSoftObjectPath*, NumPoints> ReadValuesPath{};
	TStaticArray<const FSoftClassPath*, NumPoints> ReadValuesClass{};

	GenericAttribute->GetValuesFromItemKeys<const FString*>(PCGValueRangeHelpers::MakeConstValueRange(PointArrayDataMetadataEntryRange), ReadValues);
	GenericAttributePath->GetValuesFromItemKeys<const FSoftObjectPath*>(PCGValueRangeHelpers::MakeConstValueRange(PointArrayDataMetadataEntryRange), ReadValuesPath);
	GenericAttributeClass->GetValuesFromItemKeys<const FSoftClassPath*>(PCGValueRangeHelpers::MakeConstValueRange(PointArrayDataMetadataEntryRange), ReadValuesClass);
	
	CHECK_THAT(Values, Catch::Matchers::RangeEquals(ReadValues, [](const FString& a, const FString* b){ return a == *b; } ));
	CHECK_THAT(PathValues, Catch::Matchers::RangeEquals(ReadValuesPath, [](const FSoftObjectPath& a, const FSoftObjectPath* b){ return a == *b; } ));
	CHECK_THAT(ClassValues, Catch::Matchers::RangeEquals(ReadValuesClass, [](const FSoftClassPath& a, const FSoftClassPath* b){ return a == *b; } ));
}

TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeGeneric::MultiThreadedWrite", "[PCG][AttributeGeneric]")
{
	auto Tester = [this]<typename T, typename ReadType>(PCGAttributeTestsCommonHelper::TypedAttributeTester<T, ReadType> AttributeTester, bool bLockless)
	{
		// Preparing the data
		constexpr int32 NumEntries = 10000;
		constexpr int32 Chunks = 100;
		constexpr int32 ChunksPerThread = 10;
		constexpr int32 ElementsPerThreads = Chunks * ChunksPerThread;
		constexpr int32 NumThreads = NumEntries / ElementsPerThreads;
		
		constexpr int32 Seed = 42;
		UPCGParamData* ParamData = CreateData<UPCGParamData>();
		FPCGMetadataDomain* Domain = ParamData->MutableMetadata()->GetDefaultMetadataDomain();
	
		FRandomStream RandomStream(Seed);
	
		FPCGMetadataAttributeBase* GenericAttribute = Domain->CreateAttribute<T>(PCGAttributeTestsCommonHelper::AttributeName, AttributeTester.DefaultValue);
		REQUIRE_NOT_EQUAL(GenericAttribute, nullptr);
	
		TStaticArray<T, NumEntries> Values{};
		TStaticArray<PCGMetadataEntryKey, NumEntries> EntryKeys;
		TStaticArray<PCGMetadataEntryKey*, NumEntries> EntryKeysPtr;
		int32 StartIndex = INDEX_NONE;
		
		for (int32 i = 0; i < NumEntries; ++i)
		{
			EntryKeys[i] = Domain->AddEntry();
			EntryKeysPtr[i] = &EntryKeys[i];
			AttributeTester.GenerateRandom(RandomStream, Values[i]);
		}
		
		if (bLockless)
		{
			StartIndex = GenericAttribute->PreallocateValues(EntryKeysPtr, /*bLockless=*/true);
		}
	
		bool bHasRun = false;
		bool bHasRunOnMultiThread = false;
		PCGTests::Async::AsyncRun(NumThreads, [&Values, &EntryKeysPtr, GenericAttribute, bLockless, StartIndex](int32 ThreadIndex)
		{
			for (int32 j = 0; j < ChunksPerThread; ++j)
			{
				const int32 ValuesIndex = ThreadIndex * ElementsPerThreads + j * Chunks;
				TArrayView<PCGMetadataEntryKey*> EntryKeysView = MakeArrayView(EntryKeysPtr.GetData() + ValuesIndex, Chunks);
				TArrayView<T> ValuesView = MakeArrayView(Values.GetData() + ValuesIndex, Chunks);
			
				if (bLockless)
				{
					GenericAttribute->SetValues_TryLockless<T>(EntryKeysView, ValuesView, StartIndex + ValuesIndex);
				}
				else
				{
					GenericAttribute->SetValues<T>(EntryKeysView, ValuesView);
				}
			}
		}, bHasRun, bHasRunOnMultiThread);
		
		REQUIRE(bHasRun);
		
		// Making sure we had concurrency.
		CHECK(bHasRunOnMultiThread);
		
		REQUIRE_EQUAL(GenericAttribute->GetNumberOfEntries(), Values.Num());

		SECTION("Read back and verify values")
		{
			TStaticArray<ReadType, NumEntries> ReadValues;
			GenericAttribute->GetValuesFromItemKeys<ReadType>(EntryKeys, ReadValues);
			CHECK_THAT(Values, Catch::Matchers::RangeEquals(ReadValues, [&AttributeTester](const auto& a, const auto& b){ return AttributeTester.Verify(a, b); } ));
		}
	};

	SECTION("Float Attribute")
	{
		SECTION("With lock")
		{
			Tester(PCGAttributeTestsCommonHelper::FloatTester(), /*bLockless=*/false);
		}
		
		SECTION("Without lock")
		{
			Tester(PCGAttributeTestsCommonHelper::FloatTester(), /*bLockless=*/true);
		}
	}
	
	SECTION("Double Attribute")
	{
		SECTION("With lock")
		{
			Tester(PCGAttributeTestsCommonHelper::DoubleTester(), /*bLockless=*/false);
		}
		
		SECTION("Without lock")
		{
			Tester(PCGAttributeTestsCommonHelper::DoubleTester(), /*bLockless=*/true);
		}
	}
	
	SECTION("Vector Attribute")
	{
		SECTION("With lock")
		{
			Tester(PCGAttributeTestsCommonHelper::VectorTester_Basic(), /*bLockless=*/false);
		}
		
		SECTION("Without lock")
		{
			Tester(PCGAttributeTestsCommonHelper::VectorTester_Basic(), /*bLockless=*/true);
		}
	}
	
	SECTION("String Attribute")
	{
		SECTION("With lock")
		{
			Tester(PCGAttributeTestsCommonHelper::StringTester(), /*bLockless=*/false);
		}
		
		SECTION("Without lock")
		{
			Tester(PCGAttributeTestsCommonHelper::StringTester(), /*bLockless=*/true);
		}
	}
	
	SECTION("FPCGPoint Attribute")
	{		
		SECTION("With lock")
		{
			Tester(PCGAttributeTestsCommonHelper::FPCGPointTester(), /*bLockless=*/false);
		}
		
		SECTION("Without lock")
		{
			Tester(PCGAttributeTestsCommonHelper::FPCGPointTester(), /*bLockless=*/true);
		}
	}
	
	SECTION("Array of Double Attribute")
	{		
		SECTION("With lock")
		{
			Tester(PCGAttributeTestsCommonHelper::ArrayDoubleTester(), /*bLockless=*/false);
		}
		
		SECTION("Without lock")
		{
			Tester(PCGAttributeTestsCommonHelper::ArrayDoubleTester(), /*bLockless=*/true);
		}
	}
	
	SECTION("Array of FPCGPoint Attribute")
	{	
		SECTION("With lock")
		{
			Tester(PCGAttributeTestsCommonHelper::ArrayFPCGPointTester(), /*bLockless=*/false);
		}
		
		SECTION("Without lock")
		{
			Tester(PCGAttributeTestsCommonHelper::ArrayFPCGPointTester(), /*bLockless=*/true);
		}
	}
	
	SECTION("Set of Double Attribute")
	{		
		SECTION("With lock")
		{
			Tester(PCGAttributeTestsCommonHelper::SetDoubleTester(), /*bLockless=*/false);
		}
		
		SECTION("Without lock")
		{
			Tester(PCGAttributeTestsCommonHelper::SetDoubleTester(), /*bLockless=*/true);
		}
	}
	
	SECTION("Map of String -> Double Attribute")
	{
		SECTION("With lock")
		{
			Tester(PCGAttributeTestsCommonHelper::MapStringDoubleTester(), /*bLockless=*/false);
		}
		
		SECTION("Without lock")
		{
			Tester(PCGAttributeTestsCommonHelper::MapStringDoubleTester(), /*bLockless=*/true);
		}
	}
	
	SECTION("Enum attribute")
	{
		SECTION("With lock")
		{
			Tester(PCGAttributeTestsCommonHelper::EnumTester(), /*bLockless=*/false);
		}
		
		SECTION("Without lock")
		{
			Tester(PCGAttributeTestsCommonHelper::EnumTester(), /*bLockless=*/true);
		}
	}
}

/**
 * Test that SetValue(ItemKey, this, EntryKey) works correctly when the source attribute is the same
 * as the destination. This exercises the fix where SrcPtr obtained from GetReadAddressFromEntryKey_Unsafe
 * could become dangling after AddValue() triggers a reallocation of the internal Values buffer.
 * We use enough self-copies to force multiple reallocations of the underlying FScriptArray.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeGeneric::SetValueSelfReference", "[PCG][AttributeGeneric]")
{
	auto Tester = [this]<typename T, typename ReadType>(PCGAttributeTestsCommonHelper::TypedAttributeTester<T, ReadType> AttributeTester)
	{
		constexpr int32 NumInitialEntries = 10;
		// Enough self-copies to trigger multiple reallocations of the underlying FScriptArray.
		constexpr int32 NumFullSelfCopies = 20;

		UPCGParamData* ParamData = CreateData<UPCGParamData>();
		FPCGMetadataDomain* Domain = ParamData->MutableMetadata()->GetDefaultMetadataDomain();

		FRandomStream RandomStream(42);

		FPCGMetadataAttributeBase* GenericAttribute = Domain->CreateAttribute<T>(PCGAttributeTestsCommonHelper::AttributeName, AttributeTester.DefaultValue);
		REQUIRE_NOT_EQUAL(GenericAttribute, nullptr);

		// Populate initial entries with random values.
		TStaticArray<T, NumInitialEntries> InitialValues{};
		TStaticArray<PCGMetadataEntryKey, NumInitialEntries * (NumFullSelfCopies + 1)> AllEntryKeys;

		for (int32 i = 0; i < NumInitialEntries; ++i)
		{
			AllEntryKeys[i] = Domain->AddEntry();
			AttributeTester.GenerateRandom(RandomStream, InitialValues[i]);
		}

		GenericAttribute->SetValues<T>(MakeArrayView(AllEntryKeys.GetData(), NumInitialEntries), InitialValues);
		REQUIRE_EQUAL(GenericAttribute->GetNumberOfEntries(), NumInitialEntries);

		// Copying into itself
		for (int32 i = 0; i < NumInitialEntries * NumFullSelfCopies; ++i)
		{
			AllEntryKeys[NumInitialEntries + i] = Domain->AddEntry();
			GenericAttribute->SetValue(AllEntryKeys[NumInitialEntries + i], GenericAttribute, AllEntryKeys[i % NumInitialEntries]);
		}
		
		REQUIRE_EQUAL(GenericAttribute->GetNumberOfEntries(), NumInitialEntries * (NumFullSelfCopies + 1));
		TStaticArray<ReadType, NumInitialEntries> ReadValues{};
		for (int32 i = 0; i < NumFullSelfCopies + 1; ++i)
		{
			const int32 StartIndex = i * NumInitialEntries;
			
			GenericAttribute->GetValuesFromItemKeys<ReadType>(MakeArrayView(AllEntryKeys.GetData() + StartIndex, NumInitialEntries), ReadValues);
			CHECK_THAT(InitialValues, Catch::Matchers::RangeEquals(ReadValues, [&AttributeTester](const auto& a, const auto& b){ return AttributeTester.Verify(a, b); } ));
		}
	};

	SECTION("Double (POD type)")
	{
		Tester(PCGAttributeTestsCommonHelper::DoubleTester());
	}

	SECTION("String (non-POD, compressed)")
	{
		Tester(PCGAttributeTestsCommonHelper::StringTester());
	}

	SECTION("Vector")
	{
		Tester(PCGAttributeTestsCommonHelper::VectorTester_Basic());
	}

	SECTION("FPCGPoint (Struct)")
	{
		Tester(PCGAttributeTestsCommonHelper::FPCGPointTester());
	}
}

/**
 * Test that reading continuous values on GetValues is working as intended, even with parenting.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeGeneric::GetValuesContinuous", "[PCG][AttributeGeneric]")
{
	auto Tester = [this]<typename T, typename ReadType>(PCGAttributeTestsCommonHelper::TypedAttributeTester<T, ReadType> AttributeTester)
	{
		// Preparing the data
		constexpr int32 NumPoints = 100;
		UPCGPointArrayData* PointArrayData = CreateData<UPCGPointArrayData>();

		PointArrayData->SetNumPoints(NumPoints);

		PointArrayData->AllocateProperties(EPCGPointNativeProperties::MetadataEntry);
		TPCGValueRange<int64> PointArrayDataMetadataEntryRange = PointArrayData->GetMetadataEntryValueRange(/*bAllocate=*/false);

		FPCGMetadataDomain* Domain = PointArrayData->MutableMetadata()->GetDefaultMetadataDomain();

		FRandomStream RandomStream(42);

		FPCGMetadataAttributeBase* GenericAttribute = nullptr;

		GenericAttribute = Domain->CreateAttribute<T>(PCGAttributeTestsCommonHelper::AttributeName, AttributeTester.DefaultValue);
		REQUIRE_NOT_EQUAL(GenericAttribute, nullptr);

		TStaticArray<T, NumPoints> Values{};
		TStaticArray<ReadType, NumPoints> ReadValues;

		for (int32 i = 0; i < NumPoints; ++i)
		{
			PointArrayDataMetadataEntryRange[i] = Domain->AddEntry(PointArrayDataMetadataEntryRange[i]);
			AttributeTester.GenerateRandom(RandomStream, Values[i]);
		}

		GenericAttribute->SetValues<T>(PointArrayDataMetadataEntryRange, Values);
		REQUIRE_EQUAL(GenericAttribute->GetNumberOfEntries(), Values.Num());

		SECTION("Read back and verify values")
		{
			GenericAttribute->GetValuesFromItemKeys<ReadType>(PCGValueRangeHelpers::MakeConstValueRange(PointArrayDataMetadataEntryRange), ReadValues);
			CHECK_THAT(Values, Catch::Matchers::RangeEquals(ReadValues, [&AttributeTester](const auto& a, const auto& b) { return AttributeTester.Verify(a, b); }));
		}

		SECTION("Parenting with more points")
		{
			UPCGPointArrayData* ChildPointArrayData = CreateData<UPCGPointArrayData>();
			ChildPointArrayData->InitializeFromDataWithParams(FPCGInitializeFromDataParams{ PointArrayData });

			FPCGMetadataDomain* ChildDomain = ChildPointArrayData->MutableMetadata()->GetDefaultMetadataDomain();

			ChildPointArrayData->SetNumPoints(2 * NumPoints);

			TPCGValueRange<int64> ChildPointArrayDataMetadataEntryRange = ChildPointArrayData->GetMetadataEntryValueRange(/*bAllocate=*/false);

			FPCGMetadataAttributeBase* ChildGenericAttribute = ChildDomain->GetMutableAttribute(PCGAttributeTestsCommonHelper::AttributeName);
			REQUIRE_NOT_EQUAL(ChildGenericAttribute, nullptr);

			// Add more values
			TStaticArray<T, NumPoints> ChildValues{};
			TStaticArray<PCGMetadataEntryKey, NumPoints> ChildEntryKeys{};
			for (int32 i = 0; i < NumPoints; ++i)
			{
				ChildEntryKeys[i] = ChildDomain->AddEntry();
				ChildPointArrayDataMetadataEntryRange[i + NumPoints] = ChildEntryKeys[i];
				AttributeTester.GenerateRandom(RandomStream, ChildValues[i]);
			}

			ChildGenericAttribute->SetValues<T>(ChildEntryKeys, ChildValues);
			REQUIRE_EQUAL(ChildGenericAttribute->GetNumberOfEntries(), NumPoints);
			REQUIRE_EQUAL(ChildGenericAttribute->GetNumberOfEntriesWithParents(), 2 * NumPoints);

			SECTION("Read back and verify values")
			{
				TStaticArray<ReadType, 2*NumPoints> ChildReadValues;
				ChildGenericAttribute->GetValuesFromItemKeys<ReadType>(PCGValueRangeHelpers::MakeConstValueRange(ChildPointArrayDataMetadataEntryRange), ChildReadValues);
				CHECK_THAT(Values, Catch::Matchers::RangeEquals(MakeArrayView(ChildReadValues.GetData(), NumPoints), [&AttributeTester](const auto& a, const auto& b) { return AttributeTester.Verify(a, b); }));
				CHECK_THAT(ChildValues, Catch::Matchers::RangeEquals(MakeArrayView(ChildReadValues.GetData() + NumPoints, NumPoints), [&AttributeTester](const auto& a, const auto& b) { return AttributeTester.Verify(a, b); }));
			}
		}
	};

	SECTION("Float Attribute")
	{
		Tester(PCGAttributeTestsCommonHelper::FloatTester());
	}

	SECTION("Double Attribute")
	{
		Tester(PCGAttributeTestsCommonHelper::DoubleTester());
	}

	SECTION("As Basic Type")
	{
		Tester(PCGAttributeTestsCommonHelper::VectorTester_Basic());
	}

	SECTION("String Attribute")
	{
		Tester(PCGAttributeTestsCommonHelper::StringTester());
	}

	SECTION("FPCGPoint Attribute")
	{
		Tester(PCGAttributeTestsCommonHelper::FPCGPointTester());
	}

	SECTION("Array of Double Attribute")
	{
		Tester(PCGAttributeTestsCommonHelper::ArrayDoubleTester());
	}

	SECTION("Array of FPCGPoint Attribute")
	{
		Tester(PCGAttributeTestsCommonHelper::ArrayFPCGPointTester());
	}

	SECTION("Set of Double Attribute")
	{
		Tester(PCGAttributeTestsCommonHelper::SetDoubleTester());
	}

	SECTION("Map of String -> Double Attribute")
	{
		Tester(PCGAttributeTestsCommonHelper::MapStringDoubleTester());
	}

	SECTION("Enum attribute")
	{
		Tester(PCGAttributeTestsCommonHelper::EnumTester());
	}
}