// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGTestsCommon.h"

#include "PCGParamData.h"
#include "Data/PCGPointArrayData.h"

#include "Containers/StaticArray.h"

#include <catch2/catch_test_macros.hpp>
#include <catch2/matchers/catch_matchers_all.hpp>

#include "PCGAsyncWrapper.h"
#include "TestHarness.h"
#include "Async/Async.h"
#include "HAL/Event.h"

namespace PCGAttributeTypedTests
{
	static const FName AttributeName = "TypedAttr";
	constexpr int32 Seed = 42;
	constexpr int32 NumPoints = 100;
	constexpr int32 NumEntries = 10;

	/**
	 * Helper to generate a random value of type T using a FRandomStream.
	 * There are specializations below for each supported type.
	 */
	template <typename T>
	T GenerateRandom(FRandomStream& RandomStream);

	template<> float GenerateRandom<float>(FRandomStream& RandomStream) { return RandomStream.FRand(); }
	template<> double GenerateRandom<double>(FRandomStream& RandomStream) { return RandomStream.FRand(); }
	template<> int32 GenerateRandom<int32>(FRandomStream& RandomStream) { return RandomStream.RandRange(-1000, 1000); }
	template<> int64 GenerateRandom<int64>(FRandomStream& RandomStream) { return static_cast<int64>(RandomStream.RandRange(-100000, 100000)); }
	template<> bool GenerateRandom<bool>(FRandomStream& RandomStream) { return RandomStream.RandRange(0, 1) == 1; }

	template<> FVector2D GenerateRandom<FVector2D>(FRandomStream& RandomStream)
	{
		return FVector2D(RandomStream.FRand(), RandomStream.FRand());
	}

	template<> FVector GenerateRandom<FVector>(FRandomStream& RandomStream)
	{
		return RandomStream.VRand();
	}

	template<> FVector4 GenerateRandom<FVector4>(FRandomStream& RandomStream)
	{
		return FVector4(RandomStream.FRand(), RandomStream.FRand(), RandomStream.FRand(), RandomStream.FRand());
	}

	template<> FQuat GenerateRandom<FQuat>(FRandomStream& RandomStream)
	{
		return FQuat(RandomStream.VRand(), RandomStream.FRand() * UE_PI);
	}

	template<> FTransform GenerateRandom<FTransform>(FRandomStream& RandomStream)
	{
		return FTransform(
			FQuat(RandomStream.VRand(), RandomStream.FRand() * UE_PI),
			RandomStream.VRand() * 100.0,
			FVector(RandomStream.FRand() + 0.5, RandomStream.FRand() + 0.5, RandomStream.FRand() + 0.5));
	}

	template<> FRotator GenerateRandom<FRotator>(FRandomStream& RandomStream)
	{
		return FRotator(RandomStream.FRand() * 360.0 - 180.0, RandomStream.FRand() * 360.0 - 180.0, RandomStream.FRand() * 360.0 - 180.0);
	}

	template<> FString GenerateRandom<FString>(FRandomStream& RandomStream)
	{
		static int Counter = 0;
		return FString::Printf(TEXT("TestString_%d_%d"), Counter++, RandomStream.RandRange(0, 1000));
	}

	template<> FName GenerateRandom<FName>(FRandomStream& RandomStream)
	{
		static int Counter = 0;
		return FName(*FString::Printf(TEXT("TestName_%d_%d"), Counter++, RandomStream.RandRange(0, 1000)));
	}

	template<> FSoftObjectPath GenerateRandom<FSoftObjectPath>(FRandomStream& RandomStream)
	{
		static int Counter = 0;
		return FSoftObjectPath(FString::Printf(TEXT("/Game/Test/Object_%d_%d"), Counter++, RandomStream.RandRange(0, 1000)));
	}

	template<> FSoftClassPath GenerateRandom<FSoftClassPath>(FRandomStream& RandomStream)
	{
		static int Counter = 0;
		return FSoftClassPath(FString::Printf(TEXT("/Game/Test/Class_%d_%d"), Counter++, RandomStream.RandRange(0, 1000)));
	}

	/** Default values per type. */
	template <typename T> T GetDefaultValue();
	template<> float GetDefaultValue<float>() { return 5.0f; }
	template<> double GetDefaultValue<double>() { return 5.0; }
	template<> int32 GetDefaultValue<int32>() { return 42; }
	template<> int64 GetDefaultValue<int64>() { return 42LL; }
	template<> bool GetDefaultValue<bool>() { return true; }
	template<> FVector2D GetDefaultValue<FVector2D>() { return FVector2D(1.0, 2.0); }
	template<> FVector GetDefaultValue<FVector>() { return FVector(1.0, 2.0, 3.0); }
	template<> FVector4 GetDefaultValue<FVector4>() { return FVector4(1.0, 2.0, 3.0, 4.0); }
	template<> FQuat GetDefaultValue<FQuat>() { return FQuat(0.0, 0.0, 0.7071, 0.7071); }
	template<> FTransform GetDefaultValue<FTransform>() { return FTransform(FQuat::Identity, FVector(5.0, 5.0, 5.0), FVector::OneVector); }
	template<> FRotator GetDefaultValue<FRotator>() { return FRotator(45.0, 90.0, 0.0); }
	template<> FString GetDefaultValue<FString>() { return TEXT("DefaultValue"); }
	template<> FName GetDefaultValue<FName>() { return FName(TEXT("DefaultName")); }
	template<> FSoftObjectPath GetDefaultValue<FSoftObjectPath>() { return FSoftObjectPath(TEXT("/Game/Default/Object")); }
	template<> FSoftClassPath GetDefaultValue<FSoftClassPath>() { return FSoftClassPath(TEXT("/Game/Default/Class")); }

	/** Alternative default values for testing default value changes. */
	template <typename T> T GetAlternativeDefaultValue();
	template<> float GetAlternativeDefaultValue<float>() { return 99.0f; }
	template<> double GetAlternativeDefaultValue<double>() { return 99.0; }
	template<> int32 GetAlternativeDefaultValue<int32>() { return -7; }
	template<> int64 GetAlternativeDefaultValue<int64>() { return -7LL; }
	template<> bool GetAlternativeDefaultValue<bool>() { return false; }
	template<> FVector2D GetAlternativeDefaultValue<FVector2D>() { return FVector2D(99.0, 99.0); }
	template<> FVector GetAlternativeDefaultValue<FVector>() { return FVector(99.0, 99.0, 99.0); }
	template<> FVector4 GetAlternativeDefaultValue<FVector4>() { return FVector4(99.0, 99.0, 99.0, 99.0); }
	template<> FQuat GetAlternativeDefaultValue<FQuat>() { return FQuat::Identity; }
	template<> FTransform GetAlternativeDefaultValue<FTransform>() { return FTransform::Identity; }
	template<> FRotator GetAlternativeDefaultValue<FRotator>() { return FRotator::ZeroRotator; }
	template<> FString GetAlternativeDefaultValue<FString>() { return TEXT("AlternativeDefault"); }
	template<> FName GetAlternativeDefaultValue<FName>() { return FName(TEXT("AlternativeName")); }
	template<> FSoftObjectPath GetAlternativeDefaultValue<FSoftObjectPath>() { return FSoftObjectPath(TEXT("/Game/Alternative/Object")); }
	template<> FSoftClassPath GetAlternativeDefaultValue<FSoftClassPath>() { return FSoftClassPath(TEXT("/Game/Alternative/Class")); }

	/** Expected EPCGMetadataTypes enum for each type. */
	template <typename T> EPCGMetadataTypes GetExpectedMetadataType();
	template<> EPCGMetadataTypes GetExpectedMetadataType<float>() { return EPCGMetadataTypes::Float; }
	template<> EPCGMetadataTypes GetExpectedMetadataType<double>() { return EPCGMetadataTypes::Double; }
	template<> EPCGMetadataTypes GetExpectedMetadataType<int32>() { return EPCGMetadataTypes::Integer32; }
	template<> EPCGMetadataTypes GetExpectedMetadataType<int64>() { return EPCGMetadataTypes::Integer64; }
	template<> EPCGMetadataTypes GetExpectedMetadataType<bool>() { return EPCGMetadataTypes::Boolean; }
	template<> EPCGMetadataTypes GetExpectedMetadataType<FVector2D>() { return EPCGMetadataTypes::Vector2; }
	template<> EPCGMetadataTypes GetExpectedMetadataType<FVector>() { return EPCGMetadataTypes::Vector; }
	template<> EPCGMetadataTypes GetExpectedMetadataType<FVector4>() { return EPCGMetadataTypes::Vector4; }
	template<> EPCGMetadataTypes GetExpectedMetadataType<FQuat>() { return EPCGMetadataTypes::Quaternion; }
	template<> EPCGMetadataTypes GetExpectedMetadataType<FTransform>() { return EPCGMetadataTypes::Transform; }
	template<> EPCGMetadataTypes GetExpectedMetadataType<FRotator>() { return EPCGMetadataTypes::Rotator; }
	template<> EPCGMetadataTypes GetExpectedMetadataType<FString>() { return EPCGMetadataTypes::String; }
	template<> EPCGMetadataTypes GetExpectedMetadataType<FName>() { return EPCGMetadataTypes::Name; }
	template<> EPCGMetadataTypes GetExpectedMetadataType<FSoftObjectPath>() { return EPCGMetadataTypes::SoftObjectPath; }
	template<> EPCGMetadataTypes GetExpectedMetadataType<FSoftClassPath>() { return EPCGMetadataTypes::SoftClassPath; }
	
	template <typename T>
	bool IsOfType(const FPCGMetadataAttributeBase* Attribute)
	{
		return Attribute && Attribute->IsOfType<T>();
	}
	
}

/**
 * Test creating FPCGMetadataAttribute<T> via Domain->CreateAttribute<T> for all supported types,
 * verify type checking, default values, single and bulk set/get, and parenting with PointArrayData.
 *
 * The typed tester lambda operates generically over T:
 *   1. Create a UPCGPointArrayData, allocate metadata entries.
 *   2. Create an FPCGMetadataAttribute<T> via CreateAttribute.
 *   3. Verify the attribute is of the right type (IsOfType<T>).
 *   4. Verify the default value can be read back correctly.
 *   5. Set random values in bulk, read them back and verify.
 *   6. Test parenting: create a child data, inherit, modify even indices, verify all values.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeTyped::BasicWithPointArrayData", "[PCG][AttributeTyped]")
{
	using namespace PCGAttributeTypedTests;

	auto TypedTester = [this]<typename T>()
	{
		FRandomStream RandomStream(Seed);

		// Create the data with points
		UPCGPointArrayData* PointArrayData = CreateData<UPCGPointArrayData>();
		PointArrayData->SetNumPoints(NumPoints);
		PointArrayData->AllocateProperties(EPCGPointNativeProperties::MetadataEntry);
		TPCGValueRange<int64> MetadataEntryRange = PointArrayData->GetMetadataEntryValueRange(/*bAllocate=*/false);

		FPCGMetadataDomain* Domain = PointArrayData->MutableMetadata()->GetDefaultMetadataDomain();
		REQUIRE_NOT_EQUAL(Domain, nullptr);

		// Create the typed attribute
		const T DefaultValue = GetDefaultValue<T>();
		FPCGMetadataAttribute<T>* TypedAttribute = Domain->CreateAttribute<T>(AttributeName, DefaultValue, /*bAllowsInterpolation=*/true, /*bOverrideParent=*/false);
		REQUIRE_NOT_EQUAL(TypedAttribute, nullptr);
		
		SECTION("IsOfType check")
		{
			REQUIRE(IsOfType<T>(TypedAttribute));
		}

		SECTION("Attribute descriptor matches expected type")
		{
			const FPCGMetadataAttributeDesc& Desc = TypedAttribute->GetAttributeDesc();
			REQUIRE_EQUAL(Desc.Name, AttributeName);
			REQUIRE_EQUAL(Desc.ValueType, GetExpectedMetadataType<T>());
		}

		SECTION("Default value is set correctly")
		{
			T ReadDefaultValue = TypedAttribute->GetValueFromItemKey(PCGInvalidEntryKey);
			if constexpr (std::is_same_v<T, FTransform>)
			{
				REQUIRE(ReadDefaultValue.Equals(DefaultValue));
			}
			else
			{
				REQUIRE_EQUAL(ReadDefaultValue, DefaultValue);
			}
		}

		// Add entries and set random values
		TStaticArray<T, NumPoints> Values{};
		for (int32 i = 0; i < NumPoints; ++i)
		{
			MetadataEntryRange[i] = Domain->AddEntry(MetadataEntryRange[i]);
			Values[i] = GenerateRandom<T>(RandomStream);
		}

		// Bulk set
		TypedAttribute->SetValues(PCGValueRangeHelpers::MakeConstValueRange(MetadataEntryRange), Values);
		REQUIRE_EQUAL(TypedAttribute->GetNumberOfEntries(), NumPoints);

		SECTION("Read back and verify all values")
		{
			TStaticArray<T, NumPoints> ReadValues{};
			TypedAttribute->GetValuesFromItemKeys(PCGValueRangeHelpers::MakeConstValueRange(MetadataEntryRange), ReadValues);

			for (int32 i = 0; i < NumPoints; ++i)
			{
				if constexpr (std::is_same_v<T, FTransform>)
				{
					REQUIRE(ReadValues[i].Equals(Values[i]));
				}
				else
				{
					REQUIRE_EQUAL(ReadValues[i], Values[i]);
				}
			}
		}

		SECTION("Single value set and get")
		{
			// Override the first entry with a specific value
			T SpecificValue = GetAlternativeDefaultValue<T>();
			TypedAttribute->SetValue(MetadataEntryRange[0], SpecificValue);

			T ReadBack = TypedAttribute->GetValueFromItemKey(MetadataEntryRange[0]);
			if constexpr (std::is_same_v<T, FTransform>)
			{
				REQUIRE(ReadBack.Equals(SpecificValue));
			}
			else
			{
				REQUIRE_EQUAL(ReadBack, SpecificValue);
			}
		}

		SECTION("Parenting with child PointArrayData")
		{
			UPCGPointArrayData* ChildPointArrayData = CreateData<UPCGPointArrayData>();
			ChildPointArrayData->InitializeFromDataWithParams(FPCGInitializeFromDataParams{PointArrayData});

			FPCGMetadataDomain* ChildDomain = ChildPointArrayData->MutableMetadata()->GetDefaultMetadataDomain();
			REQUIRE_NOT_EQUAL(ChildDomain, nullptr);

			// Retrieve the inherited typed attribute
			FPCGMetadataAttribute<T>* ChildTypedAttribute = ChildDomain->GetMutableTypedAttribute<T>(AttributeName);
			REQUIRE_NOT_EQUAL(ChildTypedAttribute, nullptr);

			TPCGValueRange<int64> ChildMetadataEntryRange = ChildPointArrayData->GetMetadataEntryValueRange(/*bAllocate=*/false);

			SECTION("Child inherits default value")
			{
				T ReadDefault = ChildTypedAttribute->GetValueFromItemKey(PCGInvalidEntryKey);
				if constexpr (std::is_same_v<T, FTransform>)
				{
					REQUIRE(ReadDefault.Equals(DefaultValue));
				}
				else
				{
					REQUIRE_EQUAL(ReadDefault, DefaultValue);
				}
			}

			SECTION("Child reads parent values unchanged")
			{
				TStaticArray<T, NumPoints> ReadValues{};
				ChildTypedAttribute->GetValuesFromItemKeys(PCGValueRangeHelpers::MakeConstValueRange(ChildMetadataEntryRange), ReadValues);

				for (int32 i = 0; i < NumPoints; ++i)
				{
					if constexpr (std::is_same_v<T, FTransform>)
					{
						REQUIRE(ReadValues[i].Equals(Values[i]));
					}
					else
					{
						REQUIRE_EQUAL(ReadValues[i], Values[i]);
					}
				}
			}

			SECTION("Child modifies even indices, odd indices read from parent")
			{
				// Modify even indices
				TStaticArray<T, NumPoints / 2> ChildValues{};
				TStaticArray<PCGMetadataEntryKey, NumPoints / 2> ChildEntryKeys{};
				for (int32 i = 0; i < NumPoints; ++i)
				{
					if (i % 2 == 0)
					{
						Values[i] = GenerateRandom<T>(RandomStream);
						ChildDomain->InitializeOnSet(ChildMetadataEntryRange[i]);
						ChildValues[i / 2] = Values[i];
						ChildEntryKeys[i / 2] = ChildMetadataEntryRange[i];
					}
				}

				ChildTypedAttribute->SetValues(TConstArrayView<PCGMetadataEntryKey>(ChildEntryKeys), ChildValues);

				REQUIRE_EQUAL(ChildTypedAttribute->GetNumberOfEntries(), NumPoints / 2);
				REQUIRE_EQUAL(ChildTypedAttribute->GetNumberOfEntriesWithParents(), NumPoints + NumPoints / 2);

				// Read all values - even indices should have new values, odd indices should have parent values
				TStaticArray<T, NumPoints> ReadValues{};
				ChildTypedAttribute->GetValuesFromItemKeys(PCGValueRangeHelpers::MakeConstValueRange(ChildMetadataEntryRange), ReadValues);

				for (int32 i = 0; i < NumPoints; ++i)
				{
					if constexpr (std::is_same_v<T, FTransform>)
					{
						REQUIRE(ReadValues[i].Equals(Values[i]));
					}
					else
					{
						REQUIRE_EQUAL(ReadValues[i], Values[i]);
					}
				}
			}
		}
	};

	SECTION("float") { TypedTester.template operator()<float>(); }
	SECTION("double") { TypedTester.template operator()<double>(); }
	SECTION("int32") { TypedTester.template operator()<int32>(); }
	SECTION("int64") { TypedTester.template operator()<int64>(); }
	SECTION("bool") { TypedTester.template operator()<bool>(); }
	SECTION("FVector2D") { TypedTester.template operator()<FVector2D>(); }
	SECTION("FVector") { TypedTester.template operator()<FVector>(); }
	SECTION("FVector4") { TypedTester.template operator()<FVector4>(); }
	SECTION("FQuat") { TypedTester.template operator()<FQuat>(); }
	SECTION("FTransform") { TypedTester.template operator()<FTransform>(); }
	SECTION("FRotator") { TypedTester.template operator()<FRotator>(); }
	SECTION("FString") { TypedTester.template operator()<FString>(); }
	SECTION("FName") { TypedTester.template operator()<FName>(); }
	SECTION("FSoftObjectPath") { TypedTester.template operator()<FSoftObjectPath>(); }
	SECTION("FSoftClassPath") { TypedTester.template operator()<FSoftClassPath>(); }
}

/**
 * Test creating FPCGMetadataAttribute<T> via Domain->CreateAttribute<T> with ParamData (copy-based parenting).
 * Tests that inherited values through copy are preserved correctly.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeTyped::CopyWithParamData", "[PCG][AttributeTyped]")
{
	using namespace PCGAttributeTypedTests;

	auto TypedTester = [this]<typename T>()
	{
		FRandomStream RandomStream(Seed);

		UPCGParamData* ParamData = CreateData<UPCGParamData>();
		FPCGMetadataDomain* Domain = ParamData->MutableMetadata()->GetDefaultMetadataDomain();
		REQUIRE_NOT_EQUAL(Domain, nullptr);

		const T DefaultValue = GetDefaultValue<T>();
		FPCGMetadataAttribute<T>* TypedAttribute = Domain->CreateAttribute<T>(AttributeName, DefaultValue, /*bAllowsInterpolation=*/true, /*bOverrideParent=*/false);
		REQUIRE_NOT_EQUAL(TypedAttribute, nullptr);

		TStaticArray<T, NumEntries> Values{};
		TStaticArray<PCGMetadataEntryKey, NumEntries> EntryKeys;

		for (int32 i = 0; i < NumEntries; ++i)
		{
			EntryKeys[i] = Domain->AddEntry();
			Values[i] = GenerateRandom<T>(RandomStream);
		}

		TypedAttribute->SetValues(TConstArrayView<PCGMetadataEntryKey>(EntryKeys), Values);
		REQUIRE_EQUAL(TypedAttribute->GetNumberOfEntries(), NumEntries);

		SECTION("Values read back correctly")
		{
			TStaticArray<T, NumEntries> ReadValues{};
			TypedAttribute->GetValuesFromItemKeys(TConstArrayView<PCGMetadataEntryKey>(EntryKeys), ReadValues);

			for (int32 i = 0; i < NumEntries; ++i)
			{
				if constexpr (std::is_same_v<T, FTransform>)
				{
					REQUIRE(ReadValues[i].Equals(Values[i]));
				}
				else
				{
					REQUIRE_EQUAL(ReadValues[i], Values[i]);
				}
			}
		}

		SECTION("Copy-based parenting")
		{
			UPCGParamData* ChildParamData = CreateData<UPCGParamData>();
			ChildParamData->MutableMetadata()->Initialize(ParamData->ConstMetadata());

			FPCGMetadataDomain* ChildDomain = ChildParamData->MutableMetadata()->GetDefaultMetadataDomain();
			REQUIRE_NOT_EQUAL(ChildDomain, nullptr);

			FPCGMetadataAttribute<T>* ChildTypedAttribute = ChildDomain->GetMutableTypedAttribute<T>(AttributeName);
			REQUIRE_NOT_EQUAL(ChildTypedAttribute, nullptr);

			SECTION("Inherited default value")
			{
				T ReadDefault = ChildTypedAttribute->GetValueFromItemKey(PCGInvalidEntryKey);
				if constexpr (std::is_same_v<T, FTransform>)
				{
					REQUIRE(ReadDefault.Equals(DefaultValue));
				}
				else
				{
					REQUIRE_EQUAL(ReadDefault, DefaultValue);
				}
			}

			SECTION("Modify even entries, verify all")
			{
				TStaticArray<T, NumEntries / 2> ChildValues{};
				TStaticArray<PCGMetadataEntryKey, NumEntries / 2> ChildEntryKeys{};
				for (int32 i = 0; i < NumEntries; ++i)
				{
					if (i % 2 == 0)
					{
						Values[i] = GenerateRandom<T>(RandomStream);
						ChildDomain->InitializeOnSet(EntryKeys[i]);
						ChildValues[i / 2] = Values[i];
						ChildEntryKeys[i / 2] = EntryKeys[i];
					}
				}

				ChildTypedAttribute->SetValues(TConstArrayView<PCGMetadataEntryKey>(ChildEntryKeys), ChildValues);

				// For copy-based parenting, entries are copied, so total should be NumEntries (not additive)
				REQUIRE_EQUAL(ChildTypedAttribute->GetNumberOfEntries(), NumEntries);
				REQUIRE_EQUAL(ChildTypedAttribute->GetNumberOfEntriesWithParents(), NumEntries);

				TStaticArray<T, NumEntries> ReadValues{};
				ChildTypedAttribute->GetValuesFromItemKeys(TConstArrayView<PCGMetadataEntryKey>(EntryKeys), ReadValues);

				for (int32 i = 0; i < NumEntries; ++i)
				{
					if constexpr (std::is_same_v<T, FTransform>)
					{
						REQUIRE(ReadValues[i].Equals(Values[i]));
					}
					else
					{
						REQUIRE_EQUAL(ReadValues[i], Values[i]);
					}
				}
			}
		}
	};

	SECTION("float") { TypedTester.template operator()<float>(); }
	SECTION("double") { TypedTester.template operator()<double>(); }
	SECTION("int32") { TypedTester.template operator()<int32>(); }
	SECTION("int64") { TypedTester.template operator()<int64>(); }
	SECTION("bool") { TypedTester.template operator()<bool>(); }
	SECTION("FVector2D") { TypedTester.template operator()<FVector2D>(); }
	SECTION("FVector") { TypedTester.template operator()<FVector>(); }
	SECTION("FVector4") { TypedTester.template operator()<FVector4>(); }
	SECTION("FQuat") { TypedTester.template operator()<FQuat>(); }
	SECTION("FTransform") { TypedTester.template operator()<FTransform>(); }
	SECTION("FRotator") { TypedTester.template operator()<FRotator>(); }
	SECTION("FString") { TypedTester.template operator()<FString>(); }
	SECTION("FName") { TypedTester.template operator()<FName>(); }
	SECTION("FSoftObjectPath") { TypedTester.template operator()<FSoftObjectPath>(); }
	SECTION("FSoftClassPath") { TypedTester.template operator()<FSoftClassPath>(); }
}

/**
 * Test FindOrCreateAttribute: first call creates, second call finds the existing one.
 * Also test that FindOrCreateAttribute with a different type and bOverwriteIfTypeMismatch=true
 * replaces the attribute.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeTyped::FindOrCreate", "[PCG][AttributeTyped]")
{
	using namespace PCGAttributeTypedTests;

	UPCGParamData* ParamData = CreateData<UPCGParamData>();
	FPCGMetadataDomain* Domain = ParamData->MutableMetadata()->GetDefaultMetadataDomain();
	REQUIRE_NOT_EQUAL(Domain, nullptr);

	SECTION("Create then find returns same attribute")
	{
		FPCGMetadataAttribute<float>* Attr1 = Domain->FindOrCreateAttribute<float>(AttributeName, 1.0f);
		REQUIRE_NOT_EQUAL(Attr1, nullptr);

		FPCGMetadataAttribute<float>* Attr2 = Domain->FindOrCreateAttribute<float>(AttributeName, 2.0f);
		REQUIRE_NOT_EQUAL(Attr2, nullptr);

		// Should be the same attribute pointer
		REQUIRE_EQUAL(Attr1, Attr2);

		// Default value should still be from the first creation
		float DefaultVal = Attr2->GetValueFromItemKey(PCGInvalidEntryKey);
		REQUIRE_EQUAL(DefaultVal, 1.0f);
	}

	SECTION("FindOrCreate with type mismatch replaces attribute")
	{
		FPCGMetadataAttribute<float>* FloatAttr = Domain->FindOrCreateAttribute<float>(AttributeName, 1.0f);
		REQUIRE_NOT_EQUAL(FloatAttr, nullptr);
		REQUIRE(IsOfType<float>(FloatAttr));

		// Now request an int32 attribute with the same name - should overwrite
		FPCGMetadataAttribute<int32>* IntAttr = Domain->FindOrCreateAttribute<int32>(AttributeName, 42, /*bAllowsInterpolation=*/true, /*bOverrideParent=*/true, /*bOverwriteIfTypeMismatch=*/true);
		REQUIRE_NOT_EQUAL(IntAttr, nullptr);
		REQUIRE(IsOfType<int32>(IntAttr));

		int32 DefaultVal = IntAttr->GetValueFromItemKey(PCGInvalidEntryKey);
		REQUIRE_EQUAL(DefaultVal, 42);
	}
}

/**
 * Test that GetValue and GetValues using value keys (not entry keys) work correctly.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeTyped::GetValueByValueKey", "[PCG][AttributeTyped]")
{
	using namespace PCGAttributeTypedTests;

	auto TypedTester = [this]<typename T>()
	{
		UPCGParamData* ParamData = CreateData<UPCGParamData>();
		FPCGMetadataDomain* Domain = ParamData->MutableMetadata()->GetDefaultMetadataDomain();

		const T DefaultValue = GetDefaultValue<T>();
		FPCGMetadataAttribute<T>* Attr = Domain->CreateAttribute<T>(AttributeName, DefaultValue, /*bAllowsInterpolation=*/true, /*bOverrideParent=*/false);
		REQUIRE_NOT_EQUAL(Attr, nullptr);

		// GetValue with PCGDefaultValueKey should return the default
		T ReadDefault = Attr->GetValue(PCGDefaultValueKey);
		if constexpr (std::is_same_v<T, FTransform>)
		{
			REQUIRE(ReadDefault.Equals(DefaultValue));
		}
		else
		{
			REQUIRE_EQUAL(ReadDefault, DefaultValue);
		}

		// Add a value and read it by value key
		FRandomStream RandomStream(Seed);
		T TestValue = GenerateRandom<T>(RandomStream);
		PCGMetadataValueKey ValueKey = Attr->AddValue(TestValue);
		REQUIRE(ValueKey != PCGDefaultValueKey);

		T ReadValue = Attr->GetValue(ValueKey);
		if constexpr (std::is_same_v<T, FTransform>)
		{
			REQUIRE(ReadValue.Equals(TestValue));
		}
		else
		{
			REQUIRE_EQUAL(ReadValue, TestValue);
		}
	};

	SECTION("float") { TypedTester.template operator()<float>(); }
	SECTION("double") { TypedTester.template operator()<double>(); }
	SECTION("int32") { TypedTester.template operator()<int32>(); }
	SECTION("int64") { TypedTester.template operator()<int64>(); }
	SECTION("bool") { TypedTester.template operator()<bool>(); }
	SECTION("FVector2D") { TypedTester.template operator()<FVector2D>(); }
	SECTION("FVector") { TypedTester.template operator()<FVector>(); }
	SECTION("FVector4") { TypedTester.template operator()<FVector4>(); }
	SECTION("FQuat") { TypedTester.template operator()<FQuat>(); }
	SECTION("FTransform") { TypedTester.template operator()<FTransform>(); }
	SECTION("FRotator") { TypedTester.template operator()<FRotator>(); }
	SECTION("FString") { TypedTester.template operator()<FString>(); }
	SECTION("FName") { TypedTester.template operator()<FName>(); }
	SECTION("FSoftObjectPath") { TypedTester.template operator()<FSoftObjectPath>(); }
	SECTION("FSoftClassPath") { TypedTester.template operator()<FSoftClassPath>(); }
}

/**
 * Test AddValue and AddValues return correct value keys, and that FindValue works
 * for compressed data types (FString, FSoftObjectPath, FSoftClassPath).
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeTyped::AddValueAndFindValue", "[PCG][AttributeTyped]")
{
	using namespace PCGAttributeTypedTests;

	UPCGParamData* ParamData = CreateData<UPCGParamData>();
	FPCGMetadataDomain* Domain = ParamData->MutableMetadata()->GetDefaultMetadataDomain();

	SECTION("FString - compressed, FindValue should work")
	{
		FPCGMetadataAttribute<FString>* Attr = Domain->CreateAttribute<FString>(AttributeName, FString(TEXT("Default")), /*bAllowsInterpolation=*/true, /*bOverrideParent=*/false);
		REQUIRE_NOT_EQUAL(Attr, nullptr);

		PCGMetadataValueKey Key1 = Attr->AddValue(FString(TEXT("Hello")));
		PCGMetadataValueKey Key2 = Attr->AddValue(FString(TEXT("World")));
		REQUIRE(Key1 != PCGDefaultValueKey);
		REQUIRE(Key2 != PCGDefaultValueKey);
		REQUIRE(Key1 != Key2);

		// Adding the same value should return the same key (compressed)
		PCGMetadataValueKey Key1Again = Attr->AddValue(FString(TEXT("Hello")));
		REQUIRE_EQUAL(Key1, Key1Again);

		// FindValue should locate existing values
		PCGMetadataValueKey FoundKey = Attr->FindValue(FString(TEXT("Hello")));
		REQUIRE_EQUAL(FoundKey, Key1);

		PCGMetadataValueKey FoundKey2 = Attr->FindValue(FString(TEXT("World")));
		REQUIRE_EQUAL(FoundKey2, Key2);
	}

	SECTION("FSoftObjectPath - compressed, FindValue should work")
	{
		FPCGMetadataAttribute<FSoftObjectPath>* Attr = Domain->CreateAttribute<FSoftObjectPath>(AttributeName, FSoftObjectPath(), /*bAllowsInterpolation=*/true, /*bOverrideParent=*/false);
		REQUIRE_NOT_EQUAL(Attr, nullptr);

		FSoftObjectPath Path1(TEXT("/Game/Test/A"));
		FSoftObjectPath Path2(TEXT("/Game/Test/B"));

		PCGMetadataValueKey Key1 = Attr->AddValue(Path1);
		PCGMetadataValueKey Key2 = Attr->AddValue(Path2);
		REQUIRE(Key1 != Key2);

		PCGMetadataValueKey FoundKey = Attr->FindValue(Path1);
		REQUIRE_EQUAL(FoundKey, Key1);
	}

	SECTION("FSoftClassPath - compressed, FindValue should work")
	{
		FPCGMetadataAttribute<FSoftClassPath>* Attr = Domain->CreateAttribute<FSoftClassPath>(AttributeName, FSoftClassPath(), /*bAllowsInterpolation=*/true, /*bOverrideParent=*/false);
		REQUIRE_NOT_EQUAL(Attr, nullptr);

		FSoftClassPath Path1(TEXT("/Game/Test/ClassA"));
		FSoftClassPath Path2(TEXT("/Game/Test/ClassB"));

		PCGMetadataValueKey Key1 = Attr->AddValue(Path1);
		PCGMetadataValueKey Key2 = Attr->AddValue(Path2);
		REQUIRE(Key1 != Key2);

		PCGMetadataValueKey FoundKey = Attr->FindValue(Path1);
		REQUIRE_EQUAL(FoundKey, Key1);
	}

	SECTION("float - not compressed, FindValue returns PCGNotFoundValueKey")
	{
		FPCGMetadataAttribute<float>* Attr = Domain->CreateAttribute<float>(AttributeName, 0.0f, /*bAllowsInterpolation=*/true, /*bOverrideParent=*/false);
		REQUIRE_NOT_EQUAL(Attr, nullptr);

		Attr->AddValue(1.0f);
		PCGMetadataValueKey FoundKey = Attr->FindValue(1.0f);
		REQUIRE_EQUAL(FoundKey, PCGNotFoundValueKey);
	}
}

/**
 * Test that the typed attribute is accessible through both the generic and typed getter APIs.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeTyped::GenericAndTypedAccess", "[PCG][AttributeTyped]")
{
	using namespace PCGAttributeTypedTests;

	UPCGParamData* ParamData = CreateData<UPCGParamData>();
	FPCGMetadataDomain* Domain = ParamData->MutableMetadata()->GetDefaultMetadataDomain();

	FPCGMetadataAttribute<double>* TypedAttr = Domain->CreateAttribute<double>(AttributeName, 3.14, /*bAllowsInterpolation=*/true, /*bOverrideParent=*/false);
	REQUIRE_NOT_EQUAL(TypedAttr, nullptr);

	SECTION("GetMutableAttribute returns the same attribute")
	{
		FPCGMetadataAttributeBase* GenericAttr = Domain->GetMutableAttribute(AttributeName);
		REQUIRE_NOT_EQUAL(GenericAttr, nullptr);
		REQUIRE(IsOfType<double>(GenericAttr));
	
		REQUIRE_EQUAL(static_cast<FPCGMetadataAttributeBase*>(TypedAttr), GenericAttr);
	}

	SECTION("GetMutableTypedAttribute returns typed attribute")
	{
		FPCGMetadataAttribute<double>* FoundAttr = Domain->GetMutableTypedAttribute<double>(AttributeName);
		REQUIRE_EQUAL(FoundAttr, TypedAttr);
	}

	SECTION("GetMutableTypedAttribute with wrong type returns nullptr")
	{
		FPCGMetadataAttribute<float>* WrongAttr = Domain->GetMutableTypedAttribute<float>(AttributeName);
		REQUIRE_EQUAL(WrongAttr, nullptr);
	}
	
	SECTION("Setting via generic, reading via typed")
	{
		PCGMetadataEntryKey EntryKey = Domain->AddEntry();
		FPCGMetadataAttributeBase* GenericAttr = Domain->GetMutableAttribute(AttributeName);
		GenericAttr->SetValue<double>(EntryKey, 2.718);

		double ReadValue = TypedAttr->GetValueFromItemKey(EntryKey);
		REQUIRE_EQUAL(ReadValue, 2.718);
	}

	 SECTION("Setting via typed, reading via generic")
	 {
	 	PCGMetadataEntryKey EntryKey = Domain->AddEntry();
	 	TypedAttr->SetValue(EntryKey, 1.618);
	
	 	FPCGMetadataAttributeBase* GenericAttr = Domain->GetMutableAttribute(AttributeName);
	 	double ReadValue = GenericAttr->GetValueFromItemKey<double>(EntryKey);
	 	REQUIRE_EQUAL(ReadValue, 1.618);
	 }
}

/**
 * Test TypedCopy: creating a copy of a typed attribute with a new name,
 * verifying that entries and values are preserved correctly.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeTyped::TypedCopy", "[PCG][AttributeTyped]")
{
	using namespace PCGAttributeTypedTests;

	auto TypedTester = [this]<typename T>()
	{
		FRandomStream RandomStream(Seed);

		UPCGParamData* ParamData = CreateData<UPCGParamData>();
		FPCGMetadataDomain* Domain = ParamData->MutableMetadata()->GetDefaultMetadataDomain();

		const T DefaultValue = GetDefaultValue<T>();
		FPCGMetadataAttribute<T>* OriginalAttr = Domain->CreateAttribute<T>(AttributeName, DefaultValue, /*bAllowsInterpolation=*/true, /*bOverrideParent=*/false);
		REQUIRE_NOT_EQUAL(OriginalAttr, nullptr);

		constexpr int32 CopyCount = 5;
		TStaticArray<T, CopyCount> Values{};
		TStaticArray<PCGMetadataEntryKey, CopyCount> EntryKeys;

		for (int32 i = 0; i < CopyCount; ++i)
		{
			EntryKeys[i] = Domain->AddEntry();
			Values[i] = GenerateRandom<T>(RandomStream);
		}

		OriginalAttr->SetValues(TConstArrayView<PCGMetadataEntryKey>(EntryKeys), Values);

		FPCGMetadataAttribute<T>* CopiedAttr = OriginalAttr->TypedCopy(FName(TEXT("CopiedAttr")), Domain, /*bKeepParent=*/false, /*bCopyEntries=*/true, /*bCopyValues=*/true);
		REQUIRE_NOT_EQUAL(CopiedAttr, nullptr);

		SECTION("Copied attribute has correct default")
		{
			T ReadDefault = CopiedAttr->GetValue(PCGDefaultValueKey);
			REQUIRE_EQUAL(ReadDefault, DefaultValue);
		}

		SECTION("Copied attribute has correct values")
		{
			TStaticArray<T, CopyCount> ReadValues{};
			CopiedAttr->GetValuesFromItemKeys(TConstArrayView<PCGMetadataEntryKey>(EntryKeys), ReadValues);

			for (int32 i = 0; i < CopyCount; ++i)
			{
				REQUIRE_EQUAL(ReadValues[i], Values[i]);
			}
		}

		// Clean up the copied attribute as it was created with new
		delete CopiedAttr;
	};

	SECTION("float") { TypedTester.template operator()<float>(); }
	SECTION("double") { TypedTester.template operator()<double>(); }
	SECTION("int32") { TypedTester.template operator()<int32>(); }
	SECTION("int64") { TypedTester.template operator()<int64>(); }
	SECTION("FVector") { TypedTester.template operator()<FVector>(); }
	SECTION("FString") { TypedTester.template operator()<FString>(); }
	SECTION("FName") { TypedTester.template operator()<FName>(); }
}

/**
 * Edge case: creating an attribute with zero entries and reading the default value.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeTyped::EmptyAttribute", "[PCG][AttributeTyped]")
{
	using namespace PCGAttributeTypedTests;

	UPCGParamData* ParamData = CreateData<UPCGParamData>();
	FPCGMetadataDomain* Domain = ParamData->MutableMetadata()->GetDefaultMetadataDomain();

	SECTION("No entries, read default for float")
	{
		FPCGMetadataAttribute<float>* Attr = Domain->CreateAttribute<float>(AttributeName, 42.0f, /*bAllowsInterpolation=*/true, /*bOverrideParent=*/false);
		REQUIRE_NOT_EQUAL(Attr, nullptr);
		REQUIRE_EQUAL(Attr->GetNumberOfEntries(), 0);

		float ReadDefault = Attr->GetValueFromItemKey(PCGInvalidEntryKey);
		REQUIRE_EQUAL(ReadDefault, 42.0f);
	}

	SECTION("No entries, read default for FString")
	{
		FPCGMetadataAttribute<FString>* Attr = Domain->CreateAttribute<FString>(AttributeName, FString(TEXT("Empty")), /*bAllowsInterpolation=*/true, /*bOverrideParent=*/false);
		REQUIRE_NOT_EQUAL(Attr, nullptr);
		REQUIRE_EQUAL(Attr->GetNumberOfEntries(), 0);

		FString ReadDefault = Attr->GetValueFromItemKey(PCGInvalidEntryKey);
		REQUIRE_EQUAL(ReadDefault, FString(TEXT("Empty")));
	}

	SECTION("No entries, read default for bool")
	{
		FPCGMetadataAttribute<bool>* Attr = Domain->CreateAttribute<bool>(AttributeName, true, /*bAllowsInterpolation=*/true, /*bOverrideParent=*/false);
		REQUIRE_NOT_EQUAL(Attr, nullptr);

		bool ReadDefault = Attr->GetValueFromItemKey(PCGInvalidEntryKey);
		REQUIRE_EQUAL(ReadDefault, true);
	}
}

/**
 * Test overwriting the same entry key multiple times and verifying only the last value persists.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeTyped::OverwriteValues", "[PCG][AttributeTyped]")
{
	using namespace PCGAttributeTypedTests;

	UPCGParamData* ParamData = CreateData<UPCGParamData>();
	FPCGMetadataDomain* Domain = ParamData->MutableMetadata()->GetDefaultMetadataDomain();

	FPCGMetadataAttribute<int32>* Attr = Domain->CreateAttribute<int32>(AttributeName, 0, /*bAllowsInterpolation=*/true, /*bOverrideParent=*/false);
	REQUIRE_NOT_EQUAL(Attr, nullptr);

	PCGMetadataEntryKey EntryKey = Domain->AddEntry();

	// Write multiple times to the same entry
	Attr->SetValue(EntryKey, 10);
	REQUIRE_EQUAL(Attr->GetValueFromItemKey(EntryKey), 10);

	Attr->SetValue(EntryKey, 20);
	REQUIRE_EQUAL(Attr->GetValueFromItemKey(EntryKey), 20);

	Attr->SetValue(EntryKey, 30);
	REQUIRE_EQUAL(Attr->GetValueFromItemKey(EntryKey), 30);
}

/**
 * Test that we can create multiple attributes of different types on the same metadata domain
 * and they do not interfere with each other.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeTyped::MultipleAttributesSameData", "[PCG][AttributeTyped]")
{
	using namespace PCGAttributeTypedTests;

	UPCGParamData* ParamData = CreateData<UPCGParamData>();
	FPCGMetadataDomain* Domain = ParamData->MutableMetadata()->GetDefaultMetadataDomain();

	FPCGMetadataAttribute<float>* FloatAttr = Domain->CreateAttribute<float>(FName(TEXT("FloatAttr")), 1.0f, true, false);
	FPCGMetadataAttribute<int32>* IntAttr = Domain->CreateAttribute<int32>(FName(TEXT("IntAttr")), 42, true, false);
	FPCGMetadataAttribute<FString>* StrAttr = Domain->CreateAttribute<FString>(FName(TEXT("StrAttr")), FString(TEXT("Hello")), true, false);
	FPCGMetadataAttribute<FVector>* VecAttr = Domain->CreateAttribute<FVector>(FName(TEXT("VecAttr")), FVector::ZeroVector, true, false);

	REQUIRE_NOT_EQUAL(FloatAttr, nullptr);
	REQUIRE_NOT_EQUAL(IntAttr, nullptr);
	REQUIRE_NOT_EQUAL(StrAttr, nullptr);
	REQUIRE_NOT_EQUAL(VecAttr, nullptr);

	REQUIRE_EQUAL(Domain->GetAttributeCount(), 4);

	PCGMetadataEntryKey EntryKey = Domain->AddEntry();

	FloatAttr->SetValue(EntryKey, 3.14f);
	IntAttr->SetValue(EntryKey, 100);
	StrAttr->SetValue(EntryKey, FString(TEXT("World")));
	VecAttr->SetValue(EntryKey, FVector(1.0, 2.0, 3.0));

	REQUIRE_EQUAL(FloatAttr->GetValueFromItemKey(EntryKey), 3.14f);
	REQUIRE_EQUAL(IntAttr->GetValueFromItemKey(EntryKey), 100);
	REQUIRE_EQUAL(StrAttr->GetValueFromItemKey(EntryKey), FString(TEXT("World")));
	REQUIRE_EQUAL(VecAttr->GetValueFromItemKey(EntryKey), FVector(1.0, 2.0, 3.0));

	// Defaults should be unaffected
	REQUIRE_EQUAL(FloatAttr->GetValueFromItemKey(PCGInvalidEntryKey), 1.0f);
	REQUIRE_EQUAL(IntAttr->GetValueFromItemKey(PCGInvalidEntryKey), 42);
	REQUIRE_EQUAL(StrAttr->GetValueFromItemKey(PCGInvalidEntryKey), FString(TEXT("Hello")));
	REQUIRE_EQUAL(VecAttr->GetValueFromItemKey(PCGInvalidEntryKey), FVector::ZeroVector);
}

/**
 * Test that creating a duplicate attribute with CreateAttribute (not FindOrCreate) produces a warning
 * but returns the existing attribute when types match.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeTyped::DuplicateCreateAttribute", "[PCG][AttributeTyped]")
{
	using namespace PCGAttributeTypedTests;

	UPCGParamData* ParamData = CreateData<UPCGParamData>();
	FPCGMetadataDomain* Domain = ParamData->MutableMetadata()->GetDefaultMetadataDomain();

	FPCGMetadataAttribute<float>* Attr1 = Domain->CreateAttribute<float>(AttributeName, 1.0f, true, false);
	REQUIRE_NOT_EQUAL(Attr1, nullptr);

	// Creating again with the same type should log a warning but succeed
	{
		PCGTests::FPCGTestsLogOutputDevice LogCapture(/*bSuppressErrors=*/true);
		FPCGMetadataAttribute<float>* Attr2 = Domain->CreateAttribute<float>(AttributeName, 2.0f, true, false);
		// Should return the existing attribute
		REQUIRE_NOT_EQUAL(Attr2, nullptr);
	}
}

/**
 * Test that a large number of entries works correctly.
 * This helps catch off-by-one errors and capacity issues.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeTyped::LargeEntryCount", "[PCG][AttributeTyped]")
{
	using namespace PCGAttributeTypedTests;

	constexpr int32 LargeCount = 10000;

	UPCGPointArrayData* PointArrayData = CreateData<UPCGPointArrayData>();
	PointArrayData->SetNumPoints(LargeCount);
	PointArrayData->AllocateProperties(EPCGPointNativeProperties::MetadataEntry);
	TPCGValueRange<int64> MetadataEntryRange = PointArrayData->GetMetadataEntryValueRange(/*bAllocate=*/false);

	FPCGMetadataDomain* Domain = PointArrayData->MutableMetadata()->GetDefaultMetadataDomain();

	FPCGMetadataAttribute<int32>* Attr = Domain->CreateAttribute<int32>(PCGAttributeTypedTests::AttributeName, -1, true, false);
	REQUIRE_NOT_EQUAL(Attr, nullptr);

	TArray<int32> Values;
	Values.SetNum(LargeCount);

	for (int32 i = 0; i < LargeCount; ++i)
	{
		MetadataEntryRange[i] = Domain->AddEntry(MetadataEntryRange[i]);
		Values[i] = i;
	}

	Attr->SetValues(PCGValueRangeHelpers::MakeConstValueRange(MetadataEntryRange), Values);
	REQUIRE_EQUAL(Attr->GetNumberOfEntries(), LargeCount);

	// Verify first, last, and middle
	REQUIRE_EQUAL(Attr->GetValueFromItemKey(MetadataEntryRange[0]), 0);
	REQUIRE_EQUAL(Attr->GetValueFromItemKey(MetadataEntryRange[LargeCount - 1]), LargeCount - 1);
	REQUIRE_EQUAL(Attr->GetValueFromItemKey(MetadataEntryRange[LargeCount / 2]), LargeCount / 2);

	// Verify all values
	TArray<int32> ReadValues;
	ReadValues.SetNum(LargeCount);
	Attr->GetValuesFromItemKeys(PCGValueRangeHelpers::MakeConstValueRange(MetadataEntryRange), ReadValues);

	for (int32 i = 0; i < LargeCount; ++i)
	{
		REQUIRE_EQUAL(ReadValues[i], i);
	}
}

/**
 * Test compressed data behavior for FString attributes: duplicate values should share value keys.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeTyped::CompressedData", "[PCG][AttributeTyped]")
{
	using namespace PCGAttributeTypedTests;

	constexpr int32 Count = 100;

	UPCGPointArrayData* PointArrayData = CreateData<UPCGPointArrayData>();
	PointArrayData->SetNumPoints(Count);
	PointArrayData->AllocateProperties(EPCGPointNativeProperties::MetadataEntry);
	TPCGValueRange<int64> MetadataEntryRange = PointArrayData->GetMetadataEntryValueRange(/*bAllocate=*/false);

	FPCGMetadataDomain* Domain = PointArrayData->MutableMetadata()->GetDefaultMetadataDomain();

	// Use the default value as one of the values we'll set
	FPCGMetadataAttribute<FString>* Attr = Domain->CreateAttribute<FString>(AttributeName, FString(TEXT("Value: 0")), true, false);
	REQUIRE_NOT_EQUAL(Attr, nullptr);

	TStaticArray<FString, Count> Values{};

	// First half: 10 unique values (mod 10)
	for (int32 i = 0; i < Count / 2; ++i)
	{
		Domain->InitializeOnSet(MetadataEntryRange[i]);
		Values[i] = FString::Printf(TEXT("Value: %d"), i % 10);
	}

	TArray<PCGMetadataEntryKey> FirstHalfKeys;
	FirstHalfKeys.SetNum(Count / 2);
	for (int32 i = 0; i < Count / 2; ++i)
	{
		FirstHalfKeys[i] = MetadataEntryRange[i];
	}

	Attr->SetValues(TConstArrayView<PCGMetadataEntryKey>(FirstHalfKeys), MakeArrayView(Values.GetData(), Count / 2));

	// Should have only 9 unique value keys (the tenth one is the default "Value: 0")
	REQUIRE_EQUAL(Attr->GetValueKeyOffsetForChild(), 9);

	// Second half: mix of existing and new strings
	TArray<PCGMetadataEntryKey> SecondHalfKeys;
	SecondHalfKeys.SetNum(Count / 2);
	for (int32 i = Count / 2; i < Count; ++i)
	{
		const int32 Index = i - Count / 2;
		Domain->InitializeOnSet(MetadataEntryRange[i]);
		SecondHalfKeys[Index] = MetadataEntryRange[i];

		if (Index < 10)
		{
			Values[i] = FString::Printf(TEXT("Value: %d"), Index % 10);
		}
		else
		{
			Values[i] = FString::Printf(TEXT("New Value: %d"), Index % 10);
		}
	}

	Attr->SetValues(TConstArrayView<PCGMetadataEntryKey>(SecondHalfKeys), MakeArrayView(Values.GetData() + Count / 2, Count / 2));

	// Should have 19 unique value keys now (9 old + 10 new, the twentieth one is the default which matches "Value: 0")
	REQUIRE_EQUAL(Attr->GetValueKeyOffsetForChild(), 19);

	// Verify all values can be read back
	TStaticArray<FString, Count> ReadValues{};
	Attr->GetValuesFromItemKeys(PCGValueRangeHelpers::MakeConstValueRange(MetadataEntryRange), ReadValues);

	for (int32 i = 0; i < Count; ++i)
	{
		REQUIRE_EQUAL(ReadValues[i], Values[i]);
	}
}

/**
 * Stress-test the system to verify that we have no race conditions.
 * Adding the same value from a lot of different threads should yield with a single value key.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeTyped::CompressedData::MultiThreading", "[PCG][AttributeTyped]")
{
	constexpr int32 NumEntries = 100000;
	constexpr int32 Chunks = 100;
	constexpr int32 ChunksPerThread = 10;
	constexpr int32 ElementsPerThreads = Chunks * ChunksPerThread;
	constexpr int32 NumThreads = NumEntries / ElementsPerThreads;
	
	constexpr int32 Seed = 42;
	UPCGParamData* ParamData = CreateData<UPCGParamData>();
	FPCGMetadataDomain* Domain = ParamData->MutableMetadata()->GetDefaultMetadataDomain();

	FRandomStream RandomStream(Seed);

	FPCGMetadataAttribute<FString>* Attribute = Domain->CreateAttribute<FString>(PCGAttributeTypedTests::AttributeName, FString{});
	REQUIRE_NOT_EQUAL(Attribute, nullptr);

	TStaticArray<FString, NumEntries> Values{};
	TStaticArray<PCGMetadataEntryKey, NumEntries> EntryKeys;
	TStaticArray<PCGMetadataEntryKey*, NumEntries> EntryKeysPtr;
	
	for (int32 i = 0; i < NumEntries; ++i)
	{
		EntryKeys[i] = Domain->AddEntry();
		EntryKeysPtr[i] = &EntryKeys[i];
		Values[i] = TEXT("Hi");
	}
	
	bool bHasRun = false;
	bool bHasRunOnMultiThread = false;
	PCGTests::Async::AsyncRun(NumThreads, [&Values, &EntryKeysPtr, Attribute](int32 ThreadIndex)
	{
		for (int32 j = 0; j < ChunksPerThread; ++j)
		{
			const int32 ValuesIndex = ThreadIndex * ElementsPerThreads + j * Chunks;
			TArrayView<PCGMetadataEntryKey*> EntryKeysView = MakeArrayView(EntryKeysPtr.GetData() + ValuesIndex, Chunks);
			TArrayView<FString> ValuesView = MakeArrayView(Values.GetData() + ValuesIndex, Chunks);
			
			Attribute->SetValues(EntryKeysView, ValuesView);
		}
	}, bHasRun, bHasRunOnMultiThread);
	
	REQUIRE(bHasRun);
	
	// Making sure we had concurrency.
	CHECK(bHasRunOnMultiThread);
	
	REQUIRE_EQUAL(Attribute->GetNumberOfEntries(), Values.Num());
	REQUIRE_EQUAL(Attribute->GetValueKeyOffsetForChild(), 1);
	
	// Verify all values can be read back
	TArray<FString> ReadValues{};
	ReadValues.SetNum(NumEntries);
	Attribute->GetValuesFromItemKeys(MakeArrayView(EntryKeys), ReadValues);

	for (int32 i = 0; i < NumEntries; ++i)
	{
		REQUIRE_EQUAL(ReadValues[i], Values[i]);
	}
}

/**
 * Test that deleting an attribute actually removes it from the domain.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeTyped::DeleteAttribute", "[PCG][AttributeTyped]")
{
	using namespace PCGAttributeTypedTests;

	UPCGParamData* ParamData = CreateData<UPCGParamData>();
	FPCGMetadataDomain* Domain = ParamData->MutableMetadata()->GetDefaultMetadataDomain();

	FPCGMetadataAttribute<float>* Attr = Domain->CreateAttribute<float>(AttributeName, 1.0f, true, false);
	REQUIRE_NOT_EQUAL(Attr, nullptr);
	REQUIRE(Domain->HasAttribute(AttributeName));
	REQUIRE_EQUAL(Domain->GetAttributeCount(), 1);

	Domain->DeleteAttribute(AttributeName);
	REQUIRE(!Domain->HasAttribute(AttributeName));
	REQUIRE_EQUAL(Domain->GetAttributeCount(), 0);

	// Should be able to create a new attribute with the same name after deletion
	FPCGMetadataAttribute<int32>* NewAttr = Domain->CreateAttribute<int32>(AttributeName, 42, true, false);
	REQUIRE_NOT_EQUAL(NewAttr, nullptr);
	REQUIRE(IsOfType<int32>(NewAttr));
}

/**
 * Test setting values one at a time (single SetValue) vs bulk SetValues, and verify results match.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeTyped::SingleVsBulkSetValues", "[PCG][AttributeTyped]")
{
	using namespace PCGAttributeTypedTests;

	FRandomStream RandomStream(Seed);

	UPCGParamData* ParamData1 = CreateData<UPCGParamData>();
	UPCGParamData* ParamData2 = CreateData<UPCGParamData>();
	FPCGMetadataDomain* Domain1 = ParamData1->MutableMetadata()->GetDefaultMetadataDomain();
	FPCGMetadataDomain* Domain2 = ParamData2->MutableMetadata()->GetDefaultMetadataDomain();

	FPCGMetadataAttribute<FVector>* SingleAttr = Domain1->CreateAttribute<FVector>(AttributeName, FVector::ZeroVector, true, false);
	FPCGMetadataAttribute<FVector>* BulkAttr = Domain2->CreateAttribute<FVector>(AttributeName, FVector::ZeroVector, true, false);

	REQUIRE_NOT_EQUAL(SingleAttr, nullptr);
	REQUIRE_NOT_EQUAL(BulkAttr, nullptr);

	constexpr int32 Count = 20;
	TStaticArray<FVector, Count> Values{};
	TStaticArray<PCGMetadataEntryKey, Count> EntryKeys1;
	TStaticArray<PCGMetadataEntryKey, Count> EntryKeys2;

	for (int32 i = 0; i < Count; ++i)
	{
		EntryKeys1[i] = Domain1->AddEntry();
		EntryKeys2[i] = Domain2->AddEntry();
		Values[i] = GenerateRandom<FVector>(RandomStream);
	}

	// Single set
	for (int32 i = 0; i < Count; ++i)
	{
		SingleAttr->SetValue(EntryKeys1[i], Values[i]);
	}

	// Bulk set
	BulkAttr->SetValues(TConstArrayView<PCGMetadataEntryKey>(EntryKeys2), Values);

	// Read both and compare
	TStaticArray<FVector, Count> ReadSingle{};
	TStaticArray<FVector, Count> ReadBulk{};
	SingleAttr->GetValuesFromItemKeys(TConstArrayView<PCGMetadataEntryKey>(EntryKeys1), ReadSingle);
	BulkAttr->GetValuesFromItemKeys(TConstArrayView<PCGMetadataEntryKey>(EntryKeys2), ReadBulk);

	for (int32 i = 0; i < Count; ++i)
	{
		REQUIRE_EQUAL(ReadSingle[i], Values[i]);
		REQUIRE_EQUAL(ReadBulk[i], Values[i]);
		REQUIRE_EQUAL(ReadSingle[i], ReadBulk[i]);
	}
}

/**
 * Test that unset entry keys return the default value.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeTyped::UnsetEntryReturnsDefault", "[PCG][AttributeTyped]")
{
	using namespace PCGAttributeTypedTests;

	UPCGParamData* ParamData = CreateData<UPCGParamData>();
	FPCGMetadataDomain* Domain = ParamData->MutableMetadata()->GetDefaultMetadataDomain();

	FPCGMetadataAttribute<double>* Attr = Domain->CreateAttribute<double>(AttributeName, 99.9, true, false);
	REQUIRE_NOT_EQUAL(Attr, nullptr);

	// Add entries but don't set any values on them
	PCGMetadataEntryKey Key1 = Domain->AddEntry();
	PCGMetadataEntryKey Key2 = Domain->AddEntry();

	// Reading unset entries should return the default value
	REQUIRE_EQUAL(Attr->GetValueFromItemKey(Key1), 99.9);
	REQUIRE_EQUAL(Attr->GetValueFromItemKey(Key2), 99.9);

	// Set one, the other should still be default
	Attr->SetValue(Key1, 1.0);
	REQUIRE_EQUAL(Attr->GetValueFromItemKey(Key1), 1.0);
	REQUIRE_EQUAL(Attr->GetValueFromItemKey(Key2), 99.9);
}

/**
 * Test that AddValues (bulk) returns correct value keys and that they can be used with GetValue.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeTyped::BulkAddValues", "[PCG][AttributeTyped]")
{
	using namespace PCGAttributeTypedTests;

	UPCGParamData* ParamData = CreateData<UPCGParamData>();
	FPCGMetadataDomain* Domain = ParamData->MutableMetadata()->GetDefaultMetadataDomain();

	FPCGMetadataAttribute<int32>* Attr = Domain->CreateAttribute<int32>(AttributeName, 0, true, false);
	REQUIRE_NOT_EQUAL(Attr, nullptr);

	TArray<int32> ValuesBatch = {10, 20, 30, 40, 50};
	TArray<PCGMetadataValueKey> ValueKeys = Attr->AddValues(ValuesBatch);
	REQUIRE_EQUAL(ValueKeys.Num(), ValuesBatch.Num());

	for (int32 i = 0; i < ValuesBatch.Num(); ++i)
	{
		int32 ReadVal = Attr->GetValue(ValueKeys[i]);
		REQUIRE_EQUAL(ReadVal, ValuesBatch[i]);
	}
}

/**
 * Test that GetValues with multiple value keys works correctly in bulk.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeTyped::BulkGetValues", "[PCG][AttributeTyped]")
{
	using namespace PCGAttributeTypedTests;

	UPCGParamData* ParamData = CreateData<UPCGParamData>();
	FPCGMetadataDomain* Domain = ParamData->MutableMetadata()->GetDefaultMetadataDomain();

	FPCGMetadataAttribute<float>* Attr = Domain->CreateAttribute<float>(AttributeName, 0.0f, true, false);
	REQUIRE_NOT_EQUAL(Attr, nullptr);

	TArray<float> ValuesBatch = {1.0f, 2.0f, 3.0f, 4.0f};
	TArray<PCGMetadataValueKey> ValueKeys = Attr->AddValues(ValuesBatch);

	TArray<float> ReadValues;
	ReadValues.SetNum(ValueKeys.Num());
	Attr->GetValues(ValueKeys, ReadValues);

	for (int32 i = 0; i < ValuesBatch.Num(); ++i)
	{
		REQUIRE_EQUAL(ReadValues[i], ValuesBatch[i]);
	}
}

/**
 * Test CopyExistingAttribute through the domain for typed attributes.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeTyped::CopyExistingAttribute", "[PCG][AttributeTyped]")
{
	using namespace PCGAttributeTypedTests;

	UPCGParamData* ParamData = CreateData<UPCGParamData>();
	FPCGMetadataDomain* Domain = ParamData->MutableMetadata()->GetDefaultMetadataDomain();

	FPCGMetadataAttribute<FVector>* Attr = Domain->CreateAttribute<FVector>(AttributeName, FVector(1.0, 2.0, 3.0), true, false);
	REQUIRE_NOT_EQUAL(Attr, nullptr);

	PCGMetadataEntryKey EntryKey = Domain->AddEntry();
	Attr->SetValue(EntryKey, FVector(10.0, 20.0, 30.0));

	FName CopyName(TEXT("CopiedVecAttr"));
	bool bCopied = Domain->CopyExistingAttribute(AttributeName, CopyName, /*bKeepParent=*/false);
	REQUIRE(bCopied);

	FPCGMetadataAttribute<FVector>* CopiedAttr = Domain->GetMutableTypedAttribute<FVector>(CopyName);
	REQUIRE_NOT_EQUAL(CopiedAttr, nullptr);

	// Default should match
	REQUIRE_EQUAL(CopiedAttr->GetValueFromItemKey(PCGInvalidEntryKey), FVector(1.0, 2.0, 3.0));

	// Values should match
	REQUIRE_EQUAL(CopiedAttr->GetValueFromItemKey(EntryKey), FVector(10.0, 20.0, 30.0));
}

/**
 * Test RenameAttribute through the domain for typed attributes.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeTyped::RenameAttribute", "[PCG][AttributeTyped]")
{
	using namespace PCGAttributeTypedTests;

	UPCGParamData* ParamData = CreateData<UPCGParamData>();
	FPCGMetadataDomain* Domain = ParamData->MutableMetadata()->GetDefaultMetadataDomain();

	FPCGMetadataAttribute<int32>* Attr = Domain->CreateAttribute<int32>(AttributeName, 7, true, false);
	REQUIRE_NOT_EQUAL(Attr, nullptr);

	PCGMetadataEntryKey EntryKey = Domain->AddEntry();
	Attr->SetValue(EntryKey, 42);

	FName NewName(TEXT("RenamedAttr"));
	bool bRenamed = Domain->RenameAttribute(AttributeName, NewName);
	REQUIRE(bRenamed);

	// Old name should not exist
	REQUIRE(!Domain->HasAttribute(AttributeName));

	// New name should exist and have the correct values
	FPCGMetadataAttribute<int32>* RenamedAttr = Domain->GetMutableTypedAttribute<int32>(NewName);
	REQUIRE_NOT_EQUAL(RenamedAttr, nullptr);
	REQUIRE_EQUAL(RenamedAttr->GetValueFromItemKey(PCGInvalidEntryKey), 7);
	REQUIRE_EQUAL(RenamedAttr->GetValueFromItemKey(EntryKey), 42);
}

/**
 * Compressed-array attribute type matrix.
 * Verifies that DoesCompressData()/UsesValueKeys() is enabled for TArray attributes of
 * string-like types, and stays disabled for Set/Map containers and numeric arrays.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeTyped::CompressedArray::TypeMatrix", "[PCG][AttributeTyped]")
{
	UPCGParamData* ParamData = CreateData<UPCGParamData>();
	FPCGMetadataDomain* Domain = ParamData->MutableMetadata()->GetDefaultMetadataDomain();

	SECTION("TArray<FString> compresses")
	{
		FPCGMetadataAttributeBase* Attr = Domain->CreateAttribute<TArray<FString>>(TEXT("StringArr"), TArray<FString>{}, true, false);
		REQUIRE_NOT_EQUAL(Attr, nullptr);
		REQUIRE(Attr->DoesCompressData());
		REQUIRE(Attr->UsesValueKeys());
	}

	SECTION("TArray<FSoftObjectPath> compresses")
	{
		FPCGMetadataAttributeBase* Attr = Domain->CreateAttribute<TArray<FSoftObjectPath>>(TEXT("PathArr"), TArray<FSoftObjectPath>{}, true, false);
		REQUIRE_NOT_EQUAL(Attr, nullptr);
		REQUIRE(Attr->DoesCompressData());
	}

	SECTION("TArray<FSoftClassPath> compresses")
	{
		FPCGMetadataAttributeBase* Attr = Domain->CreateAttribute<TArray<FSoftClassPath>>(TEXT("ClassPathArr"), TArray<FSoftClassPath>{}, true, false);
		REQUIRE_NOT_EQUAL(Attr, nullptr);
		REQUIRE(Attr->DoesCompressData());
	}

	SECTION("TArray<float> does not compress")
	{
		FPCGMetadataAttributeBase* Attr = Domain->CreateAttribute<TArray<float>>(TEXT("FloatArr"), TArray<float>{}, true, false);
		REQUIRE_NOT_EQUAL(Attr, nullptr);
		REQUIRE_FALSE(Attr->DoesCompressData());
	}

	SECTION("TSet<FString> does not compress")
	{
		FPCGMetadataAttributeBase* Attr = Domain->CreateAttribute<TSet<FString>>(TEXT("StringSet"), TSet<FString>{}, false, false);
		REQUIRE_NOT_EQUAL(Attr, nullptr);
		REQUIRE_FALSE(Attr->DoesCompressData());
	}

	SECTION("TMap<int32, FString> does not compress")
	{
		FPCGMetadataAttributeBase* Attr = Domain->CreateAttribute<TMap<int32, FString>>(TEXT("StringMap"), TMap<int32, FString>{}, false, false);
		REQUIRE_NOT_EQUAL(Attr, nullptr);
		REQUIRE_FALSE(Attr->DoesCompressData());
	}
}

/**
 * Compressed-array deduplication within a single bulk SetValues call.
 * Duplicate array payloads submitted in one call should collapse into a single value key,
 * distinct payloads should produce distinct keys, and values must round-trip correctly.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeTyped::CompressedArray::BulkSetDeduplicates", "[PCG][AttributeTyped]")
{
	UPCGParamData* ParamData = CreateData<UPCGParamData>();
	FPCGMetadataDomain* Domain = ParamData->MutableMetadata()->GetDefaultMetadataDomain();
	FPCGMetadataAttributeBase* Attr = Domain->CreateAttribute<TArray<FString>>(TEXT("StringArr"), TArray<FString>{}, true, false);
	REQUIRE_NOT_EQUAL(Attr, nullptr);
	REQUIRE(Attr->DoesCompressData());

	constexpr int32 NumEntries = 10;
	TArray<PCGMetadataEntryKey> EntryKeys;
	EntryKeys.Reserve(NumEntries);
	for (int32 i = 0; i < NumEntries; ++i)
	{
		EntryKeys.Add(Domain->AddEntry());
	}

	TArray<TArray<FString>> InValues;
	InValues.Add({TEXT("hello"), TEXT("world")}); // 0
	InValues.Add({TEXT("unique")});               // 1
	InValues.Add({TEXT("hello"), TEXT("world")}); // 2 -> same as 0
	InValues.Add({TEXT("solo")});                 // 3
	InValues.Add({TEXT("unique")});               // 4 -> same as 1
	InValues.Add({TEXT("hello"), TEXT("world")}); // 5 -> same as 0
	InValues.Add({TEXT("world"), TEXT("hello")}); // 6 -> reverse order, should be different
	InValues.Add({});                             // 7 -> empty
	InValues.Add({});                             // 8 -> same as 7
	InValues.Add({TEXT("world"), TEXT("hello")}); // 9 -> same as 6

	Attr->SetValues<TArray<FString>>(MakeConstArrayView(EntryKeys), MakeConstArrayView(InValues));
	
	// Attribute should have 8 entries since default values are not added to the entry key map.
	// It should also just have 4 values, the 5th one is the default value (empty array)
	REQUIRE_EQUAL(Attr->GetNumberOfEntries(), 8);
	REQUIRE_EQUAL(Attr->GetValueKeyOffsetForChild(), 4);

	const PCGMetadataValueKey K0 = Attr->GetValueKey(EntryKeys[0]);
	const PCGMetadataValueKey K1 = Attr->GetValueKey(EntryKeys[1]);
	const PCGMetadataValueKey K3 = Attr->GetValueKey(EntryKeys[3]);
	const PCGMetadataValueKey K6 = Attr->GetValueKey(EntryKeys[6]);
	const PCGMetadataValueKey K7 = Attr->GetValueKey(EntryKeys[7]);
	
	// Empty array is the default key
	REQUIRE_EQUAL(K7, PCGDefaultValueKey);

	// Duplicates share a key
	REQUIRE_EQUAL(Attr->GetValueKey(EntryKeys[2]), K0);
	REQUIRE_EQUAL(Attr->GetValueKey(EntryKeys[5]), K0);
	REQUIRE_EQUAL(Attr->GetValueKey(EntryKeys[4]), K1);
	REQUIRE_EQUAL(Attr->GetValueKey(EntryKeys[8]), K7);
	REQUIRE_EQUAL(Attr->GetValueKey(EntryKeys[9]), K6);

	// Distinct values get distinct keys
	REQUIRE_NOT_EQUAL(K0, K1);
	REQUIRE_NOT_EQUAL(K0, K3);
	REQUIRE_NOT_EQUAL(K0, K6);
	REQUIRE_NOT_EQUAL(K0, K7);
	REQUIRE_NOT_EQUAL(K1, K3);
	REQUIRE_NOT_EQUAL(K1, K6);
	REQUIRE_NOT_EQUAL(K1, K7);
	REQUIRE_NOT_EQUAL(K3, K6);
	REQUIRE_NOT_EQUAL(K3, K7);
	REQUIRE_NOT_EQUAL(K6, K7);

	// AreValuesEqual reports the expected pairs
	REQUIRE(Attr->AreValuesEqualForEntryKeys(EntryKeys[0], EntryKeys[2]));
	REQUIRE(Attr->AreValuesEqualForEntryKeys(EntryKeys[1], EntryKeys[4]));
	REQUIRE(Attr->AreValuesEqualForEntryKeys(EntryKeys[8], EntryKeys[7]));
	REQUIRE(Attr->AreValuesEqualForEntryKeys(EntryKeys[9], EntryKeys[6]));
	REQUIRE_FALSE(Attr->AreValuesEqualForEntryKeys(EntryKeys[0], EntryKeys[1]));
	REQUIRE_FALSE(Attr->AreValuesEqualForEntryKeys(EntryKeys[0], EntryKeys[3]));
	REQUIRE_FALSE(Attr->AreValuesEqualForEntryKeys(EntryKeys[1], EntryKeys[6]));

	// Values round-trip through GetValuesFromItemKeys
	TArray<TConstArrayView<FString>> Out;
	Out.SetNum(NumEntries);
	Attr->GetValuesFromItemKeys<TConstArrayView<FString>>(MakeConstArrayView(EntryKeys), MakeArrayView(Out));
	for (int32 i = 0; i < NumEntries; ++i)
	{
		REQUIRE_EQUAL(Out[i].Num(), InValues[i].Num());
		for (int32 j = 0; j < Out[i].Num(); ++j)
		{
			REQUIRE_EQUAL(Out[i][j], InValues[i][j]);
		}
	}
}

/**
 * Compressed-array deduplication across successive SetValue calls.
 * A value already present in the attribute's storage should be reused on a subsequent
 * SetValue call rather than allocating a new value key. Exercises the FindValues path
 * against stored (non-default) array payloads.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeTyped::CompressedArray::CrossCallDeduplicates", "[PCG][AttributeTyped]")
{
	UPCGParamData* ParamData = CreateData<UPCGParamData>();
	FPCGMetadataDomain* Domain = ParamData->MutableMetadata()->GetDefaultMetadataDomain();
	FPCGMetadataAttributeBase* Attr = Domain->CreateAttribute<TArray<FString>>(TEXT("StringArr"), TArray<FString>{}, true, false);
	REQUIRE_NOT_EQUAL(Attr, nullptr);

	const TArray<FString> A = { TEXT("alpha"), TEXT("beta") };
	const TArray<FString> B = { TEXT("gamma") };

	// First call seeds the attribute with A and B
	const PCGMetadataEntryKey EntryA0 = Domain->AddEntry();
	const PCGMetadataEntryKey EntryB0 = Domain->AddEntry();
	Attr->SetValue(EntryA0, A);
	Attr->SetValue(EntryB0, B);

	const PCGMetadataValueKey KeyA = Attr->GetValueKey(EntryA0);
	const PCGMetadataValueKey KeyB = Attr->GetValueKey(EntryB0);
	REQUIRE_NOT_EQUAL(KeyA, KeyB);

	// Second call with the same array contents should reuse the existing value keys
	const PCGMetadataEntryKey EntryA1 = Domain->AddEntry();
	const PCGMetadataEntryKey EntryB1 = Domain->AddEntry();
	const TArray<FString> ASecond = { TEXT("alpha"), TEXT("beta") };
	const TArray<FString> BSecond = { TEXT("gamma") };
	Attr->SetValue(EntryA1, ASecond);
	Attr->SetValue(EntryB1, BSecond);

	REQUIRE_EQUAL(Attr->GetValueKey(EntryA1), KeyA);
	REQUIRE_EQUAL(Attr->GetValueKey(EntryB1), KeyB);

	// A brand-new value in a third call still allocates a new key
	const PCGMetadataEntryKey EntryC = Domain->AddEntry();
	const TArray<FString> C = { TEXT("delta"), TEXT("epsilon") };
	Attr->SetValue(EntryC, C);
	const PCGMetadataValueKey KeyC = Attr->GetValueKey(EntryC);
	REQUIRE_NOT_EQUAL(KeyC, KeyA);
	REQUIRE_NOT_EQUAL(KeyC, KeyB);
}

/**
 * Compressed-array deduplication against the attribute's default value.
 * Setting an entry to an array equal to the default should resolve to PCGDefaultValueKey
 * rather than allocating a new storage slot.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeTyped::CompressedArray::DefaultValueMatch", "[PCG][AttributeTyped]")
{
	UPCGParamData* ParamData = CreateData<UPCGParamData>();
	FPCGMetadataDomain* Domain = ParamData->MutableMetadata()->GetDefaultMetadataDomain();
	const TArray<FString> Default = { TEXT("default-0"), TEXT("default-1") };
	FPCGMetadataAttributeBase* Attr = Domain->CreateAttribute<TArray<FString>>(TEXT("StringArr"), Default, true, false);
	REQUIRE_NOT_EQUAL(Attr, nullptr);

	const PCGMetadataEntryKey EntrySameAsDefault = Domain->AddEntry();
	const PCGMetadataEntryKey EntryDifferent = Domain->AddEntry();

	Attr->SetValue(EntrySameAsDefault, Default);
	Attr->SetValue(EntryDifferent, TArray<FString>{TEXT("other")});

	REQUIRE_EQUAL(Attr->GetValueKey(EntrySameAsDefault), PCGDefaultValueKey);
	REQUIRE_NOT_EQUAL(Attr->GetValueKey(EntryDifferent), PCGDefaultValueKey);
}

/**
 * Compressed-array behavior for TArray<FSoftObjectPath>.
 * Verifies that dedup applies identically to path containers: duplicate path arrays
 * collapse to one value key, distinct ones get distinct keys.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributeTyped::CompressedArray::SoftObjectPathArrayDedup", "[PCG][AttributeTyped]")
{
	UPCGParamData* ParamData = CreateData<UPCGParamData>();
	FPCGMetadataDomain* Domain = ParamData->MutableMetadata()->GetDefaultMetadataDomain();
	FPCGMetadataAttributeBase* Attr = Domain->CreateAttribute<TArray<FSoftObjectPath>>(TEXT("PathArr"), TArray<FSoftObjectPath>{}, true, false);
	REQUIRE_NOT_EQUAL(Attr, nullptr);
	REQUIRE(Attr->DoesCompressData());

	const TArray<FSoftObjectPath> A = { FSoftObjectPath(TEXT("/Game/Foo")), FSoftObjectPath(TEXT("/Game/Bar")) };
	const TArray<FSoftObjectPath> ACopy = { FSoftObjectPath(TEXT("/Game/Foo")), FSoftObjectPath(TEXT("/Game/Bar")) };
	const TArray<FSoftObjectPath> B = { FSoftObjectPath(TEXT("/Game/Baz")) };

	const PCGMetadataEntryKey E0 = Domain->AddEntry();
	const PCGMetadataEntryKey E1 = Domain->AddEntry();
	const PCGMetadataEntryKey E2 = Domain->AddEntry();

	Attr->SetValue(E0, A);
	Attr->SetValue(E1, ACopy);
	Attr->SetValue(E2, B);

	REQUIRE_EQUAL(Attr->GetValueKey(E0), Attr->GetValueKey(E1));
	REQUIRE_NOT_EQUAL(Attr->GetValueKey(E0), Attr->GetValueKey(E2));
}
