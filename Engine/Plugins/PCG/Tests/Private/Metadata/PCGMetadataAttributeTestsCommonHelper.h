// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "PCGTestsCommon.h"

#include "PCGCommon.h"
#include "Metadata/PCGMetadataContainerTypes.h"

namespace PCGAttributeTestsCommonHelper
{
	/**
	 * Base class for a tester that encapsulate everything that is type specific.
	 */
	template <typename T, typename InReadType = T>
	struct TypedAttributeTester
	{
		using Type = T;
		using ReadType = InReadType;

		TypedAttributeTester(TFunction<void(FRandomStream&, T&)> InGenerateRandom, 
			TFunction<bool(const T&, const ReadType&)> InVerify, 
			T InDefaultValue,
			FPCGMetadataAttributeDesc InExpectedDesc)
			: GenerateRandom(MoveTemp(InGenerateRandom))
			, Verify(MoveTemp(InVerify))
			, DefaultValue(InDefaultValue)
			, ExpectedDesc(InExpectedDesc)
		{}

		TFunction<void(FRandomStream&, T&)> GenerateRandom;
		TFunction<bool(const T&, const ReadType&)> Verify;
		T DefaultValue;
		FPCGMetadataAttributeDesc ExpectedDesc;
		TOptional<FPCGMetadataAttributeDesc> OverrideDesc;
	};
	
	static const FName AttributeName = "Attr";
	static const FName AttributeName2 = "Attr2";
	static const FName AttributeName3 = "Attr3";
	
	/** Getter to create a tester for different types. */
	inline TypedAttributeTester<float> FloatTester()
	{
		auto GenerateRandom = [](FRandomStream& RandomStream, float& OutValue) { OutValue = RandomStream.FRand(); };
		
		auto Verify = [](const float& LHS, const float& RHS) { return LHS == RHS; };
		
		float DefaultValue = 5.0f;
		
		FPCGMetadataAttributeDesc ExpectedDesc
		{
			.Name = AttributeName,
			.ValueType = EPCGMetadataTypes::Float
		};
		
		return TypedAttributeTester<float>(GenerateRandom, Verify, DefaultValue, MoveTemp(ExpectedDesc));
	}
	
	inline TypedAttributeTester<double> DoubleTester()
	{
		auto GenerateRandom = [](FRandomStream& RandomStream, double& OutValue) { OutValue = RandomStream.FRand(); };
		
		auto Verify = [](const double& LHS, const double& RHS) { return LHS == RHS; };
		
		double DefaultValue = 5.0;
		
		FPCGMetadataAttributeDesc ExpectedDesc
		{
			.Name = AttributeName,
			.ValueType = EPCGMetadataTypes::Double
		};
		
		return TypedAttributeTester<double>(GenerateRandom, Verify, DefaultValue, MoveTemp(ExpectedDesc));
	}
	
	inline TypedAttributeTester<FVector> VectorTester_Basic()
	{
		auto GenerateRandom = [](FRandomStream& RandomStream, FVector& OutValue) { OutValue = RandomStream.VRand(); };
		
		auto Verify = [](const FVector& LHS, const FVector& RHS) { return LHS == RHS; };
		
		FVector DefaultValue = FVector(1.0, 2.0, 3.0);
		
		FPCGMetadataAttributeDesc ExpectedDesc
		{
			.Name = AttributeName,
			.ValueType = EPCGMetadataTypes::Vector
		};
			
		return TypedAttributeTester<FVector>{GenerateRandom, Verify, DefaultValue, MoveTemp(ExpectedDesc)};
	}
		
	inline TypedAttributeTester<FVector> VectorTester_Struct()
	{
		TypedAttributeTester<FVector> AttributeTester = VectorTester_Basic();
		FPCGMetadataAttributeDesc StructDesc
		{
			.Name = AttributeName,
			.ValueType = EPCGMetadataTypes::Struct,
			.ValueTypeObject = TBaseStructure<FVector>::Get()
		};
		
		AttributeTester.OverrideDesc = StructDesc;
	
		return AttributeTester;
	}
	
	inline TypedAttributeTester<FString, const FString*> StringTester()
	{
		auto GenerateRandom = [](FRandomStream& RandomStream, FString& OutValue)
			{
				static int i = 0; 
				OutValue = FString::Printf(TEXT("This is a string with some characters and a value: %d"), i++ % 25);
			};
		
		auto Verify = [](const FString& LHS, FString const* const& RHS) { return LHS == *RHS; };
		
		FString DefaultValue = TEXT("Hi");
		
		FPCGMetadataAttributeDesc ExpectedDesc
		{
			.Name = AttributeName,
			.ValueType = EPCGMetadataTypes::String
		};
		
		return TypedAttributeTester<FString, const FString*>(GenerateRandom, Verify, MoveTemp(DefaultValue), MoveTemp(ExpectedDesc));
	}
	
	inline TypedAttributeTester<FPCGPoint, const FPCGPoint*> FPCGPointTester()
	{		
		auto GenerateRandom = [](FRandomStream& RandomStream, FPCGPoint& OutValue)
			{
				OutValue.Density = RandomStream.FRand();
				OutValue.Transform.SetLocation(RandomStream.VRand());
			};
		
		auto Verify = [](const FPCGPoint& LHS, FPCGPoint const* const& RHS)
			{
				return LHS.Density == RHS->Density && LHS.Transform.GetLocation() == RHS->Transform.GetLocation();
			};
		
		FPCGPoint DefaultValue{FTransform{5.0 * FVector::OneVector}, 0.4f, 42};
		
		FPCGMetadataAttributeDesc ExpectedDesc
		{
			.Name = AttributeName,
			.ValueType = EPCGMetadataTypes::Struct,
			.ValueTypeObject = FPCGPoint::StaticStruct()
		};
		
		return TypedAttributeTester<FPCGPoint, const FPCGPoint*>(GenerateRandom, Verify, MoveTemp(DefaultValue), MoveTemp(ExpectedDesc));
	}

	inline TypedAttributeTester<TArray<double>, TConstArrayView<double>> ArrayDoubleTester()
	{
		auto GenerateRandom = [](FRandomStream& RandomStream, TArray<double>& OutValue)
		{
			const int32 RandomCount = RandomStream.RandRange(1, 4);
			for (int j = 0; j < RandomCount; ++j)
			{
				OutValue.Add(RandomStream.FRand());
			}
		};
		
		auto Verify = [](const TArray<double>& LHS, const TConstArrayView<double>& RHS)
			{
				if (LHS.Num() != RHS.Num())
				{
					return false;
				}
				
				for (int i = 0; i < LHS.Num(); ++i)
				{
					if (LHS[i] != RHS[i])
					{
						return false;
					}
				}
				
				return true;
			};
		
		TArray<double> DefaultValue = {1, 2, 3, 4, 5};
		
		FPCGMetadataAttributeDesc ExpectedDesc
		{
			.Name = AttributeName,
			.ValueType = EPCGMetadataTypes::Double,
			.ContainerTypes = {EPCGMetadataAttributeContainerTypes::Array}
		};
		
		return TypedAttributeTester<TArray<double>, TConstArrayView<double>>(GenerateRandom, Verify, MoveTemp(DefaultValue), MoveTemp(ExpectedDesc));
	}

	inline TypedAttributeTester<TArray<double>, PCG::TPCGArrayAccessorWrapper<double>> ArrayAccessorDoubleTester()
	{
		auto GenerateRandom = [](FRandomStream& RandomStream, TArray<double>& OutValue)
		{
			const int32 RandomCount = RandomStream.RandRange(1, 4);
			for (int j = 0; j < RandomCount; ++j)
			{
				OutValue.Add(RandomStream.FRand());
			}
		};

		auto Verify = [](const TArray<double>& LHS, const PCG::TPCGArrayAccessorWrapper<double>& RHS)
		{
			if (LHS.Num() != RHS.GetView().Num())
			{
				return false;
			}

			for (int i = 0; i < LHS.Num(); ++i)
			{
				if (LHS[i] != RHS.GetView()[i])
				{
					return false;
				}
			}

			return true;
		};

		TArray<double> DefaultValue = {1, 2, 3, 4, 5};

		FPCGMetadataAttributeDesc ExpectedDesc
		{
			.Name = AttributeName,
			.ValueType = EPCGMetadataTypes::Double,
			.ContainerTypes = {EPCGMetadataAttributeContainerTypes::Array}
		};

		return TypedAttributeTester<TArray<double>, PCG::TPCGArrayAccessorWrapper<double>>(GenerateRandom, Verify, MoveTemp(DefaultValue), MoveTemp(ExpectedDesc));
	}

	inline TypedAttributeTester<TArray<FPCGPoint>, TConstArrayView<FPCGPoint>> ArrayFPCGPointTester()
	{
		auto GenerateRandom = [](FRandomStream& RandomStream, TArray<FPCGPoint>& OutValue)
		{
			const int32 RandomCount = RandomStream.RandRange(1, 4);
			for (int j = 0; j < RandomCount; ++j)
			{
				FPCGPoint& Value = OutValue.Emplace_GetRef();
				Value.Density = RandomStream.FRand();
				Value.Transform.SetLocation(RandomStream.VRand());
			}
		};
		
		auto Verify = [](const TArray<FPCGPoint>& LHS, const TConstArrayView<FPCGPoint>& RHS)
		{
			if (LHS.Num() != RHS.Num())
			{
				return false;
			}
			
			for (int i = 0; i < LHS.Num(); ++i)
			{
				if (LHS[i].Density != RHS[i].Density || LHS[i].Transform.GetLocation() != RHS[i].Transform.GetLocation())
				{
					return false;
				}
			}
			
			return true;
		};
		
		TArray<FPCGPoint> DefaultValue = {
			{FTransform{5.0 * FVector::OneVector}, 0.4f, 42},
			{FTransform{6.0 * FVector::OneVector}, 0.5f, 43},
			{FTransform{7.0 * FVector::OneVector}, 0.6f, 44},
		};
		
		FPCGMetadataAttributeDesc ExpectedDesc
		{
			.Name = AttributeName,
			.ValueType = EPCGMetadataTypes::Struct,
			.ContainerTypes = {EPCGMetadataAttributeContainerTypes::Array},
			.ValueTypeObject = FPCGPoint::StaticStruct()
		};
		
		return TypedAttributeTester<TArray<FPCGPoint>, TConstArrayView<FPCGPoint>>(GenerateRandom, Verify, MoveTemp(DefaultValue), MoveTemp(ExpectedDesc));
	}

	inline TypedAttributeTester<TArray<FPCGPoint>, PCG::TPCGArrayAccessorWrapper<FPCGPoint>> ArrayAccessorFPCGPointTester()
	{
		auto GenerateRandom = [](FRandomStream& RandomStream, TArray<FPCGPoint>& OutValue)
		{
			const int32 RandomCount = RandomStream.RandRange(1, 4);
			for (int j = 0; j < RandomCount; ++j)
			{
				FPCGPoint& Value = OutValue.Emplace_GetRef();
				Value.Density = RandomStream.FRand();
				Value.Transform.SetLocation(RandomStream.VRand());
			}
		};

		auto Verify = [](const TArray<FPCGPoint>& LHS, const PCG::TPCGArrayAccessorWrapper<FPCGPoint>& RHS)
		{
			if (LHS.Num() != RHS.GetView().Num())
			{
				return false;
			}

			for (int i = 0; i < LHS.Num(); ++i)
			{
				if (LHS[i].Density != RHS.GetView()[i].Density || LHS[i].Transform.GetLocation() != RHS.GetView()[i].Transform.GetLocation())
				{
					return false;
				}
			}

			return true;
		};

		TArray<FPCGPoint> DefaultValue = {
			{FTransform{5.0 * FVector::OneVector}, 0.4f, 42},
			{FTransform{6.0 * FVector::OneVector}, 0.5f, 43},
			{FTransform{7.0 * FVector::OneVector}, 0.6f, 44},
		};

		FPCGMetadataAttributeDesc ExpectedDesc
		{
			.Name = AttributeName,
			.ValueType = EPCGMetadataTypes::Struct,
			.ContainerTypes = {EPCGMetadataAttributeContainerTypes::Array},
			.ValueTypeObject = FPCGPoint::StaticStruct()
		};

		return TypedAttributeTester<TArray<FPCGPoint>, PCG::TPCGArrayAccessorWrapper<FPCGPoint>>(GenerateRandom, Verify, MoveTemp(DefaultValue), MoveTemp(ExpectedDesc));
	}
	
	inline TypedAttributeTester<TSet<double>, PCG::TScriptSetWrapper<double>> SetDoubleTester()
	{		
		auto GenerateRandom = [](FRandomStream& RandomStream, TSet<double>& OutValue)
			{
				const int32 RandomCount = RandomStream.RandRange(1, 4);
				for (int j = 0; j < RandomCount; ++j)
				{
					OutValue.Add(RandomStream.FRand());
				}
			};
		
		auto Verify = [](const TSet<double>& LHS, const PCG::TScriptSetWrapper<double>& RHS)
			{
				if (LHS.Num() != RHS.Num())
				{
					return false;
				}
				
				for (double Value : LHS)
				{
					if (!RHS.Contains(Value))
					{
						return false;
					}
				}
				
				return true;
			};
		
		TSet<double> DefaultValue = {1, 2, 3, 4, 5};
		
		FPCGMetadataAttributeDesc ExpectedDesc
		{
			.Name = AttributeName,
			.ValueType = EPCGMetadataTypes::Double,
			.ContainerTypes = {EPCGMetadataAttributeContainerTypes::Set}
		};
		
		return TypedAttributeTester<TSet<double>, PCG::TScriptSetWrapper<double>>(GenerateRandom, Verify, MoveTemp(DefaultValue), MoveTemp(ExpectedDesc));
	}
	
	inline TypedAttributeTester<TMap<FString, double>, PCG::TScriptMapWrapper<FString, double>> MapStringDoubleTester()
	{
		auto GenerateRandom = [](FRandomStream& RandomStream, TMap<FString, double>& OutValue)
			{
				const int32 RandomCount = RandomStream.RandRange(1, 4);
				for (int j = 0; j < RandomCount; ++j)
				{
					OutValue.Emplace(LexToString(j), RandomStream.FRand());
				}
			};
		
		auto Verify = [](const TMap<FString, double>& LHS, const PCG::TScriptMapWrapper<FString, double>& RHS)
			{
				if (LHS.Num() != RHS.Num())
				{
					return false;
				}
				
				for (const auto& [Key, Value] : LHS)
				{
					const double* OtherValue = RHS.Find(Key);
					if (!OtherValue || Value != *OtherValue)
					{
						return false;
					}
				}
				
				return true;
			};
		
		TMap<FString, double> DefaultValue =
		{
			{TEXT("Hi"), 2.0},
			{TEXT("Ho"), 3.0}
		};
		
		FPCGMetadataAttributeDesc ExpectedDesc
		{
			.Name = AttributeName,
			.ValueType = EPCGMetadataTypes::Double,
			.ContainerTypes = {EPCGMetadataAttributeContainerTypes::Map},
			.KeyType = EPCGMetadataTypes::String
		};
		
		return TypedAttributeTester<TMap<FString, double>, PCG::TScriptMapWrapper<FString, double>>(GenerateRandom, Verify, MoveTemp(DefaultValue), MoveTemp(ExpectedDesc));
	}
	
	inline TypedAttributeTester<EPCGMetadataTypes> EnumTester()
	{
		auto GenerateRandom = [](FRandomStream& RandomStream, EPCGMetadataTypes& OutValue)
			{
				// Make sure to not generate a 0, to differentiate from an unset value.
				OutValue = static_cast<EPCGMetadataTypes>(RandomStream.RandRange(1, static_cast<int32>(EPCGMetadataTypes::Count)));
			};
		
		auto Verify = [](const EPCGMetadataTypes& LHS, const EPCGMetadataTypes& RHS) { return LHS == RHS; };
		
		EPCGMetadataTypes DefaultValue = EPCGMetadataTypes::Unknown;
		
		FPCGMetadataAttributeDesc ExpectedDesc
		{
			.Name = AttributeName,
			.ValueType = EPCGMetadataTypes::Enum,
			.ValueTypeObject = StaticEnum<EPCGMetadataTypes>()
		};
		
		return TypedAttributeTester<EPCGMetadataTypes>(GenerateRandom, Verify, DefaultValue, MoveTemp(ExpectedDesc));
	}
}
