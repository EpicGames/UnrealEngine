// Copyright Epic Games, Inc. All Rights Reserved.

// =============================================================================
// PCG Metadata Container Types Tests
//
// Tests for the container wrapper types used by PCG generic metadata attributes
// and by the accessor system to pass typed array data through a type-erased API.
//
// Test summary:
//  - PCG::MetadataContainers:                                  Read-only API of TScriptSetWrapper and TScriptMapWrapper on top of TSet/TMap generic attributes
//  - PCG::MetadataContainers::TPCGArrayAccessorWrapper:        Allocation, steal, copy and move semantics of the templated array wrapper (owned memory vs external view)
//  - PCG::MetadataContainers::FPCGAccessorBuffer:              Type-erased Buffer lifecycle: setup/allocate, reset, copy, move, steal (matching / mismatched type)
// =============================================================================

#include "PCGTestsCommon.h"

#include "Data/PCGPointArrayData.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include <catch2/catch_test_macros.hpp>
#include <catch2/benchmark/catch_benchmark.hpp>
#include <catch2/generators/catch_generators_range.hpp>

#include "TestHarness.h"

// Verifies that the script-helper-backed wrappers (TScriptSetWrapper, TScriptMapWrapper) expose the expected
// read-only API on top of TSet/TMap attributes populated via the generic metadata path.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::MetadataContainers", "[PCG][MetadataContainers]")
{
	// Since the containers are meant to be created by the generic attributes, we'll use the generic attributes
	// to populate them
	
	// Preparing the data
	constexpr int32 NumPoints = 1;
	UPCGPointArrayData* PointArrayData = CreateData<UPCGPointArrayData>();

	PointArrayData->SetNumPoints(NumPoints);

	PointArrayData->AllocateProperties(EPCGPointNativeProperties::MetadataEntry);
	TPCGValueRange<int64> PointArrayDataMetadataEntryRange = PointArrayData->GetMetadataEntryValueRange(/*bAllocate=*/false);
	
	FPCGMetadataDomain* Domain = PointArrayData->MutableMetadata()->GetDefaultMetadataDomain();
	PointArrayDataMetadataEntryRange[0] = Domain->AddEntry();
	
	SECTION("TScriptSetWrapper")
	{
		FPCGMetadataAttributeBase* SetGenericAttribute = Domain->CreateAttribute<TSet<double>>("SetAttr", TSet<double>{});
		REQUIRE_NOT_EQUAL(SetGenericAttribute, nullptr);
		
		const TSet<double> Values = {1.0, 2.0, 5.0, 3.0};
		
		SetGenericAttribute->SetValues<TSet<double>>(PointArrayDataMetadataEntryRange, {Values});
		PCG::TScriptSetWrapper<double> SetWrapper{};
		SetGenericAttribute->GetValuesFromItemKeys<PCG::TScriptSetWrapper<double>>(PCGValueRangeHelpers::MakeConstValueRange(PointArrayDataMetadataEntryRange), MakeArrayView(&SetWrapper, 1));
		
		REQUIRE(SetWrapper.IsValid());
		
		SECTION("Num")
		{
			REQUIRE(SetWrapper.Num() == Values.Num());
		}
		
		SECTION("Contains")
		{
			REQUIRE(SetWrapper.Contains(1.0));
			REQUIRE(SetWrapper.Contains(2.0));
			REQUIRE(SetWrapper.Contains(5.0));
			REQUIRE(SetWrapper.Contains(3.0));
			REQUIRE_FALSE(SetWrapper.Contains(4.0));
			REQUIRE_FALSE(SetWrapper.Contains(6.0));
		}
		
		SECTION("ForEach")
		{
			double Sum = 0.0;
			SetWrapper.ForEach([&Sum](double Value) { Sum += Value; });
			REQUIRE_EQUAL(Sum, 11.0);
		}
		
		SECTION("Intersection")
		{
			const TSet<double> Other = {2.0, 5.0, 4.0};
			TSet<double> Intersection = SetWrapper.Intersection(Other);
			REQUIRE_EQUAL(Intersection.Num(), 2);
			REQUIRE_FALSE(Intersection.Contains(1.0));
			REQUIRE(Intersection.Contains(2.0));
			REQUIRE(Intersection.Contains(5.0));
			REQUIRE_FALSE(Intersection.Contains(3.0));
			REQUIRE_FALSE(Intersection.Contains(4.0));
			REQUIRE_FALSE(Intersection.Contains(6.0));
		}
		
		SECTION("Union")
		{
			const TSet<double> Other = {2.0, 5.0, 4.0};
			TSet<double> Union = SetWrapper.Union(Other);
			REQUIRE_EQUAL(Union.Num(), 5);
			REQUIRE(Union.Contains(1.0));
			REQUIRE(Union.Contains(2.0));
			REQUIRE(Union.Contains(5.0));
			REQUIRE(Union.Contains(3.0));
			REQUIRE(Union.Contains(4.0));
			REQUIRE_FALSE(Union.Contains(6.0));
		}

		SECTION("Copy")
		{
			TSet<double> Copy = static_cast<TSet<double>>(SetWrapper);
			REQUIRE_EQUAL(Copy.Num(), 4);
			REQUIRE(Copy.Contains(1.0));
			REQUIRE(Copy.Contains(2.0));
			REQUIRE(Copy.Contains(5.0));
			REQUIRE(Copy.Contains(3.0));
			REQUIRE_FALSE(Copy.Contains(4.0));
			REQUIRE_FALSE(Copy.Contains(6.0));
		}
	}
	
	SECTION("TScriptMapWrapper")
	{
		FPCGMetadataAttributeBase* MapGenericAttribute = Domain->CreateAttribute<TMap<FString, double>>("MapAttr", TMap<FString, double>{});
		REQUIRE_NOT_EQUAL(MapGenericAttribute, nullptr);
		
		TMap<FString, double> Values = 
		{
			{TEXT("Hey"), 2.0},
			{TEXT("Ho"), 5.0}
		};
		
		MapGenericAttribute->SetValues<TMap<FString, double>>(PointArrayDataMetadataEntryRange, {Values});
		PCG::TScriptMapWrapper<FString, double> MapWrapper{};
		MapGenericAttribute->GetValuesFromItemKeys<PCG::TScriptMapWrapper<FString, double>>(PCGValueRangeHelpers::MakeConstValueRange(PointArrayDataMetadataEntryRange), MakeArrayView(&MapWrapper, 1));
		
		REQUIRE(MapWrapper.IsValid());
		
		SECTION("Num")
		{
			REQUIRE(MapWrapper.Num() == Values.Num());
		}
		
		SECTION("Contains")
		{
			REQUIRE(MapWrapper.Contains(TEXT("Hey")));
			REQUIRE(MapWrapper.Contains(TEXT("Ho")));
			REQUIRE_FALSE(MapWrapper.Contains(TEXT("Ah")));
		}
		
		SECTION("Find")
		{
			REQUIRE_EQUAL(*MapWrapper.Find(TEXT("Hey")), 2.0);
			REQUIRE_EQUAL(*MapWrapper.Find(TEXT("Ho")), 5.0);
			REQUIRE_EQUAL(MapWrapper.Find(TEXT("Ah")), nullptr);
		}
		
		SECTION("operator[]")
		{
			REQUIRE_EQUAL(MapWrapper[TEXT("Hey")], 2.0);
			REQUIRE_EQUAL(MapWrapper[TEXT("Ho")], 5.0);
		}
		
		SECTION("Copy")
		{
			TMap<FString, double> Copy = static_cast<TMap<FString, double>>(MapWrapper);
			REQUIRE_EQUAL(Copy.Num(), 2);
			REQUIRE_EQUAL(Copy[TEXT("Hey")], 2.0);
			REQUIRE_EQUAL(Copy[TEXT("Ho")], 5.0);
			REQUIRE_FALSE(Copy.Contains(TEXT("Ah")));
		}
		
		SECTION("ForEach")
		{
			FString SumString{};
			double Sum = 0.0;
			
			MapWrapper.ForEach([&SumString, &Sum](const FString& Key, const double& Value)
				{
					SumString += Key;
					Sum += Value;
				});
				
			REQUIRE_EQUAL(SumString, TEXT("HeyHo"));
			REQUIRE_EQUAL(Sum, 7.0);
		}
	}
}

// Verifies the allocation, ownership, steal and copy/move semantics of TPCGArrayAccessorWrapper in both
// its owned-memory mode (converted array reads) and its external-view mode (same-type reads into attribute memory).
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::MetadataContainers::TPCGArrayAccessorWrapper", "[PCG][MetadataContainers]")
{
	SECTION("AllocateOwnMemory sized")
	{
		PCG::TPCGArrayAccessorWrapper<double> Wrapper;
		TArrayView<double> Writable = Wrapper.AllocateOwnMemory(4);
		REQUIRE(Wrapper.IsOwningMemory());
		REQUIRE_EQUAL(Writable.Num(), 4);
		REQUIRE_EQUAL(Wrapper.GetView().Num(), 4);

		Writable[0] = 1.0;
		Writable[1] = 2.0;
		Writable[2] = 3.0;
		Writable[3] = 4.0;
		REQUIRE_EQUAL(Wrapper.GetView()[0], 1.0);
		REQUIRE_EQUAL(Wrapper.GetView()[3], 4.0);
	}

	SECTION("AllocateOwnMemory zero resets state")
	{
		PCG::TPCGArrayAccessorWrapper<double> Wrapper;
		Wrapper.AllocateOwnMemory(4);
		REQUIRE(Wrapper.IsOwningMemory());

		TArrayView<double> Writable = Wrapper.AllocateOwnMemory(0);
		REQUIRE_FALSE(Wrapper.IsOwningMemory());
		REQUIRE_EQUAL(Writable.Num(), 0);
		REQUIRE_EQUAL(Wrapper.GetView().Num(), 0);
	}

	SECTION("StealMemory transfers TArray and clears state")
	{
		PCG::TPCGArrayAccessorWrapper<double> Wrapper;
		TArrayView<double> Writable = Wrapper.AllocateOwnMemory(3);
		Writable[0] = 10.0;
		Writable[1] = 20.0;
		Writable[2] = 30.0;

		TArray<double> Stolen = Wrapper.StealMemory();
		REQUIRE_FALSE(Wrapper.IsOwningMemory());
		REQUIRE_EQUAL(Wrapper.GetView().Num(), 0);
		REQUIRE_EQUAL(Stolen.Num(), 3);
		REQUIRE_EQUAL(Stolen[0], 10.0);
		REQUIRE_EQUAL(Stolen[1], 20.0);
		REQUIRE_EQUAL(Stolen[2], 30.0);
	}

	SECTION("Copy ctor (owned) deep-copies memory and repoints View")
	{
		PCG::TPCGArrayAccessorWrapper<double> Source;
		TArrayView<double> Writable = Source.AllocateOwnMemory(3);
		Writable[0] = 1.0;
		Writable[1] = 2.0;
		Writable[2] = 3.0;

		PCG::TPCGArrayAccessorWrapper<double> Copy{Source};
		REQUIRE(Copy.IsOwningMemory());
		REQUIRE_EQUAL(Copy.GetView().Num(), 3);
		REQUIRE_NOT_EQUAL(Copy.GetView().GetData(), Source.GetView().GetData());
		REQUIRE_EQUAL(Copy.GetView()[0], 1.0);
		REQUIRE_EQUAL(Copy.GetView()[1], 2.0);
		REQUIRE_EQUAL(Copy.GetView()[2], 3.0);
	}

	SECTION("Copy ctor (external view) preserves external pointer")
	{
		const TArray<double> External = {100.0, 200.0, 300.0};
		PCG::TPCGArrayAccessorWrapper<double> Source;
		Source.SetupView(MakeConstArrayView(External));

		PCG::TPCGArrayAccessorWrapper<double> Copy{Source};
		REQUIRE_FALSE(Copy.IsOwningMemory());
		REQUIRE_EQUAL(Copy.GetView().Num(), 3);
		REQUIRE_EQUAL(Copy.GetView().GetData(), External.GetData());
	}

	SECTION("Copy assignment (owned) replaces prior state")
	{
		PCG::TPCGArrayAccessorWrapper<double> Source;
		TArrayView<double> Writable = Source.AllocateOwnMemory(2);
		Writable[0] = 5.0;
		Writable[1] = 6.0;

		PCG::TPCGArrayAccessorWrapper<double> Dest;
		Dest.AllocateOwnMemory(4);
		Dest = Source;

		REQUIRE(Dest.IsOwningMemory());
		REQUIRE_EQUAL(Dest.GetView().Num(), 2);
		REQUIRE_NOT_EQUAL(Dest.GetView().GetData(), Source.GetView().GetData());
		REQUIRE_EQUAL(Dest.GetView()[0], 5.0);
		REQUIRE_EQUAL(Dest.GetView()[1], 6.0);
	}

	SECTION("Copy assignment (external view) replaces prior owned state")
	{
		const TArray<double> External = {7.0, 8.0};
		PCG::TPCGArrayAccessorWrapper<double> Source;
		Source.SetupView(MakeConstArrayView(External));

		PCG::TPCGArrayAccessorWrapper<double> Dest;
		Dest.AllocateOwnMemory(3);
		Dest = Source;

		REQUIRE_FALSE(Dest.IsOwningMemory());
		REQUIRE_EQUAL(Dest.GetView().Num(), 2);
		REQUIRE_EQUAL(Dest.GetView().GetData(), External.GetData());
	}

	SECTION("Move ctor (owned)")
	{
		PCG::TPCGArrayAccessorWrapper<double> Source;
		TArrayView<double> Writable = Source.AllocateOwnMemory(3);
		Writable[0] = 1.0;
		Writable[1] = 2.0;
		Writable[2] = 3.0;

		PCG::TPCGArrayAccessorWrapper<double> Moved{MoveTemp(Source)};
		REQUIRE(Moved.IsOwningMemory());
		REQUIRE_FALSE(Source.IsOwningMemory());
		REQUIRE_EQUAL(Moved.GetView().Num(), 3);
		REQUIRE_EQUAL(Source.GetView().Num(), 0);
		REQUIRE_EQUAL(Moved.GetView()[0], 1.0);
		REQUIRE_EQUAL(Moved.GetView()[2], 3.0);
	}

	SECTION("Move ctor (external view)")
	{
		const TArray<double> External = {9.0, 10.0};
		PCG::TPCGArrayAccessorWrapper<double> Source;
		Source.SetupView(MakeConstArrayView(External));

		PCG::TPCGArrayAccessorWrapper<double> Moved{MoveTemp(Source)};
		REQUIRE_FALSE(Moved.IsOwningMemory());
		REQUIRE_EQUAL(Moved.GetView().Num(), 2);
		REQUIRE_EQUAL(Moved.GetView().GetData(), External.GetData());
		REQUIRE_EQUAL(Source.GetView().Num(), 0);
	}

	SECTION("Move assignment (owned)")
	{
		PCG::TPCGArrayAccessorWrapper<double> Source;
		TArrayView<double> Writable = Source.AllocateOwnMemory(2);
		Writable[0] = 11.0;
		Writable[1] = 22.0;

		PCG::TPCGArrayAccessorWrapper<double> Dest;
		Dest.AllocateOwnMemory(5);
		Dest = MoveTemp(Source);

		REQUIRE(Dest.IsOwningMemory());
		REQUIRE_FALSE(Source.IsOwningMemory());
		REQUIRE_EQUAL(Dest.GetView().Num(), 2);
		REQUIRE_EQUAL(Dest.GetView()[0], 11.0);
		REQUIRE_EQUAL(Dest.GetView()[1], 22.0);
	}

	SECTION("Move assignment (external view)")
	{
		const TArray<double> External = {33.0};
		PCG::TPCGArrayAccessorWrapper<double> Source;
		Source.SetupView(MakeConstArrayView(External));

		PCG::TPCGArrayAccessorWrapper<double> Dest;
		Dest.AllocateOwnMemory(3);
		Dest = MoveTemp(Source);

		REQUIRE_FALSE(Dest.IsOwningMemory());
		REQUIRE_EQUAL(Dest.GetView().Num(), 1);
		REQUIRE_EQUAL(Dest.GetView().GetData(), External.GetData());
		REQUIRE_EQUAL(Source.GetView().Num(), 0);
	}
}

// Verifies the type-erased FPCGAccessorBuffer lifecycle — allocation, ownership transfer, copy and move
// semantics (including self-assignment guards) and StealMemory's type-match safeguard.
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::MetadataContainers::FPCGAccessorBuffer", "[PCG][MetadataContainers]")
{
	SECTION("Default construction is empty")
	{
		PCG::FPCGAccessorBuffer Buffer;
		REQUIRE_FALSE(Buffer.IsOwningMemory());
		REQUIRE_EQUAL(Buffer.GetOwnedMemoryPtr(), nullptr);
		REQUIRE_FALSE(Buffer.GetUnderlyingDesc().IsValid());
	}

	SECTION("SetupAndAllocate trivially-copyable (double)")
	{
		PCG::FPCGAccessorBuffer Buffer;
		Buffer.SetupAndAllocate(4, PCG::Private::GetDefaultAttributeDesc<double>());
		REQUIRE(Buffer.IsOwningMemory());
		REQUIRE(Buffer.GetUnderlyingDesc().IsSameType(PCG::Private::GetDefaultAttributeDesc<double>()));
		REQUIRE_NOT_EQUAL(Buffer.GetOwnedMemoryPtr(), nullptr);

		double* Values = static_cast<double*>(Buffer.GetOwnedMemoryPtr());
		Values[0] = 1.0;
		Values[1] = 2.0;
		Values[2] = 3.0;
		Values[3] = 4.0;
		REQUIRE_EQUAL(Values[0], 1.0);
		REQUIRE_EQUAL(Values[3], 4.0);
	}

	SECTION("SetupAndAllocate non-trivial (FString)")
	{
		PCG::FPCGAccessorBuffer Buffer;
		Buffer.SetupAndAllocate(3, PCG::Private::GetDefaultAttributeDesc<FString>());
		REQUIRE(Buffer.IsOwningMemory());
		REQUIRE(Buffer.GetUnderlyingDesc().IsSameType(PCG::Private::GetDefaultAttributeDesc<FString>()));

		FString* Values = static_cast<FString*>(Buffer.GetOwnedMemoryPtr());
		Values[0] = TEXT("Hello");
		Values[1] = TEXT("World");
		Values[2] = TEXT("Test");
		REQUIRE_EQUAL(Values[0], TEXT("Hello"));
		REQUIRE_EQUAL(Values[2], TEXT("Test"));

		// Exercise the destructor path.
		Buffer.Reset();
		REQUIRE_FALSE(Buffer.IsOwningMemory());
	}

	SECTION("SetupAndAllocate with invalid type is a no-op")
	{
		PCG::FPCGAccessorBuffer Buffer;
		Buffer.SetupAndAllocate(4, FPCGMetadataAttributeDesc{});
		REQUIRE_FALSE(Buffer.IsOwningMemory());
		REQUIRE_EQUAL(Buffer.GetOwnedMemoryPtr(), nullptr);
	}

	SECTION("Reset clears state")
	{
		PCG::FPCGAccessorBuffer Buffer;
		Buffer.SetupAndAllocate(3, PCG::Private::GetDefaultAttributeDesc<double>());
		REQUIRE(Buffer.IsOwningMemory());

		Buffer.Reset();
		REQUIRE_FALSE(Buffer.IsOwningMemory());
		REQUIRE_EQUAL(Buffer.GetOwnedMemoryPtr(), nullptr);
	}

	// Uses FSoftObjectPath (non-trivially copyable) to exercise the construct + copy paths on the underlying memory.
	SECTION("Copy ctor deep-copies memory")
	{
		PCG::FPCGAccessorBuffer Source;
		Source.SetupAndAllocate(3, PCG::Private::GetDefaultAttributeDesc<FSoftObjectPath>());
		FSoftObjectPath* SourceValues = static_cast<FSoftObjectPath*>(Source.GetOwnedMemoryPtr());
		SourceValues[0] = FSoftObjectPath(TEXT("/Game/Path/Zero.Zero"));
		SourceValues[1] = FSoftObjectPath(TEXT("/Game/Path/One.One"));
		SourceValues[2] = FSoftObjectPath(TEXT("/Game/Path/Two.Two"));

		PCG::FPCGAccessorBuffer Copy{Source};
		REQUIRE(Copy.IsOwningMemory());
		REQUIRE(Source.IsOwningMemory());
		REQUIRE_NOT_EQUAL(Copy.GetOwnedMemoryPtr(), Source.GetOwnedMemoryPtr());

		const FSoftObjectPath* CopyValues = static_cast<const FSoftObjectPath*>(Copy.GetOwnedMemoryPtr());
		REQUIRE_EQUAL(CopyValues[0], FSoftObjectPath(TEXT("/Game/Path/Zero.Zero")));
		REQUIRE_EQUAL(CopyValues[1], FSoftObjectPath(TEXT("/Game/Path/One.One")));
		REQUIRE_EQUAL(CopyValues[2], FSoftObjectPath(TEXT("/Game/Path/Two.Two")));

		// Source mutation must not leak into the copy.
		SourceValues[0] = FSoftObjectPath(TEXT("/Game/Path/Other.Other"));
		REQUIRE_EQUAL(CopyValues[0], FSoftObjectPath(TEXT("/Game/Path/Zero.Zero")));
	}

	// Uses FSoftObjectPath so Dest's prior owned state is destructed before being replaced by Source's copied state.
	SECTION("Copy assignment replaces existing state")
	{
		PCG::FPCGAccessorBuffer Source;
		Source.SetupAndAllocate(2, PCG::Private::GetDefaultAttributeDesc<FSoftObjectPath>());
		static_cast<FSoftObjectPath*>(Source.GetOwnedMemoryPtr())[0] = FSoftObjectPath(TEXT("/Game/Src/A.A"));
		static_cast<FSoftObjectPath*>(Source.GetOwnedMemoryPtr())[1] = FSoftObjectPath(TEXT("/Game/Src/B.B"));

		PCG::FPCGAccessorBuffer Dest;
		Dest.SetupAndAllocate(5, PCG::Private::GetDefaultAttributeDesc<FSoftObjectPath>());
		static_cast<FSoftObjectPath*>(Dest.GetOwnedMemoryPtr())[0] = FSoftObjectPath(TEXT("/Game/Prior/X.X"));
		Dest = Source;

		REQUIRE(Dest.IsOwningMemory());
		REQUIRE(Dest.GetUnderlyingDesc().IsSameType(PCG::Private::GetDefaultAttributeDesc<FSoftObjectPath>()));

		const FSoftObjectPath* DestValues = static_cast<const FSoftObjectPath*>(Dest.GetOwnedMemoryPtr());
		REQUIRE_EQUAL(DestValues[0], FSoftObjectPath(TEXT("/Game/Src/A.A")));
		REQUIRE_EQUAL(DestValues[1], FSoftObjectPath(TEXT("/Game/Src/B.B")));
	}

	// Uses FSoftObjectPath so a faulty self-assign would destruct-then-copy-from-self, leaving corrupted paths.
	SECTION("Copy self-assignment is a no-op")
	{
		PCG::FPCGAccessorBuffer Buffer;
		Buffer.SetupAndAllocate(3, PCG::Private::GetDefaultAttributeDesc<FSoftObjectPath>());
		FSoftObjectPath* Values = static_cast<FSoftObjectPath*>(Buffer.GetOwnedMemoryPtr());
		Values[0] = FSoftObjectPath(TEXT("/Game/Self/A.A"));
		Values[1] = FSoftObjectPath(TEXT("/Game/Self/B.B"));
		Values[2] = FSoftObjectPath(TEXT("/Game/Self/C.C"));

		void* PriorPtr = Buffer.GetOwnedMemoryPtr();
		Buffer = Buffer;

		REQUIRE(Buffer.IsOwningMemory());
		REQUIRE_EQUAL(Buffer.GetOwnedMemoryPtr(), PriorPtr);
		REQUIRE_EQUAL(static_cast<const FSoftObjectPath*>(Buffer.GetOwnedMemoryPtr())[0], FSoftObjectPath(TEXT("/Game/Self/A.A")));
		REQUIRE_EQUAL(static_cast<const FSoftObjectPath*>(Buffer.GetOwnedMemoryPtr())[2], FSoftObjectPath(TEXT("/Game/Self/C.C")));
	}

	// Uses FSoftObjectPath so the transferred memory carries non-POD state that must remain valid after the move.
	SECTION("Move ctor transfers memory")
	{
		PCG::FPCGAccessorBuffer Source;
		Source.SetupAndAllocate(3, PCG::Private::GetDefaultAttributeDesc<FSoftObjectPath>());
		static_cast<FSoftObjectPath*>(Source.GetOwnedMemoryPtr())[0] = FSoftObjectPath(TEXT("/Game/Move/A.A"));
		static_cast<FSoftObjectPath*>(Source.GetOwnedMemoryPtr())[1] = FSoftObjectPath(TEXT("/Game/Move/B.B"));
		static_cast<FSoftObjectPath*>(Source.GetOwnedMemoryPtr())[2] = FSoftObjectPath(TEXT("/Game/Move/C.C"));

		PCG::FPCGAccessorBuffer Moved{MoveTemp(Source)};
		REQUIRE(Moved.IsOwningMemory());
		REQUIRE_FALSE(Source.IsOwningMemory());
		REQUIRE(Moved.GetUnderlyingDesc().IsSameType(PCG::Private::GetDefaultAttributeDesc<FSoftObjectPath>()));

		const FSoftObjectPath* MovedValues = static_cast<const FSoftObjectPath*>(Moved.GetOwnedMemoryPtr());
		REQUIRE_EQUAL(MovedValues[0], FSoftObjectPath(TEXT("/Game/Move/A.A")));
		REQUIRE_EQUAL(MovedValues[2], FSoftObjectPath(TEXT("/Game/Move/C.C")));
	}

	// Uses FSoftObjectPath so Dest's prior owned state must be destructed before the move transfer, catching leaks.
	SECTION("Move assignment transfers memory")
	{
		PCG::FPCGAccessorBuffer Source;
		Source.SetupAndAllocate(2, PCG::Private::GetDefaultAttributeDesc<FSoftObjectPath>());
		static_cast<FSoftObjectPath*>(Source.GetOwnedMemoryPtr())[0] = FSoftObjectPath(TEXT("/Game/Move/One.One"));
		static_cast<FSoftObjectPath*>(Source.GetOwnedMemoryPtr())[1] = FSoftObjectPath(TEXT("/Game/Move/Two.Two"));

		PCG::FPCGAccessorBuffer Dest;
		Dest.SetupAndAllocate(4, PCG::Private::GetDefaultAttributeDesc<FSoftObjectPath>());
		static_cast<FSoftObjectPath*>(Dest.GetOwnedMemoryPtr())[0] = FSoftObjectPath(TEXT("/Game/Move/Prior.Prior"));
		Dest = MoveTemp(Source);

		REQUIRE(Dest.IsOwningMemory());
		REQUIRE_FALSE(Source.IsOwningMemory());

		const FSoftObjectPath* DestValues = static_cast<const FSoftObjectPath*>(Dest.GetOwnedMemoryPtr());
		REQUIRE_EQUAL(DestValues[0], FSoftObjectPath(TEXT("/Game/Move/One.One")));
		REQUIRE_EQUAL(DestValues[1], FSoftObjectPath(TEXT("/Game/Move/Two.Two")));
	}

	// Uses FSoftObjectPath so a faulty self-move would destruct-then-move-from-self, zeroing out the paths.
	SECTION("Move self-assignment is a no-op")
	{
		PCG::FPCGAccessorBuffer Buffer;
		Buffer.SetupAndAllocate(3, PCG::Private::GetDefaultAttributeDesc<FSoftObjectPath>());
		FSoftObjectPath* Values = static_cast<FSoftObjectPath*>(Buffer.GetOwnedMemoryPtr());
		Values[0] = FSoftObjectPath(TEXT("/Game/SelfMove/A.A"));
		Values[1] = FSoftObjectPath(TEXT("/Game/SelfMove/B.B"));
		Values[2] = FSoftObjectPath(TEXT("/Game/SelfMove/C.C"));

		void* PriorPtr = Buffer.GetOwnedMemoryPtr();
		Buffer = MoveTemp(Buffer);

		REQUIRE(Buffer.IsOwningMemory());
		REQUIRE_EQUAL(Buffer.GetOwnedMemoryPtr(), PriorPtr);
		REQUIRE_EQUAL(static_cast<const FSoftObjectPath*>(Buffer.GetOwnedMemoryPtr())[0], FSoftObjectPath(TEXT("/Game/SelfMove/A.A")));
		REQUIRE_EQUAL(static_cast<const FSoftObjectPath*>(Buffer.GetOwnedMemoryPtr())[2], FSoftObjectPath(TEXT("/Game/SelfMove/C.C")));
	}

	// Uses FSoftObjectPath so the underlying memory carries non-POD state that must survive the StealMemory transfer.
	SECTION("StealMemory with matching type transfers to FScriptArray")
	{
		PCG::FPCGAccessorBuffer Buffer;
		Buffer.SetupAndAllocate(3, PCG::Private::GetDefaultAttributeDesc<FSoftObjectPath>());
		static_cast<FSoftObjectPath*>(Buffer.GetOwnedMemoryPtr())[0] = FSoftObjectPath(TEXT("/Game/Steal/A.A"));
		static_cast<FSoftObjectPath*>(Buffer.GetOwnedMemoryPtr())[1] = FSoftObjectPath(TEXT("/Game/Steal/B.B"));
		static_cast<FSoftObjectPath*>(Buffer.GetOwnedMemoryPtr())[2] = FSoftObjectPath(TEXT("/Game/Steal/C.C"));

		FScriptArray DestArray;
		Buffer.MoveMemoryTo(DestArray, PCG::Private::GetDefaultAttributeDesc<FSoftObjectPath>());

		REQUIRE_FALSE(Buffer.IsOwningMemory());
		REQUIRE_EQUAL(DestArray.Num(), 3);

		FSoftObjectPath* DestValues = static_cast<FSoftObjectPath*>(DestArray.GetData());
		REQUIRE_EQUAL(DestValues[0], FSoftObjectPath(TEXT("/Game/Steal/A.A")));
		REQUIRE_EQUAL(DestValues[1], FSoftObjectPath(TEXT("/Game/Steal/B.B")));
		REQUIRE_EQUAL(DestValues[2], FSoftObjectPath(TEXT("/Game/Steal/C.C")));

		// FScriptArray is type-erased: destruct each element manually, then free the raw buffer.
		DestructItems(DestValues, DestArray.Num());
		DestArray.Empty(0, sizeof(FSoftObjectPath), alignof(FSoftObjectPath));
	}

	SECTION("StealMemory with mismatched type is a no-op")
	{
		PCG::FPCGAccessorBuffer Buffer;
		Buffer.SetupAndAllocate(3, PCG::Private::GetDefaultAttributeDesc<FSoftObjectPath>());

		PCGTests::FPCGTestsLogOutputDevice LogCapture(/*bSuppressErrors=*/true);

		FScriptArray DestArray;
		Buffer.MoveMemoryTo(DestArray, PCG::Private::GetDefaultAttributeDesc<FString>());
		
		REQUIRE(LogCapture.NbMessageReceived.Contains(ELogVerbosity::Error));

		REQUIRE(Buffer.IsOwningMemory());
		REQUIRE_EQUAL(DestArray.Num(), 0);
	}
}