// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGTestsCommon.h"

#include "Data/PCGPointArrayData.h"
#include "Metadata/PCGMetadata.h"

#include <catch2/catch_test_macros.hpp>
#include <catch2/benchmark/catch_benchmark.hpp>
#include <catch2/generators/catch_generators_range.hpp>

#include "TestHarness.h"

TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Metadata::Flatten", "[PCG][Metadata][Flatten]")
{
	static const FName Attribute1Name = TEXT("FloatAttr");
	static const FName Attribute2Name = TEXT("StringAttr");
	static const FName Attribute3Name = TEXT("IntAttr");

	constexpr int32 NumPoints = 10;
	UPCGPointArrayData* RootPointData = CreateData<UPCGPointArrayData>();
	RootPointData->AllocateProperties(EPCGPointNativeProperties::MetadataEntry);
	RootPointData->SetNumPoints(NumPoints);
	
	FPCGMetadataAttribute<float>* Attribute1 = RootPointData->Metadata->CreateAttribute<float>(Attribute1Name, -0.1f, true, true);
	FPCGMetadataAttribute<FString>* Attribute2 = RootPointData->Metadata->CreateAttribute<FString>(Attribute2Name, TEXT("Default"), true, true);

	check(Attribute1 && Attribute2);

	TPCGValueRange<int64> MetadataEntryRange = RootPointData->GetMetadataEntryValueRange();
	for (int32 i = 0; i < RootPointData->GetNumPoints(); ++i)
	{
		RootPointData->Metadata->InitializeOnSet(MetadataEntryRange[i]);
		Attribute1->SetValue(MetadataEntryRange[i], static_cast<float>(i) * 0.1f);

		if (i % 2 == 0)
		{
			// Will be either "0" or "1"
			Attribute2->SetValue(MetadataEntryRange[i], FString::Printf(TEXT("%d"), i % 4));
		}
	}

	// At the end of the first set, metadata has 10 entries, and values for each points are
	// for Attribute 1: [0.0, 0.1, 0.2, 0.3, 0.4, 0.5, 0.6, 0.7, 0.8, 0.9]
	// for Attribute 2: ["0", "Default", "2", "Default", "0", ...]
	SECTION("RootMetadata has the right number of entries")
	{
		REQUIRE_EQUAL(RootPointData->Metadata->GetItemCountForChild(), NumPoints);
	}

	UPCGBasePointData* FirstChildPointData = Cast<UPCGBasePointData>(RootPointData->DuplicateData(nullptr));
	Attribute1 = FirstChildPointData->Metadata->GetMutableTypedAttribute<float>(Attribute1Name);
	Attribute2 = FirstChildPointData->Metadata->GetMutableTypedAttribute<FString>(Attribute2Name);

	SECTION("Attributes exists in first child")
	{
		REQUIRE_NOT_EQUAL(Attribute1, nullptr);
		REQUIRE_NOT_EQUAL(Attribute2, nullptr);
	}
	
	check(Attribute1 && Attribute2);

	// Override all the values for Attribute 1, and replace all "1" by "0" in attribute 2
	// Also add a third attribute
	FPCGMetadataAttribute<int>* Attribute3 = FirstChildPointData->Metadata->CreateAttribute<int>(Attribute3Name, -1, true, true);
	check(Attribute3);

	SECTION("First child has right number of points")
	{
		REQUIRE_EQUAL(FirstChildPointData->GetNumPoints(), NumPoints);
	}

	TPCGValueRange<int64> FirstChildMetadataEntryRange = FirstChildPointData->GetMetadataEntryValueRange();
	for (int32 i = 0; i < FirstChildPointData->GetNumPoints(); ++i)
	{
		FirstChildPointData->Metadata->InitializeOnSet(FirstChildMetadataEntryRange[i]);
		Attribute1->SetValue(FirstChildMetadataEntryRange[i], static_cast<float>(i) * 1.1f);

		if (i % 2 == 0)
		{
			Attribute2->SetValue(FirstChildMetadataEntryRange[i], TEXT("0"));
		}

		Attribute3->SetValue(FirstChildMetadataEntryRange[i], i);
	}

	// At the end of the second set, metadata has 20 entries, and values for each points are
	// for Attribute 1: [0.0, 1.1, 2.2, 3.3, 4.4, 5.5, 6.6, 7.7, 8.8, 9.9]
	// for Attribute 2: ["0", "Default", "0", "Default", "0", ...]
	// for Attribute 3: [0, 1, 2, 3, 4, 5, 6, 7, 8, 9]
	SECTION("First child metadata has right number of entries")
	{
		REQUIRE_EQUAL(FirstChildPointData->Metadata->GetItemCountForChild(), 2 * NumPoints);
	}
	

	// Second child is keeping the metadata entry for even numbers and reset metadata entry for the rest (point back to default). Also override Attribute 3 values for even numbers
	UPCGBasePointData* SecondChildPointData = CreateData<UPCGPointArrayData>();

	FPCGInitializeFromDataParams InitializeFromDataParams(FirstChildPointData);
	InitializeFromDataParams.bInheritSpatialData = false;
	SecondChildPointData->InitializeFromDataWithParams(InitializeFromDataParams);
	
	Attribute3 = SecondChildPointData->Metadata->GetMutableTypedAttribute<int>(Attribute3Name);
	SECTION("Attributes exists in second child")
	{
		REQUIRE_NOT_EQUAL(Attribute3, nullptr);
	}

	check(Attribute3);

	SecondChildPointData->SetNumPoints(FirstChildPointData->GetNumPoints());
	UPCGBasePointData::SetPoints(FirstChildPointData, SecondChildPointData, {}, /*bCopyAll=*/true);

	TPCGValueRange<int64> SecondChildMetadataEntryRange = SecondChildPointData->GetMetadataEntryValueRange();
	for (int32 i = 0; i < SecondChildPointData->GetNumPoints(); ++i)
	{
		if (i % 2 == 0)
		{
			SecondChildPointData->Metadata->InitializeOnSet(SecondChildMetadataEntryRange[i]);
			Attribute3->SetValue(SecondChildMetadataEntryRange[i], 10 * i);
		}
		else
		{
			SecondChildMetadataEntryRange[i] = PCGInvalidEntryKey;
		}
	}

	// At the end of the third set, metadata has 25 entries, and values for each points are:
	// for Attribute 1: [0.0, -0.1, 2.2, -0.1, 4.4, -0.1, 6.6, -0.1, 8.8, -0.1]
	// for Attribute 2: ["0", "Default", "0", "Default", "0", ...]
	// for Attribute 3: [0, -1, 20, -1, 40, -1, 60, -1, 80, -1]
	SECTION("Second Child Metadata has the right number of entries")
	{
		REQUIRE_EQUAL(SecondChildPointData->Metadata->GetItemCountForChild(), 2 * NumPoints + NumPoints / 2);
	}
	

	// For final set, duplicate the data and flatten it
	UPCGBasePointData* FinalPointData = Cast<UPCGBasePointData>(SecondChildPointData->DuplicateData(nullptr));
	FinalPointData->Flatten();

	Attribute1 = FinalPointData->Metadata->GetMutableTypedAttribute<float>(Attribute1Name);
	Attribute2 = FinalPointData->Metadata->GetMutableTypedAttribute<FString>(Attribute2Name);
	Attribute3 = FinalPointData->Metadata->GetMutableTypedAttribute<int>(Attribute3Name);

	SECTION("Attributes exists in final child")
	{
		REQUIRE_NOT_EQUAL(Attribute1, nullptr);
		REQUIRE_NOT_EQUAL(Attribute2, nullptr);
		REQUIRE_NOT_EQUAL(Attribute3, nullptr);
	}
	
	check(Attribute1 && Attribute2 && Attribute3);

	SECTION("Flatten metadata has the right number of entries/values")
	{
		REQUIRE_EQUAL(FinalPointData->Metadata->GetItemCountForChild(), 5);
		REQUIRE_EQUAL(Attribute1->GetValueKeyOffsetForChild(), 5);
		REQUIRE_EQUAL(Attribute2->GetValueKeyOffsetForChild(), 1);
		REQUIRE_EQUAL(Attribute3->GetValueKeyOffsetForChild(), 5);
		REQUIRE_EQUAL(FinalPointData->GetNumPoints(), NumPoints);
	}

	// Validate the values
	const TConstPCGValueRange<int64> FinalMetadataEntryRange = FinalPointData->GetConstMetadataEntryValueRange();
	auto i = GENERATE_COPY(range(0, NumPoints));
	
	SECTION("Validate values")
	{
		const int64 MetadataEntry = FinalMetadataEntryRange[i];
		if (i % 2 == 0)
		{
			REQUIRE_EQUAL(MetadataEntry, static_cast<PCGMetadataEntryKey>(i / 2));
			REQUIRE_EQUAL(Attribute1->GetValueFromItemKey(MetadataEntry), i * 1.1f);
			REQUIRE_EQUAL(Attribute2->GetValueFromItemKey(MetadataEntry), TEXT("0"));
			REQUIRE_EQUAL(Attribute3->GetValueFromItemKey(MetadataEntry), i * 10);
		}
		else
		{
			REQUIRE_EQUAL(MetadataEntry, PCGInvalidEntryKey);
			REQUIRE_EQUAL(Attribute1->GetValueFromItemKey(MetadataEntry), -0.1f);
			REQUIRE_EQUAL(Attribute2->GetValueFromItemKey(MetadataEntry), TEXT("Default"));
			REQUIRE_EQUAL(Attribute3->GetValueFromItemKey(MetadataEntry), -1);
		}
	}
}

TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Metadata::FlattenGeneric", "[PCG][Metadata][Flatten]")
{
	static const FName Attribute1Name = TEXT("FloatAttr");
	static const FName Attribute2Name = TEXT("StringAttr");
	static const FName Attribute3Name = TEXT("IntAttr");

	constexpr int32 NumPoints = 10;
	UPCGPointArrayData* RootPointData = CreateData<UPCGPointArrayData>();
	RootPointData->AllocateProperties(EPCGPointNativeProperties::MetadataEntry);
	RootPointData->SetNumPoints(NumPoints);
	
	FPCGMetadataAttributeBase* Attribute1 = RootPointData->MutableMetadata()->GetDefaultMetadataDomain()->CreateAttribute<float>(Attribute1Name, -0.1f);
	FPCGMetadataAttributeBase* Attribute2 = RootPointData->MutableMetadata()->GetDefaultMetadataDomain()->CreateAttribute<FString>(Attribute2Name, TEXT("Default"));

	check(Attribute1 && Attribute2);

	TPCGValueRange<int64> MetadataEntryRange = RootPointData->GetMetadataEntryValueRange();
	for (int32 i = 0; i < RootPointData->GetNumPoints(); ++i)
	{
		RootPointData->MutableMetadata()->InitializeOnSet(MetadataEntryRange[i]);
		Attribute1->SetValue<float>(MetadataEntryRange[i], static_cast<float>(i) * 0.1f);

		if (i % 2 == 0)
		{
			Attribute2->SetValue<FString>(MetadataEntryRange[i], FString::Printf(TEXT("%d"), i % 4));
		}
	}

	// At the end of the first set, metadata has 10 entries, and values for each points are
	// for Attribute 1: [0.0, 0.1, 0.2, 0.3, 0.4, 0.5, 0.6, 0.7, 0.8, 0.9]
	// for Attribute 2: ["0", "Default", "2", "Default", "0", ...]
	SECTION("RootMetadata has the right number of entries")
	{
		REQUIRE_EQUAL(RootPointData->Metadata->GetItemCountForChild(), NumPoints);
	}

	UPCGBasePointData* FirstChildPointData = Cast<UPCGBasePointData>(RootPointData->DuplicateData(nullptr));
	Attribute1 = FirstChildPointData->MutableMetadata()->GetDefaultMetadataDomain()->GetMutableAttribute(Attribute1Name);
	Attribute2 = FirstChildPointData->MutableMetadata()->GetDefaultMetadataDomain()->GetMutableAttribute(Attribute2Name);

	SECTION("Attributes exists in first child")
	{
		REQUIRE_NOT_EQUAL(Attribute1, nullptr);
		REQUIRE_NOT_EQUAL(Attribute2, nullptr);
	}
	
	check(Attribute1 && Attribute2);

	// Override all the values for Attribute 1, and replace all "1" by "0" in attribute 2
	// Also add a third attribute
	FPCGMetadataAttributeBase* Attribute3 = FirstChildPointData->MutableMetadata()->GetDefaultMetadataDomain()->CreateAttribute<int>(Attribute3Name, -1);
	check(Attribute3);

	SECTION("First child has right number of points")
	{
		REQUIRE_EQUAL(FirstChildPointData->GetNumPoints(), NumPoints);
	}

	TPCGValueRange<int64> FirstChildMetadataEntryRange = FirstChildPointData->GetMetadataEntryValueRange();
	for (int32 i = 0; i < FirstChildPointData->GetNumPoints(); ++i)
	{
		FirstChildPointData->Metadata->InitializeOnSet(FirstChildMetadataEntryRange[i]);
		Attribute1->SetValue<float>(FirstChildMetadataEntryRange[i], static_cast<float>(i) * 1.1f);

		if (i % 2 == 0)
		{
			Attribute2->SetValue<FString>(FirstChildMetadataEntryRange[i], TEXT("0"));
		}

		Attribute3->SetValue<int>(FirstChildMetadataEntryRange[i], i);
	}

	// At the end of the second set, metadata has 20 entries, and values for each points are
	// for Attribute 1: [0.0, 1.1, 2.2, 3.3, 4.4, 5.5, 6.6, 7.7, 8.8, 9.9]
	// for Attribute 2: ["0", "Default", "0", "Default", "0", ...]
	// for Attribute 3: [0, 1, 2, 3, 4, 5, 6, 7, 8, 9]
	SECTION("First child metadata has right number of entries")
	{
		REQUIRE_EQUAL(FirstChildPointData->Metadata->GetItemCountForChild(), 2 * NumPoints);
	}
	

	// Second child is keeping the metadata entry for even numbers and reset metadata entry for the rest (point back to default). Also override Attribute 3 values for even numbers
	UPCGBasePointData* SecondChildPointData = CreateData<UPCGPointArrayData>();

	FPCGInitializeFromDataParams InitializeFromDataParams(FirstChildPointData);
	InitializeFromDataParams.bInheritSpatialData = false;
	SecondChildPointData->InitializeFromDataWithParams(InitializeFromDataParams);
	
	Attribute3 = SecondChildPointData->MutableMetadata()->GetDefaultMetadataDomain()->GetMutableAttribute(Attribute3Name);
	SECTION("Attributes exists in second child")
	{
		REQUIRE_NOT_EQUAL(Attribute3, nullptr);
	}

	check(Attribute3);

	SecondChildPointData->SetNumPoints(FirstChildPointData->GetNumPoints());
	UPCGBasePointData::SetPoints(FirstChildPointData, SecondChildPointData, {}, /*bCopyAll=*/true);

	TPCGValueRange<int64> SecondChildMetadataEntryRange = SecondChildPointData->GetMetadataEntryValueRange();
	for (int32 i = 0; i < SecondChildPointData->GetNumPoints(); ++i)
	{
		if (i % 2 == 0)
		{
			SecondChildPointData->Metadata->InitializeOnSet(SecondChildMetadataEntryRange[i]);
			Attribute3->SetValue<int>(SecondChildMetadataEntryRange[i], 10 * i);
		}
		else
		{
			SecondChildMetadataEntryRange[i] = PCGInvalidEntryKey;
		}
	}

	// At the end of the third set, metadata has 25 entries, and values for each points are:
	// for Attribute 1: [0.0, -0.1, 2.2, -0.1, 4.4, -0.1, 6.6, -0.1, 8.8, -0.1]
	// for Attribute 2: ["0", "Default", "0", "Default", "0", ...]
	// for Attribute 3: [0, -1, 20, -1, 40, -1, 60, -1, 80, -1]
	SECTION("Second Child Metadata has the right number of entries")
	{
		REQUIRE_EQUAL(SecondChildPointData->Metadata->GetItemCountForChild(), 2 * NumPoints + NumPoints / 2);
	}
	

	// For final set, duplicate the data and flatten it
	UPCGBasePointData* FinalPointData = Cast<UPCGBasePointData>(SecondChildPointData->DuplicateData(nullptr));
	FinalPointData->Flatten();

	Attribute1 = FinalPointData->MutableMetadata()->GetDefaultMetadataDomain()->GetMutableAttribute(Attribute1Name);
	Attribute2 = FinalPointData->MutableMetadata()->GetDefaultMetadataDomain()->GetMutableAttribute(Attribute2Name);
	Attribute3 = FinalPointData->MutableMetadata()->GetDefaultMetadataDomain()->GetMutableAttribute(Attribute3Name);

	SECTION("Attributes exists in final child")
	{
		REQUIRE_NOT_EQUAL(Attribute1, nullptr);
		REQUIRE_NOT_EQUAL(Attribute2, nullptr);
		REQUIRE_NOT_EQUAL(Attribute3, nullptr);
	}
	
	check(Attribute1 && Attribute2 && Attribute3);

	SECTION("Flatten metadata has the right number of entries/values")
	{
		REQUIRE_EQUAL(FinalPointData->Metadata->GetItemCountForChild(), 5);
		REQUIRE_EQUAL(Attribute1->GetValueKeyOffsetForChild(), 5);
		REQUIRE_EQUAL(Attribute2->GetValueKeyOffsetForChild(), 1);
		REQUIRE_EQUAL(Attribute3->GetValueKeyOffsetForChild(), 5);
		REQUIRE_EQUAL(FinalPointData->GetNumPoints(), NumPoints);
	}

	// Validate the values
	const TConstPCGValueRange<int64> FinalMetadataEntryRange = FinalPointData->GetConstMetadataEntryValueRange();
	auto i = GENERATE_COPY(range(0, NumPoints));
	
	SECTION("Validate values")
	{
		const int64 MetadataEntry = FinalMetadataEntryRange[i];
		if (i % 2 == 0)
		{
			REQUIRE_EQUAL(MetadataEntry, static_cast<PCGMetadataEntryKey>(i / 2));
			REQUIRE_EQUAL(Attribute1->GetValueFromItemKey<float>(MetadataEntry), i * 1.1f);
			REQUIRE_EQUAL(Attribute2->GetValueFromItemKey<FString>(MetadataEntry), TEXT("0"));
			REQUIRE_EQUAL(Attribute3->GetValueFromItemKey<int>(MetadataEntry), i * 10);
		}
		else
		{
			REQUIRE_EQUAL(MetadataEntry, PCGInvalidEntryKey);
			REQUIRE_EQUAL(Attribute1->GetValueFromItemKey<float>(MetadataEntry), -0.1f);
			REQUIRE_EQUAL(Attribute2->GetValueFromItemKey<FString>(MetadataEntry), TEXT("Default"));
			REQUIRE_EQUAL(Attribute3->GetValueFromItemKey<int>(MetadataEntry), -1);
		}
	}
}

/**
 * Flattening metadata that has no attribute still must be flattened. Same for Data domain.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Metadata::FlattenEmptyMetadata", "[PCG][Metadata][Flatten]")
{
	// Since the containers are meant to be created by the generic attributes, we'll use the generic attributes
	// to populate them
	
	// Preparing the data
	constexpr int32 NumPoints = 1;
	UPCGPointArrayData* PointArrayData = CreateData<UPCGPointArrayData>();
	PointArrayData->AllocateProperties(EPCGPointNativeProperties::MetadataEntry);
	PointArrayData->SetNumPoints(NumPoints);
	
	FPCGMetadataDomain* DataDomain = PointArrayData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Data);
	REQUIRE_NOT_EQUAL(DataDomain, nullptr);
	
	FPCGMetadataDomain* ElementsDomain = PointArrayData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Elements);
	REQUIRE_NOT_EQUAL(ElementsDomain, nullptr);
	
	const PCGMetadataEntryKey EntryKey = DataDomain->AddEntry();
	
	const FName AttributeName = TEXT("Double");
	
	FPCGMetadataAttribute<double>* Attribute = DataDomain->CreateAttribute<double>(AttributeName, 1.0f, false, false);
	Attribute->SetValue(EntryKey, 5.0);
	
	REQUIRE_EQUAL(Attribute->GetNumberOfEntries(), 1);
	
	UPCGPointArrayData* ChildPointArrayData = CreateData<UPCGPointArrayData>();
	ChildPointArrayData->InitializeFromDataWithParams(FPCGInitializeFromDataParams{PointArrayData});
	
	FPCGMetadataDomain* ChildDataDomain = ChildPointArrayData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Data);
	REQUIRE_NOT_EQUAL(ChildDataDomain, nullptr);
	
	FPCGMetadataDomain* ChildElementDomain = ChildPointArrayData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Elements);
	REQUIRE_NOT_EQUAL(ChildElementDomain, nullptr);
	
	// Data domain should have a single attribute, and inherit its parent
	REQUIRE_EQUAL(ChildDataDomain->GetAttributeCount(), 1);
	REQUIRE_EQUAL(ChildDataDomain->GetParent(), DataDomain);
	
	// Elements domain should have no attribute, and inherit its parent.
	REQUIRE_EQUAL(ChildElementDomain->GetAttributeCount(), 0);
	REQUIRE_EQUAL(ChildElementDomain->GetParent(), ElementsDomain);
	
	ChildPointArrayData->Flatten();
	
	// After flattening, we should still have the same attribute count by domain, but no parent
	REQUIRE_EQUAL(ChildDataDomain->GetAttributeCount(), 1);
	REQUIRE_EQUAL(ChildDataDomain->GetParent(), nullptr);
	
	REQUIRE_EQUAL(ChildElementDomain->GetAttributeCount(), 0);
	REQUIRE_EQUAL(ChildElementDomain->GetParent(), nullptr);
}

TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Metadata::FlattenWithoutCompression", "[PCG][Metadata][Flatten]")
{
	static const FName Attribute1Name = TEXT("FloatAttr");
	static const FName Attribute2Name = TEXT("StringAttr");

	constexpr int32 NumPoints = 5;
	UPCGPointArrayData* RootPointData = CreateData<UPCGPointArrayData>();
	RootPointData->AllocateProperties(EPCGPointNativeProperties::MetadataEntry);
	RootPointData->SetNumPoints(NumPoints);
	
	FPCGMetadataAttributeBase* Attribute1 = RootPointData->MutableMetadata()->GetDefaultMetadataDomain()->CreateAttribute<float>(Attribute1Name, -0.1f);
	FPCGMetadataAttributeBase* Attribute2 = RootPointData->MutableMetadata()->GetDefaultMetadataDomain()->CreateAttribute<FString>(Attribute2Name, TEXT("Default"));
	
	REQUIRE_NOT_EQUAL(Attribute1, nullptr);
	REQUIRE_NOT_EQUAL(Attribute2, nullptr);
	
	TArray<float> FloatValues;
	TArray<FString> StringValues;
	
	for (int32 i = 0; i < NumPoints * 2; ++i)
	{
		FloatValues.Emplace(i);
		StringValues.Emplace(FString::Printf(TEXT("%d"), i % 3));
	}
	
	TPCGValueRange<int64> MetadataEntryRange = RootPointData->GetMetadataEntryValueRange();
	for (int32 i = 0; i < NumPoints; ++i)
	{
		RootPointData->MutableMetadata()->InitializeOnSet(MetadataEntryRange[i]);
		Attribute1->SetValue(MetadataEntryRange[i], FloatValues[i]);
		Attribute2->SetValue(MetadataEntryRange[i], StringValues[i]);
	}
	
	UPCGPointArrayData* ChildPointData = CreateData<UPCGPointArrayData>();
	ChildPointData->InitializeFromDataWithParams(FPCGInitializeFromDataParams{RootPointData});
	ChildPointData->SetNumPoints(NumPoints * 2);
	
	Attribute1 = ChildPointData->MutableMetadata()->GetDefaultMetadataDomain()->GetMutableAttribute(Attribute1Name);
	Attribute2 = ChildPointData->MutableMetadata()->GetDefaultMetadataDomain()->GetMutableAttribute(Attribute2Name);
	
	REQUIRE_NOT_EQUAL(Attribute1, nullptr);
	REQUIRE_NOT_EQUAL(Attribute2, nullptr);
	
	TPCGValueRange<int64> ChildMetadataEntryRange = ChildPointData->GetMetadataEntryValueRange();
	for (int32 i = NumPoints; i < 2 * NumPoints; ++i)
	{
		ChildPointData->MutableMetadata()->InitializeOnSet(ChildMetadataEntryRange[i]);
		Attribute1->SetValue(ChildMetadataEntryRange[i], FloatValues[i]);
		Attribute2->SetValue(ChildMetadataEntryRange[i], StringValues[i]);
	}
	
	REQUIRE_EQUAL(Attribute1->GetNumberOfEntries(), NumPoints);
	REQUIRE_EQUAL(Attribute2->GetNumberOfEntries(), NumPoints);
	
	ChildPointData->MutableMetadata()->GetDefaultMetadataDomain()->FlattenImpl();
	
	REQUIRE_EQUAL(Attribute1->GetParent(), nullptr);
	REQUIRE_EQUAL(Attribute2->GetParent(), nullptr);
	
	REQUIRE_EQUAL(Attribute1->GetNumberOfEntries(), 2 * NumPoints);
	REQUIRE_EQUAL(Attribute2->GetNumberOfEntries(), 2 * NumPoints);
	
	REQUIRE_EQUAL(Attribute1->GetValueFromItemKey<float>(PCGInvalidEntryKey), -0.1f);
	REQUIRE_EQUAL(Attribute2->GetValueFromItemKey<FString>(PCGInvalidEntryKey), TEXT("Default"));
	
	for (int32 i = 0; i < 2 * NumPoints; ++i)
	{
		CAPTURE(i);
		REQUIRE_EQUAL(Attribute1->GetValueFromItemKey<float>(ChildMetadataEntryRange[i]), FloatValues[i]);
		REQUIRE_EQUAL(Attribute2->GetValueFromItemKey<FString>(ChildMetadataEntryRange[i]), StringValues[i]);
	}
}

/**
 * Flattening point data that has metadata entries for all the points but some of the entries that point to the default value
 * should only allocate the number of values of all the non-default values.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Metadata::FlattenPartiallySetAttributes", "[PCG][Metadata][Flatten]")
{
	constexpr int32 NumPoints = 10;
	UPCGPointArrayData* PointData = CreateData<UPCGPointArrayData>();
	PointData->AllocateProperties(EPCGPointNativeProperties::MetadataEntry);
	PointData->SetNumPoints(NumPoints);
	
	FPCGMetadataAttribute<FName>* NameAttribute = PointData->Metadata->CreateAttribute<FName>(TEXT("Attr"), NAME_None, false, false);

	TPCGValueRange<int64> MetadataEntryRange = PointData->GetMetadataEntryValueRange();
	for (int32 i = 0; i < NumPoints; ++i)
	{
		PointData->Metadata->InitializeOnSet(MetadataEntryRange[i]);
		// Set half the values
		if (i % 2 == 0)
		{
			NameAttribute->SetValue(MetadataEntryRange[i], FName("Hi"));
		}
	}

	PointData->Flatten();

	REQUIRE_EQUAL(NameAttribute->GetValueKeyOffsetForChild(), NumPoints/2);
}