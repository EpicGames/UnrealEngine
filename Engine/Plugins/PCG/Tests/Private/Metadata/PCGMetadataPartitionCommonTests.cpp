// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGTestsCommon.h"

#include "PCGContext.h"
#include "PCGParamData.h"
#include "Data/PCGPointArrayData.h"
#include "Metadata/PCGMetadata.h"
#include "Elements/Metadata/PCGMetadataPartition.h"
#include "Metadata/PCGMetadataPartitionCommon.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include <catch2/catch_test_macros.hpp>

#include "TestHarness.h"
#include "Misc/ScopedCVar.h"

namespace PCGMetadataPartitionCommonTests
{
	static const FName AttrA = TEXT("AttrA");
	static const FName AttrB = TEXT("AttrB");
	static const FName AttrC = TEXT("AttrC");

	void SetupTestPointData(UPCGPointArrayData* Data, int32 NumPoints)
	{
		Data->SetNumPoints(NumPoints);
		Data->SetDensity(1.0f);
	}

	void SetupParamDataWithIntAttribute(UPCGParamData* Data, const FName AttributeName, TConstArrayView<int32> Values)
	{
		FPCGMetadataAttribute<int32>* Attr = Data->Metadata->CreateAttribute<int32>(AttributeName, 0, /*bAllowInterpolation=*/false, /*bOverrideParent=*/false);
		check(Attr);
		for (int32 Value : Values)
		{
			Attr->SetValue(Data->Metadata->AddEntry(), Value);
		}
	}

	/** Filter empty buckets and verify contents against expected. */
	void VerifyGenericPartition(const TArray<TArray<int32>>& Result, const TArray<TArray<int32>>& Expected)
	{
		TArray<TArray<int32>> NonEmptyResult;
		for (const TArray<int32>& Bucket : Result)
		{
			if (!Bucket.IsEmpty())
			{
				NonEmptyResult.Add(Bucket);
			}
		}

		REQUIRE_EQUAL(NonEmptyResult.Num(), Expected.Num());
		for (int32 i = 0; i < Expected.Num(); ++i)
		{
			CHECK(NonEmptyResult[i] == Expected[i]);
		}
	}
}

/**
 * AttributeGenericPartition with a single selector.
 * Fundamental index-based API.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributePartition::GenericPartition::SingleSelector", "[PCG][AttributePartition]")
{
	using namespace PCGMetadataPartitionCommonTests;

	SECTION("Null data returns empty")
	{
		const FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateAttributeSelector(AttrA);
		TArray<TArray<int32>> Result = PCGMetadataPartitionCommon::AttributeGenericPartition(nullptr, Selector);
		CHECK(Result.IsEmpty());
	}

	SECTION("Empty param data returns empty")
	{
		UPCGParamData* Data = CreateData<UPCGParamData>();
		Data->Metadata->CreateAttribute<int32>(AttrA, 0, false, false);
		// Attribute exists but no entries
		const FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateAttributeSelector(AttrA);
		TArray<TArray<int32>> Result = PCGMetadataPartitionCommon::AttributeGenericPartition(Data, Selector, &Context);
		for (const TArray<int32>& Bucket : Result)
		{
			CHECK(Bucket.IsEmpty());
		}
	}

	SECTION("Single entry produces one bucket")
	{
		UPCGParamData* Data = CreateData<UPCGParamData>();
		SetupParamDataWithIntAttribute(Data, AttrA, {42});
		const FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateAttributeSelector(AttrA);

		TArray<TArray<int32>> Result = PCGMetadataPartitionCommon::AttributeGenericPartition(Data, Selector, &Context);
		VerifyGenericPartition(Result, {{0}});
	}

	SECTION("All same value produces single bucket with all indices")
	{
		UPCGParamData* Data = CreateData<UPCGParamData>();
		SetupParamDataWithIntAttribute(Data, AttrA, {5, 5, 5, 5});
		const FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateAttributeSelector(AttrA);

		TArray<TArray<int32>> Result = PCGMetadataPartitionCommon::AttributeGenericPartition(Data, Selector, &Context);
		VerifyGenericPartition(Result, {{0, 1, 2, 3}});
	}

	SECTION("All unique values produce N buckets of 1")
	{
		UPCGParamData* Data = CreateData<UPCGParamData>();
		SetupParamDataWithIntAttribute(Data, AttrA, {10, 20, 30});
		const FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateAttributeSelector(AttrA);

		TArray<TArray<int32>> Result = PCGMetadataPartitionCommon::AttributeGenericPartition(Data, Selector, &Context);
		VerifyGenericPartition(Result, {{0}, {1}, {2}});
	}

	SECTION("Interleaved values preserve stable order within buckets")
	{
		// Values: [A, B, A, B, A] -- first appearance order determines bucket order
		UPCGParamData* Data = CreateData<UPCGParamData>();
		SetupParamDataWithIntAttribute(Data, AttrA, {1, 2, 1, 2, 1});
		const FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateAttributeSelector(AttrA);

		TArray<TArray<int32>> Result = PCGMetadataPartitionCommon::AttributeGenericPartition(Data, Selector, &Context);
		VerifyGenericPartition(Result, {{0, 2, 4}, {1, 3}});
	}

	SECTION("Default value entries group together")
	{
		// Entries that were never explicitly set should resolve to the default
		// and partition into the same bucket.
		UPCGParamData* Data = CreateData<UPCGParamData>();
		constexpr int32 DefaultValue = 42;
		constexpr int32 Sentinel = 99;
		FPCGMetadataAttribute<int32>* Attr = Data->Metadata->CreateAttribute<int32>(AttrA, DefaultValue, false, false);
		check(Attr);

		Data->Metadata->AddEntry();                           // entry 0: implicit default (42)
		Attr->SetValue(Data->Metadata->AddEntry(), Sentinel); // entry 1: explicit 99
		Data->Metadata->AddEntry();                           // entry 2: implicit default (42)
		Attr->SetValue(Data->Metadata->AddEntry(), Sentinel); // entry 3: explicit 99

		const FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateAttributeSelector(AttrA);
		TArray<TArray<int32>> Result = PCGMetadataPartitionCommon::AttributeGenericPartition(Data, Selector, &Context);
		VerifyGenericPartition(Result, {{0, 2}, {1, 3}});
	}

	SECTION("Partition on point property Density")
	{
		UPCGPointArrayData* Data = CreateData<UPCGPointArrayData>();
		SetupTestPointData(Data, 6);
		TPCGValueRange<float> DensityRange = Data->GetDensityValueRange();
		DensityRange[0] = 0.0f;
		DensityRange[1] = 0.5f;
		DensityRange[2] = 1.0f;
		DensityRange[3] = 0.0f;
		DensityRange[4] = 0.5f;
		DensityRange[5] = 1.0f;

		FPCGAttributePropertySelector Selector;
		Selector.SetPointProperty(EPCGPointProperties::Density);

		TArray<TArray<int32>> Result = PCGMetadataPartitionCommon::AttributeGenericPartition(Data, Selector, &Context);
		VerifyGenericPartition(Result, {{0, 3}, {1, 4}, {2, 5}});
	}

	SECTION("Missing attribute returns empty and logs error")
	{
		UPCGParamData* Data = CreateData<UPCGParamData>();
		SetupParamDataWithIntAttribute(Data, AttrA, {1, 2, 3});
		const FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("NonExistent"));

		PCGTests::FPCGTestsLogOutputDevice LogCapture(/*bSuppressErrors=*/true);
		TArray<TArray<int32>> Result = PCGMetadataPartitionCommon::AttributeGenericPartition(Data, Selector, &Context);
		CHECK(Result.IsEmpty());
		CHECK(LogCapture.NbMessageReceived.Contains(ELogVerbosity::Error));
	}

	SECTION("Missing attribute with bSilenceMissingAttributeErrors does not log")
	{
		UPCGParamData* Data = CreateData<UPCGParamData>();
		SetupParamDataWithIntAttribute(Data, AttrA, {1, 2, 3});
		const FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("NonExistent"));

		PCGTests::FPCGTestsLogOutputDevice LogCapture(/*bSuppressErrors=*/true);
		TArray<TArray<int32>> Result = PCGMetadataPartitionCommon::AttributeGenericPartition(Data, Selector, &Context, /*bSilenceMissingAttributeErrors=*/true);
		CHECK(Result.IsEmpty());
		CHECK_FALSE(LogCapture.NbMessageReceived.Contains(ELogVerbosity::Error));
	}

	SECTION("FTransform attribute is unsupported and logs error")
	{
		UPCGParamData* Data = CreateData<UPCGParamData>();
		FPCGMetadataAttribute<FTransform>* Attr = Data->Metadata->CreateAttribute<FTransform>(AttrA, FTransform::Identity, false, false);
		check(Attr);
		Attr->SetValue(Data->Metadata->AddEntry(), FTransform::Identity);

		const FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateAttributeSelector(AttrA);

		PCGTests::FPCGTestsLogOutputDevice LogCapture(/*bSuppressErrors=*/true);
		TArray<TArray<int32>> Result = PCGMetadataPartitionCommon::AttributeGenericPartition(Data, Selector, &Context);
		CHECK(Result.IsEmpty());
		CHECK(LogCapture.NbMessageReceived.Contains(ELogVerbosity::Error));
	}
}

/**
 * AttributeGenericPartition with multiple selectors.
 * Each unique combination of attribute values should produce its own partition bucket.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributePartition::GenericPartition::MultiSelector", "[PCG][AttributePartition]")
{
	using namespace PCGMetadataPartitionCommonTests;

	SECTION("Single selector in array uses single-selector fast path")
	{
		UPCGParamData* Data = CreateData<UPCGParamData>();
		SetupParamDataWithIntAttribute(Data, AttrA, {1, 2, 1});
		TArray<FPCGAttributePropertySelector> Selectors;
		Selectors.Add(FPCGAttributePropertySelector::CreateAttributeSelector(AttrA));

		TArray<TArray<int32>> Result = PCGMetadataPartitionCommon::AttributeGenericPartition(Data, Selectors, &Context);
		VerifyGenericPartition(Result, {{0, 2}, {1}});
	}

	SECTION("Two attributes refine partition into unique (A,B) groups")
	{
		// Pt  A  B        Expected groups: {0}, {1}, {2}, {3,4}
		//  0  0  0        (0,0) unique
		//  1  0  1        (0,1) unique
		//  2  1  1        (1,1) unique
		//  3  1  2        (1,2) shared with pt 4
		//  4  1  2
		UPCGParamData* Data = CreateData<UPCGParamData>();
		FPCGMetadataAttribute<int32>* A = Data->Metadata->CreateAttribute<int32>(AttrA, 0, false, false);
		FPCGMetadataAttribute<int32>* B = Data->Metadata->CreateAttribute<int32>(AttrB, 0, false, false);
		check(A && B);

		TArray<int32> ValA = {0, 0, 1, 1, 1};
		TArray<int32> ValB = {0, 1, 1, 2, 2};
		for (int32 i = 0; i < ValA.Num(); ++i)
		{
			PCGMetadataEntryKey Key = Data->Metadata->AddEntry();
			A->SetValue(Key, ValA[i]);
			B->SetValue(Key, ValB[i]);
		}

		TArray<FPCGAttributePropertySelector> Selectors;
		Selectors.Add(FPCGAttributePropertySelector::CreateAttributeSelector(AttrA));
		Selectors.Add(FPCGAttributePropertySelector::CreateAttributeSelector(AttrB));

		TArray<TArray<int32>> Result = PCGMetadataPartitionCommon::AttributeGenericPartition(Data, Selectors, &Context);
		VerifyGenericPartition(Result, {{0}, {1}, {2}, {3, 4}});
	}

	SECTION("Three attributes produce unique (A,B,C) groups")
	{
		// Pt  A  B  C        Expected groups: {0}, {1}, {2}, {3,4}
		//  0  0  0  0        (0,0,0) unique
		//  1  0  1  0        (0,1,0) unique
		//  2  1  1  1        (1,1,1) unique
		//  3  1  2  2        (1,2,2) shared with pt 4
		//  4  1  2  2
		UPCGParamData* Data = CreateData<UPCGParamData>();
		FPCGMetadataAttribute<int32>* A = Data->Metadata->CreateAttribute<int32>(AttrA, 0, false, false);
		FPCGMetadataAttribute<int32>* B = Data->Metadata->CreateAttribute<int32>(AttrB, 0, false, false);
		FPCGMetadataAttribute<int32>* C = Data->Metadata->CreateAttribute<int32>(AttrC, 0, false, false);
		check(A && B && C);

		TArray<int32> ValA = {0, 0, 1, 1, 1};
		TArray<int32> ValB = {0, 1, 1, 2, 2};
		TArray<int32> ValC = {0, 0, 1, 2, 2};
		for (int32 i = 0; i < ValA.Num(); ++i)
		{
			PCGMetadataEntryKey Key = Data->Metadata->AddEntry();
			A->SetValue(Key, ValA[i]);
			B->SetValue(Key, ValB[i]);
			C->SetValue(Key, ValC[i]);
		}

		TArray<FPCGAttributePropertySelector> Selectors;
		Selectors.Add(FPCGAttributePropertySelector::CreateAttributeSelector(AttrA));
		Selectors.Add(FPCGAttributePropertySelector::CreateAttributeSelector(AttrB));
		Selectors.Add(FPCGAttributePropertySelector::CreateAttributeSelector(AttrC));

		TArray<TArray<int32>> Result = PCGMetadataPartitionCommon::AttributeGenericPartition(Data, Selectors, &Context);
		VerifyGenericPartition(Result, {{0}, {1}, {2}, {3, 4}});
	}

	SECTION("Empty selector array returns empty")
	{
		UPCGParamData* Data = CreateData<UPCGParamData>();
		SetupParamDataWithIntAttribute(Data, AttrA, {1, 2, 3});
		TArray<FPCGAttributePropertySelector> EmptySelectors;

		TArray<TArray<int32>> Result = PCGMetadataPartitionCommon::AttributeGenericPartition(Data, EmptySelectors, &Context);
		CHECK(Result.IsEmpty());
	}

	SECTION("Null data with multi selectors returns empty")
	{
		TArray<FPCGAttributePropertySelector> Selectors;
		Selectors.Add(FPCGAttributePropertySelector::CreateAttributeSelector(AttrA));
		Selectors.Add(FPCGAttributePropertySelector::CreateAttributeSelector(AttrB));

		TArray<TArray<int32>> Result = PCGMetadataPartitionCommon::AttributeGenericPartition(nullptr, Selectors, &Context);
		CHECK(Result.IsEmpty());
	}

	SECTION("Bitwise intersection fallback produces same result as mixed radix")
	{
		// Disable mixed radix via CVar to force the bitwise intersection slow path.
		// Verify it produces the same partition as the default algorithm.
		UPCGParamData* Data = CreateData<UPCGParamData>();
		FPCGMetadataAttribute<int32>* A = Data->Metadata->CreateAttribute<int32>(AttrA, 0, false, false);
		FPCGMetadataAttribute<int32>* B = Data->Metadata->CreateAttribute<int32>(AttrB, 0, false, false);
		check(A && B);

		TArray<int32> ValA = {0, 0, 1, 1, 1};
		TArray<int32> ValB = {0, 1, 1, 2, 2};
		for (int32 i = 0; i < ValA.Num(); ++i)
		{
			PCGMetadataEntryKey Key = Data->Metadata->AddEntry();
			A->SetValue(Key, ValA[i]);
			B->SetValue(Key, ValB[i]);
		}

		TArray<FPCGAttributePropertySelector> Selectors;
		Selectors.Add(FPCGAttributePropertySelector::CreateAttributeSelector(AttrA));
		Selectors.Add(FPCGAttributePropertySelector::CreateAttributeSelector(AttrB));

		FScopedCVar<int> DisableMixedRadix(TEXT("pcg.MetadataPartition.UseMixedRadixHash"), 0);

		TArray<TArray<int32>> Result = PCGMetadataPartitionCommon::AttributeGenericPartition(Data, Selectors, &Context);
		VerifyGenericPartition(Result, {{0}, {1}, {2}, {3, 4}});
	}
}

/**
 * AttributePartition data-producing API on point data.
 * Tests point partition correctness at the system level rather than through element execution.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributePartition::Points", "[PCG][AttributePartition]")
{
	using namespace PCGMetadataPartitionCommonTests;

	SECTION("Partitions point data on metadata attribute with correct counts and uniform values")
	{
		constexpr int32 NumPoints = 6;
		UPCGPointArrayData* Data = CreateData<UPCGPointArrayData>();
		SetupTestPointData(Data, NumPoints);

		FPCGMetadataAttribute<int32>* Attr = Data->Metadata->CreateAttribute<int32>(AttrA, 0, false, false);
		check(Attr);

		TPCGValueRange<int64> MetadataEntryRange = Data->GetMetadataEntryValueRange();
		// 3 unique values: [10, 20, 30, 10, 20, 30] -> 3 partitions of 2
		TArray<int32> Values = {10, 20, 30, 10, 20, 30};
		for (int32 i = 0; i < NumPoints; ++i)
		{
			Data->Metadata->InitializeOnSet(MetadataEntryRange[i]);
			Attr->SetValue(MetadataEntryRange[i], Values[i]);
		}

		const FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateAttributeSelector(AttrA);
		TArray<UPCGData*> Result = PCGMetadataPartitionCommon::AttributePartition(Data, Selector, &Context);
		REQUIRE_EQUAL(Result.Num(), 3);

		for (UPCGData* PartData : Result)
		{
			const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(PartData);
			REQUIRE_NOT_EQUAL(OutPoints, nullptr);
			CHECK(OutPoints->GetNumPoints() == 2);

			// Attribute should be carried to output
			const FPCGMetadataAttribute<int32>* OutAttr = OutPoints->Metadata->GetConstTypedAttribute<int32>(AttrA);
			CHECK(OutAttr != nullptr);
		}
	}

	SECTION("Partitions preserve stable ordering by first appearance")
	{
		// Input values appear in order [B=20, A=10, B=20, A=10, B=20]
		// First partition should be B (indices 0,2,4), second should be A (indices 1,3)
		// We encode original index in density to verify ordering in output.
		constexpr int32 NumPoints = 5;
		UPCGPointArrayData* Data = CreateData<UPCGPointArrayData>();
		SetupTestPointData(Data, NumPoints);

		FPCGMetadataAttribute<int32>* Attr = Data->Metadata->CreateAttribute<int32>(AttrA, -1, false, false);
		check(Attr);

		TPCGValueRange<float> DensityRange = Data->GetDensityValueRange();
		TPCGValueRange<int64> MetadataEntryRange = Data->GetMetadataEntryValueRange();

		TArray<int32> Values = {20, 10, 20, 10, 20};
		for (int32 i = 0; i < NumPoints; ++i)
		{
			DensityRange[i] = static_cast<float>(i);
			Data->Metadata->InitializeOnSet(MetadataEntryRange[i]);
			Attr->SetValue(MetadataEntryRange[i], Values[i]);
		}

		const FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateAttributeSelector(AttrA);
		TArray<UPCGData*> Result = PCGMetadataPartitionCommon::AttributePartition(Data, Selector, &Context);
		REQUIRE_EQUAL(Result.Num(), 2);

		// First partition: value=20 at original indices 0, 2, 4
		const UPCGBasePointData* First = Cast<UPCGBasePointData>(Result[0]);
		REQUIRE_NOT_EQUAL(First, nullptr);
		REQUIRE_EQUAL(First->GetNumPoints(), 3);
		{
			const TConstPCGValueRange<float> OutDensity = First->GetConstDensityValueRange();
			CHECK(OutDensity[0] == 0.0f);
			CHECK(OutDensity[1] == 2.0f);
			CHECK(OutDensity[2] == 4.0f);
		}

		// Second partition: value=10 at original indices 1, 3
		const UPCGBasePointData* Second = Cast<UPCGBasePointData>(Result[1]);
		REQUIRE_NOT_EQUAL(Second, nullptr);
		REQUIRE_EQUAL(Second->GetNumPoints(), 2);
		{
			const TConstPCGValueRange<float> OutDensity = Second->GetConstDensityValueRange();
			CHECK(OutDensity[0] == 1.0f);
			CHECK(OutDensity[1] == 3.0f);
		}
	}

	SECTION("Null data returns empty")
	{
		FPCGAttributePropertySelector Selector;
		Selector.SetPointProperty(EPCGPointProperties::Density);
		TArray<UPCGData*> Result = PCGMetadataPartitionCommon::AttributePartition(nullptr, Selector);
		CHECK(Result.IsEmpty());
	}

	SECTION("Null data with multi selectors returns empty")
	{
		TArray<FPCGAttributePropertySelector> Selectors;
		Selectors.Add(FPCGAttributePropertySelector::CreateAttributeSelector(AttrA));
		Selectors.Add(FPCGAttributePropertySelector::CreateAttributeSelector(AttrB));

		TArray<UPCGData*> Result = PCGMetadataPartitionCommon::AttributePartition(nullptr, Selectors);
		CHECK(Result.IsEmpty());
	}
}

/**
 * AttributePartition data-producing API on param data.
 * Tests param/attribute-set partition at the system level.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributePartition::ParamData", "[PCG][AttributePartition]")
{
	using namespace PCGMetadataPartitionCommonTests;

	SECTION("Param data partitions into separate param data objects with uniform values per bucket")
	{
		// 15 entries with 3 unique string values -> 3 partitions of 5
		UPCGParamData* Data = CreateData<UPCGParamData>();
		FPCGMetadataAttribute<FString>* Attr = Data->Metadata->CreateAttribute<FString>(AttrA, TEXT(""), false, false);
		check(Attr);

		TArray<FString> Labels = {TEXT("Red"), TEXT("Green"), TEXT("Blue")};
		for (int32 i = 0; i < 15; ++i)
		{
			Attr->SetValue(Data->Metadata->AddEntry(), Labels[i % 3]);
		}

		const FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateAttributeSelector(AttrA);
		TArray<UPCGData*> Result = PCGMetadataPartitionCommon::AttributePartition(Data, Selector, &Context);
		REQUIRE_EQUAL(Result.Num(), 3);

		for (UPCGData* PartData : Result)
		{
			const UPCGParamData* OutParam = Cast<UPCGParamData>(PartData);
			REQUIRE_NOT_EQUAL(OutParam, nullptr);
			CHECK(OutParam->Metadata->GetLocalItemCount() == 5);

			const FPCGMetadataAttribute<FString>* OutAttr = OutParam->Metadata->GetConstTypedAttribute<FString>(AttrA);
			REQUIRE_NOT_EQUAL(OutAttr, nullptr);

			// All entries within a partition should have the same string value
			const FString FirstVal = OutAttr->GetValueFromItemKey(0);
			for (PCGMetadataEntryKey Key = 1; Key < OutParam->Metadata->GetLocalItemCount(); ++Key)
			{
				CHECK(OutAttr->GetValueFromItemKey(Key) == FirstVal);
			}
		}
	}

	SECTION("Multi-selector partition on param data refines into tuple-based groups")
	{
		UPCGParamData* Data = CreateData<UPCGParamData>();
		FPCGMetadataAttribute<int32>* A = Data->Metadata->CreateAttribute<int32>(AttrA, 0, false, false);
		FPCGMetadataAttribute<int32>* B = Data->Metadata->CreateAttribute<int32>(AttrB, 0, false, false);
		check(A && B);

		// 4 entries with 4 unique (A,B) tuples -> 4 partitions of 1
		TArray<int32> ValA = {0, 0, 1, 1};
		TArray<int32> ValB = {0, 1, 0, 1};
		for (int32 i = 0; i < ValA.Num(); ++i)
		{
			PCGMetadataEntryKey Key = Data->Metadata->AddEntry();
			A->SetValue(Key, ValA[i]);
			B->SetValue(Key, ValB[i]);
		}

		TArray<FPCGAttributePropertySelector> Selectors;
		Selectors.Add(FPCGAttributePropertySelector::CreateAttributeSelector(AttrA));
		Selectors.Add(FPCGAttributePropertySelector::CreateAttributeSelector(AttrB));

		TArray<UPCGData*> Result = PCGMetadataPartitionCommon::AttributePartition(Data, Selectors, &Context);
		REQUIRE_EQUAL(Result.Num(), 4);

		for (UPCGData* PartitionData : Result)
		{
			const UPCGParamData* OutParam = Cast<UPCGParamData>(PartitionData);
			REQUIRE_NOT_EQUAL(OutParam, nullptr);
			CHECK(OutParam->Metadata->GetLocalItemCount() == 1);
		}
	}
	
	SECTION("Partition on TArray<FString> attribute groups matching arrays")
	{
		// A TArray<FString> attribute uses the compressed-data storage path, so this also
		// verifies that partition works correctly when the attribute deduplicates array
		// payloads into shared value keys.
		UPCGParamData* Data = CreateData<UPCGParamData>();
		FPCGMetadataAttributeBase* Attr = Data->Metadata->CreateAttribute<TArray<FString>>(AttrA, TArray<FString>{}, false, false);
		check(Attr);
		REQUIRE(Attr->DoesCompressData());

		Attr->SetValue(Data->Metadata->AddEntry(), TArray<FString>{TEXT("Alpha"), TEXT("Beta")});  // 0
		Attr->SetValue(Data->Metadata->AddEntry(), TArray<FString>{TEXT("Gamma")});                // 1
		Attr->SetValue(Data->Metadata->AddEntry(), TArray<FString>{TEXT("Alpha"), TEXT("Beta")});  // 2 -> same as 0
		Attr->SetValue(Data->Metadata->AddEntry(), TArray<FString>{TEXT("Delta")});                // 3
		Attr->SetValue(Data->Metadata->AddEntry(), TArray<FString>{TEXT("Gamma")});                // 4 -> same as 1

		const FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateAttributeSelector(AttrA);
		TArray<TArray<int32>> Result = PCGMetadataPartitionCommon::AttributeGenericPartition(Data, Selector, &Context);
		VerifyGenericPartition(Result, {{0, 2}, {1, 4}, {3}});
	}

}

/** RemoveDuplicates API tests. */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributePartition::RemoveDuplicates", "[PCG][AttributePartition]")
{
	using namespace PCGMetadataPartitionCommonTests;

	SECTION("Keeps first occurrence per unique value on point data")
	{
		constexpr int32 NumPoints = 6;
		UPCGPointArrayData* Data = CreateData<UPCGPointArrayData>();
		SetupTestPointData(Data, NumPoints);
		TPCGValueRange<float> DensityRange = Data->GetDensityValueRange();
		// 3 unique density values, each repeated twice
		DensityRange[0] = 1.0f;
		DensityRange[1] = 2.0f;
		DensityRange[2] = 3.0f;
		DensityRange[3] = 1.0f;
		DensityRange[4] = 2.0f;
		DensityRange[5] = 3.0f;

		FPCGAttributePropertySelector Selector;
		Selector.SetPointProperty(EPCGPointProperties::Density);

		UPCGData* Result = PCGMetadataPartitionCommon::RemoveDuplicates(Data, Selector, &Context);
		REQUIRE_NOT_EQUAL(Result, nullptr);

		const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(Result);
		REQUIRE_NOT_EQUAL(OutPoints, nullptr);
		CHECK(OutPoints->GetNumPoints() == 3);
	}

	SECTION("All unique values returns all points unchanged")
	{
		constexpr int32 NumPoints = 4;
		UPCGPointArrayData* Data = CreateData<UPCGPointArrayData>();
		SetupTestPointData(Data, NumPoints);
		TPCGValueRange<float> DensityRange = Data->GetDensityValueRange();
		DensityRange[0] = 1.0f;
		DensityRange[1] = 2.0f;
		DensityRange[2] = 3.0f;
		DensityRange[3] = 4.0f;

		FPCGAttributePropertySelector Selector;
		Selector.SetPointProperty(EPCGPointProperties::Density);

		UPCGData* Result = PCGMetadataPartitionCommon::RemoveDuplicates(Data, Selector, &Context);
		REQUIRE_NOT_EQUAL(Result, nullptr);

		const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(Result);
		REQUIRE_NOT_EQUAL(OutPoints, nullptr);
		CHECK(OutPoints->GetNumPoints() == 4);
	}

	SECTION("All same value returns single point")
	{
		constexpr int32 NumPoints = 5;
		// Default density is uniform from SetupTestPointData
		UPCGPointArrayData* Data = CreateData<UPCGPointArrayData>();
		SetupTestPointData(Data, NumPoints);

		FPCGAttributePropertySelector Selector;
		Selector.SetPointProperty(EPCGPointProperties::Density);

		UPCGData* Result = PCGMetadataPartitionCommon::RemoveDuplicates(Data, Selector, &Context);
		REQUIRE_NOT_EQUAL(Result, nullptr);

		const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(Result);
		REQUIRE_NOT_EQUAL(OutPoints, nullptr);
		CHECK(OutPoints->GetNumPoints() == 1);
	}

	SECTION("Removes duplicate entries from param data")
	{
		UPCGParamData* Data = CreateData<UPCGParamData>();
		SetupParamDataWithIntAttribute(Data, AttrA, {1, 2, 3, 1, 2, 3, 1, 2, 3});
		const FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateAttributeSelector(AttrA);

		UPCGData* Result = PCGMetadataPartitionCommon::RemoveDuplicates(Data, Selector, &Context);
		REQUIRE_NOT_EQUAL(Result, nullptr);

		const UPCGParamData* OutParam = Cast<UPCGParamData>(Result);
		REQUIRE_NOT_EQUAL(OutParam, nullptr);
		CHECK(OutParam->Metadata->GetLocalItemCount() == 3);
	}

	SECTION("Multi-selector removes duplicates by attribute tuple")
	{
		UPCGParamData* Data = CreateData<UPCGParamData>();
		FPCGMetadataAttribute<int32>* A = Data->Metadata->CreateAttribute<int32>(AttrA, 0, false, false);
		FPCGMetadataAttribute<int32>* B = Data->Metadata->CreateAttribute<int32>(AttrB, 0, false, false);
		check(A && B);

		// 6 entries, 3 unique (A,B) tuples: (0,0), (0,1), (1,0) each repeated twice
		TArray<int32> ValA = {0, 0, 1, 0, 0, 1};
		TArray<int32> ValB = {0, 1, 0, 0, 1, 0};
		for (int32 i = 0; i < ValA.Num(); ++i)
		{
			PCGMetadataEntryKey Key = Data->Metadata->AddEntry();
			A->SetValue(Key, ValA[i]);
			B->SetValue(Key, ValB[i]);
		}

		TArray<FPCGAttributePropertySelector> Selectors;
		Selectors.Add(FPCGAttributePropertySelector::CreateAttributeSelector(AttrA));
		Selectors.Add(FPCGAttributePropertySelector::CreateAttributeSelector(AttrB));

		UPCGData* Result = PCGMetadataPartitionCommon::RemoveDuplicates(Data, Selectors, &Context);
		REQUIRE_NOT_EQUAL(Result, nullptr);

		const UPCGParamData* OutParam = Cast<UPCGParamData>(Result);
		REQUIRE_NOT_EQUAL(OutParam, nullptr);
		CHECK(OutParam->Metadata->GetLocalItemCount() == 3);
	}

	SECTION("Null data returns nullptr")
	{
		FPCGAttributePropertySelector Selector;
		Selector.SetPointProperty(EPCGPointProperties::Density);
		UPCGData* Result = PCGMetadataPartitionCommon::RemoveDuplicates(nullptr, Selector);
		CHECK(Result == nullptr);
	}

	SECTION("Missing attribute returns nullptr")
	{
		UPCGParamData* Data = CreateData<UPCGParamData>();
		SetupParamDataWithIntAttribute(Data, AttrA, {1, 2, 3});
		const FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("DoesNotExist"));

		PCGTests::FPCGTestsLogOutputDevice LogCapture(/*bSuppressErrors=*/true);
		UPCGData* Result = PCGMetadataPartitionCommon::RemoveDuplicates(Data, Selector, &Context);
		CHECK(Result == nullptr);
	}

	SECTION("Multi-selector removes duplicates on point data by attribute tuple")
	{
		constexpr int32 NumPoints = 6;
		UPCGPointArrayData* Data = CreateData<UPCGPointArrayData>();
		SetupTestPointData(Data, NumPoints);

		FPCGMetadataAttribute<int32>* A = Data->Metadata->CreateAttribute<int32>(AttrA, 0, false, false);
		FPCGMetadataAttribute<int32>* B = Data->Metadata->CreateAttribute<int32>(AttrB, 0, false, false);
		check(A && B);

		TPCGValueRange<int64> MetadataEntryRange = Data->GetMetadataEntryValueRange();

		// 6 points, 3 unique (A,B) tuples: (0,0), (0,1), (1,0) each repeated twice
		TArray<int32> ValA = {0, 0, 1, 0, 0, 1};
		TArray<int32> ValB = {0, 1, 0, 0, 1, 0};
		for (int32 i = 0; i < NumPoints; ++i)
		{
			Data->Metadata->InitializeOnSet(MetadataEntryRange[i]);
			A->SetValue(MetadataEntryRange[i], ValA[i]);
			B->SetValue(MetadataEntryRange[i], ValB[i]);
		}

		TArray<FPCGAttributePropertySelector> Selectors;
		Selectors.Add(FPCGAttributePropertySelector::CreateAttributeSelector(AttrA));
		Selectors.Add(FPCGAttributePropertySelector::CreateAttributeSelector(AttrB));

		UPCGData* Result = PCGMetadataPartitionCommon::RemoveDuplicates(Data, Selectors, &Context);
		REQUIRE_NOT_EQUAL(Result, nullptr);

		const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(Result);
		REQUIRE_NOT_EQUAL(OutPoints, nullptr);
		CHECK(OutPoints->GetNumPoints() == 3);
	}
}

/** Edge cases -- type variations and boundary conditions. */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::AttributePartition::EdgeCases", "[PCG][AttributePartition]")
{
	using namespace PCGMetadataPartitionCommonTests;

	SECTION("Large data crossing internal chunk boundary")
	{
		// Internal ChunkSize is 256. 300 points exercises multi-chunk iteration.
		constexpr int32 NumPoints = 300;
		UPCGPointArrayData* Data = CreateData<UPCGPointArrayData>();
		SetupTestPointData(Data, NumPoints);
		TPCGValueRange<float> DensityRange = Data->GetDensityValueRange();
		for (int32 i = 0; i < NumPoints; ++i)
		{
			DensityRange[i] = static_cast<float>(i % 3);
		}

		FPCGAttributePropertySelector Selector;
		Selector.SetPointProperty(EPCGPointProperties::Density);

		TArray<UPCGData*> Result = PCGMetadataPartitionCommon::AttributePartition(Data, Selector, &Context);
		REQUIRE_EQUAL(Result.Num(), 3);

		for (UPCGData* PartData : Result)
		{
			const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(PartData);
			REQUIRE_NOT_EQUAL(OutPoints, nullptr);
			CHECK(OutPoints->GetNumPoints() == 100);
		}
	}

	SECTION("Partition on FVector attribute groups identical vectors")
	{
		UPCGParamData* Data = CreateData<UPCGParamData>();
		FPCGMetadataAttribute<FVector>* Attr = Data->Metadata->CreateAttribute<FVector>(AttrA, FVector::ZeroVector, false, false);
		check(Attr);

		Attr->SetValue(Data->Metadata->AddEntry(), FVector(1, 0, 0));
		Attr->SetValue(Data->Metadata->AddEntry(), FVector(0, 1, 0));
		Attr->SetValue(Data->Metadata->AddEntry(), FVector(1, 0, 0));
		Attr->SetValue(Data->Metadata->AddEntry(), FVector(0, 1, 0));

		const FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateAttributeSelector(AttrA);
		TArray<TArray<int32>> Result = PCGMetadataPartitionCommon::AttributeGenericPartition(Data, Selector, &Context);
		VerifyGenericPartition(Result, {{0, 2}, {1, 3}});
	}

	SECTION("Partition on string attribute groups matching strings")
	{
		UPCGParamData* Data = CreateData<UPCGParamData>();
		FPCGMetadataAttribute<FString>* Attr = Data->Metadata->CreateAttribute<FString>(AttrA, TEXT(""), false, false);
		check(Attr);

		Attr->SetValue(Data->Metadata->AddEntry(), FString(TEXT("Alpha")));
		Attr->SetValue(Data->Metadata->AddEntry(), FString(TEXT("Beta")));
		Attr->SetValue(Data->Metadata->AddEntry(), FString(TEXT("Alpha")));

		const FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateAttributeSelector(AttrA);
		TArray<TArray<int32>> Result = PCGMetadataPartitionCommon::AttributeGenericPartition(Data, Selector, &Context);
		VerifyGenericPartition(Result, {{0, 2}, {1}});
	}

	SECTION("Multi-selector point partition with Position")
	{
		// Testing the system API directly with Position.X, Position.Y selectors on point data
		constexpr int32 NumPoints = 20;
		UPCGPointArrayData* Data = CreateData<UPCGPointArrayData>();
		SetupTestPointData(Data, NumPoints);
		TPCGValueRange<FTransform> TransformRange = Data->GetTransformValueRange();

		for (int32 i = 0; i < NumPoints; ++i)
		{
			// Small jitter to verify floating point equality tolerance
			const double Jitter = i >= 10 ? UE_DOUBLE_SMALL_NUMBER : 0.0;
			TransformRange[i].SetLocation(FVector(i % 10, i % 5 + Jitter, 0.0));
		}

		FPCGAttributePropertySelector SelectorX = FPCGAttributePropertySelector::CreatePointPropertySelector(EPCGPointProperties::Position, NAME_None, TArray<FString>{TEXT("X")});
		FPCGAttributePropertySelector SelectorY = FPCGAttributePropertySelector::CreatePointPropertySelector(EPCGPointProperties::Position, NAME_None, TArray<FString>{TEXT("Y")});

		TArray<FPCGAttributePropertySelector> Selectors = {SelectorX, SelectorY};

		TArray<UPCGData*> Result = PCGMetadataPartitionCommon::AttributePartition(Data, Selectors, &Context);
		// 10 unique X values x 5 unique Y values, but with the given pattern only 10 unique (X,Y) pairs exist
		REQUIRE_EQUAL(Result.Num(), 10);

		for (UPCGData* PartData : Result)
		{
			const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(PartData);
			REQUIRE_NOT_EQUAL(OutPoints, nullptr);
			CHECK(OutPoints->GetNumPoints() == 2);
		}
	}

	// @todo_pcg: Test uncovered a bug. To be triaged, investigated, and fixed in a follow up pass.
	// SECTION("Partition on FRotator attribute groups identical rotations")
	// {
	// 	// Verify that identical rotations group together.
	// 	UPCGParamData* Data = CreateData<UPCGParamData>();
	// 	FPCGMetadataAttribute<FRotator>* Attr = Data->Metadata->CreateAttribute<FRotator>(AttrA, FRotator::ZeroRotator, false, false);
	// 	check(Attr);
	//
	// 	Attr->SetValue(Data->Metadata->AddEntry(), FRotator(0.0, 90.0, 0.0));
	// 	Attr->SetValue(Data->Metadata->AddEntry(), FRotator(45.0, 0.0, 0.0));
	// 	Attr->SetValue(Data->Metadata->AddEntry(), FRotator(0.0, 90.0, 0.0));
	// 	Attr->SetValue(Data->Metadata->AddEntry(), FRotator(45.0, 0.0, 0.0));
	//
	// 	const FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateAttributeSelector(AttrA);
	// 	TArray<TArray<int32>> Result = PCGMetadataPartitionCommon::AttributeGenericPartition(Data, Selector, &Context);
	// 	VerifyGenericPartition(Result, {{0, 2}, {1, 3}});
	// }
}
