// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGTestsCommon.h"

#include "PCGParamData.h"
#include "Data/PCGPointArrayData.h"
#include "Elements/PCGCopyAttributes.h"
#include "Elements/PCGCopyPoints.h"
#include "Elements/PCGMergeElement.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/PCGMetadataAttributeTpl.h"

#include <catch2/catch_test_macros.hpp>

#include "Containers/StaticArray.h"
#include "Serialization/ArchiveCrc32.h"
#include "TestHarness.h"
#include "Engine/Engine.h"

/**
 * Test metadata inheritance chain with pointer-based parenting through point data.
 * Creates PointData (5 pts, values 0-4) -> PointData2 inherits, adds pts 5-9 -> PointData3 inherits.
 * Verifies all 10 values are readable from PointData3.
 *
 * Note: The original automation test forced a GC run between setup and verification to test that
 * metadata chains survive parent object collection. This cannot be replicated in low-level Catch2 tests
 * because objects created through the test context are kept alive for the test duration.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Metadata::Inherit", "[PCG][Metadata]")
{
	const FName AttributeName = TEXT("MyAttr");

	UPCGPointArrayData* PointData = CreateData<UPCGPointArrayData>();
	PointData->SetNumPoints(5);
	UPCGMetadata* PointMetadata = PointData->MutableMetadata();
	FPCGMetadataAttribute<int32>* Attribute = PointMetadata->CreateAttribute<int32>(AttributeName, 0, true, false);

	TPCGValueRange<int64> MetadataEntryRange = PointData->GetMetadataEntryValueRange();
	for (int32 i = 0; i < PointData->GetNumPoints(); ++i)
	{
		PointMetadata->InitializeOnSet(MetadataEntryRange[i]);
		Attribute->SetValue(MetadataEntryRange[i], i);
	}

	UPCGPointArrayData* PointData2 = CreateData<UPCGPointArrayData>();
	UPCGMetadata* PointMetadata2 = PointData2->MutableMetadata();
	PointMetadata2->Initialize(PointMetadata);

	// Verify that the initialization inherited (pointer-based, no local entries)
	REQUIRE_EQUAL(PointMetadata2->GetLocalItemCount(), (int64)0);

	FPCGMetadataAttribute<int32>* Attribute2 = PointMetadata2->GetMutableTypedAttribute<int32>(AttributeName);
	REQUIRE_NOT_EQUAL(Attribute2, nullptr);

	UPCGBasePointData::SetPoints(PointData, PointData2, {}, /*bCopyAll=*/true);
	PointData2->SetNumPoints(10);

	TPCGValueRange<int64> MetadataEntryRange2 = PointData2->GetMetadataEntryValueRange();
	for (int32 i = PointData->GetNumPoints(); i < PointData2->GetNumPoints(); ++i)
	{
		PointMetadata2->InitializeOnSet(MetadataEntryRange2[i]);
		Attribute2->SetValue(MetadataEntryRange2[i], i);
	}

	UPCGPointArrayData* PointData3 = CreateData<UPCGPointArrayData>();
	UPCGMetadata* PointMetadata3 = PointData3->MutableMetadata();
	PointMetadata3->Initialize(PointMetadata2);

	REQUIRE_EQUAL(PointMetadata3->GetLocalItemCount(), (int64)0);

	UPCGBasePointData::SetPoints(PointData2, PointData3, {}, /*bCopyAll=*/true);

	REQUIRE_EQUAL(PointData2->GetNumPoints(), PointData3->GetNumPoints());

	// Verify that all the values are readable from the metadata
	const FPCGMetadataAttribute<int32>* Attribute3 = PointData3->ConstMetadata()->GetConstTypedAttribute<int32>(AttributeName);
	REQUIRE_NOT_EQUAL(Attribute3, nullptr);

	TConstPCGValueRange<int64> MetadataEntryRange3 = PointData3->GetConstMetadataEntryValueRange();
	for (int32 i = 0; i < PointData3->GetNumPoints(); ++i)
	{
		REQUIRE_EQUAL(Attribute3->GetValueFromItemKey(MetadataEntryRange3[i]), i);
	}
}

/**
 * Test metadata inheritance with copy-based parenting (InitializeAsCopy).
 * Same chain as Inherit, but PointData3 uses InitializeAsCopy instead of Initialize.
 * This means PointData3 gets its own copy of entries rather than pointing to a parent.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Metadata::InheritCopy", "[PCG][Metadata]")
{
	const FName AttributeName = TEXT("MyAttr");

	UPCGPointArrayData* PointData = CreateData<UPCGPointArrayData>();
	PointData->SetNumPoints(5);
	UPCGMetadata* PointMetadata = PointData->MutableMetadata();
	FPCGMetadataAttribute<int32>* Attribute = PointMetadata->CreateAttribute<int32>(AttributeName, 0, true, false);

	TPCGValueRange<int64> MetadataEntryRange = PointData->GetMetadataEntryValueRange();
	for (int32 i = 0; i < PointData->GetNumPoints(); ++i)
	{
		PointMetadata->InitializeOnSet(MetadataEntryRange[i]);
		Attribute->SetValue(MetadataEntryRange[i], i);
	}

	UPCGPointArrayData* PointData2 = CreateData<UPCGPointArrayData>();
	UPCGMetadata* PointMetadata2 = PointData2->MutableMetadata();
	PointMetadata2->Initialize(PointMetadata);

	// Verify that the initialization inherited (pointer-based, no local entries)
	REQUIRE_EQUAL(PointMetadata2->GetLocalItemCount(), (int64)0);

	FPCGMetadataAttribute<int32>* Attribute2 = PointMetadata2->GetMutableTypedAttribute<int32>(AttributeName);
	REQUIRE_NOT_EQUAL(Attribute2, nullptr);

	UPCGBasePointData::SetPoints(PointData, PointData2, {}, /*bCopyAll=*/true);
	PointData2->SetNumPoints(10);

	TPCGValueRange<int64> MetadataEntryRange2 = PointData2->GetMetadataEntryValueRange();
	for (int32 i = PointData->GetNumPoints(); i < PointData2->GetNumPoints(); ++i)
	{
		PointMetadata2->InitializeOnSet(MetadataEntryRange2[i]);
		Attribute2->SetValue(MetadataEntryRange2[i], i);
	}

	UPCGPointArrayData* PointData3 = CreateData<UPCGPointArrayData>();
	UPCGMetadata* PointMetadata3 = PointData3->MutableMetadata();
	PointMetadata3->InitializeAsCopy(FPCGMetadataInitializeParams(PointMetadata2));

	// Verify that the initialization made a copy (PointData2 had 5 local entries for indices 5-9)
	REQUIRE_EQUAL(PointMetadata3->GetLocalItemCount(), (int64)5);

	UPCGBasePointData::SetPoints(PointData2, PointData3, {}, /*bCopyAll=*/true);

	REQUIRE_EQUAL(PointData2->GetNumPoints(), PointData3->GetNumPoints());

	// Verify that all the values are readable from the metadata
	const FPCGMetadataAttribute<int32>* Attribute3 = PointData3->ConstMetadata()->GetConstTypedAttribute<int32>(AttributeName);
	REQUIRE_NOT_EQUAL(Attribute3, nullptr);

	TConstPCGValueRange<int64> MetadataEntryRange3 = PointData3->GetConstMetadataEntryValueRange();
	for (int32 i = 0; i < PointData3->GetNumPoints(); ++i)
	{
		REQUIRE_EQUAL(Attribute3->GetValueFromItemKey(MetadataEntryRange3[i]), i);
	}
}

/**
 * Test metadata inheritance with ParamData, which does not support pointer-based parenting.
 * Initialize always copies entries for ParamData.
 * Creates ParamData (5 entries, values 0-4) -> ParamData2 (copies + 5 more, values 0-9) -> ParamData3 (copies all 10).
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Metadata::InheritWithNoParenting", "[PCG][Metadata]")
{
	const FName AttributeName = TEXT("MyAttr");

	UPCGParamData* ParamData = CreateData<UPCGParamData>();
	UPCGMetadata* ParamMetadata = ParamData->MutableMetadata();
	FPCGMetadataAttribute<int32>* Attribute = ParamMetadata->CreateAttribute<int32>(AttributeName, 0, true, false);

	for (int32 i = 0; i < 5; ++i)
	{
		PCGMetadataEntryKey EntryKey = PCGInvalidEntryKey;
		ParamMetadata->InitializeOnSet(EntryKey);
		Attribute->SetValue(EntryKey, i);
	}

	UPCGParamData* ParamData2 = CreateData<UPCGParamData>();
	UPCGMetadata* ParamMetadata2 = ParamData2->MutableMetadata();
	ParamMetadata2->Initialize(ParamMetadata);

	// Verify that the initialization made a copy (ParamData doesn't support pointer-based parenting)
	REQUIRE_EQUAL(ParamMetadata2->GetLocalItemCount(), (int64)5);

	FPCGMetadataAttribute<int32>* Attribute2 = ParamMetadata2->GetMutableTypedAttribute<int32>(AttributeName);
	REQUIRE_NOT_EQUAL(Attribute2, nullptr);

	for (int32 i = 5; i < 10; ++i)
	{
		PCGMetadataEntryKey EntryKey = PCGInvalidEntryKey;
		ParamMetadata2->InitializeOnSet(EntryKey);
		Attribute2->SetValue(EntryKey, i);
	}

	UPCGParamData* ParamData3 = CreateData<UPCGParamData>();
	UPCGMetadata* ParamMetadata3 = ParamData3->MutableMetadata();
	ParamMetadata3->Initialize(ParamMetadata2);

	// Verify that the initialization made a copy
	REQUIRE_EQUAL(ParamMetadata3->GetLocalItemCount(), (int64)10);

	// Verify that all the values are readable from the metadata
	const FPCGMetadataAttribute<int32>* Attribute3 = ParamData3->ConstMetadata()->GetConstTypedAttribute<int32>(AttributeName);
	REQUIRE_NOT_EQUAL(Attribute3, nullptr);

	for (int32 i = 0; i < 10; ++i)
	{
		REQUIRE_EQUAL(Attribute3->GetValueFromItemKey(i), i);
	}
}

/**
 * Test that the Data metadata domain only supports a single entry.
 * Adding two entries via InitializeOnSet should produce an error and keep only one entry with the latest value.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Metadata::TwoAddEntryToData", "[PCG][Metadata]")
{
	const FName AttributeName = TEXT("MyAttr");

	UPCGParamData* ParamData = CreateData<UPCGParamData>();
	UPCGMetadata* ParamMetadata = ParamData->MutableMetadata();
	FPCGMetadataDomain* ParamMetadataDomain = ParamMetadata->GetMetadataDomain(PCGMetadataDomainID::Data);
	FPCGMetadataAttribute<int32>* Attribute = ParamMetadataDomain->CreateAttribute<int32>(AttributeName, 0, false, false);
	REQUIRE_NOT_EQUAL(Attribute, nullptr);

	{
		PCGTests::FPCGTestsLogOutputDevice LogCapture(/*bSuppressErrors=*/true);

		for (int32 i = 0; i < 2; ++i)
		{
			PCGMetadataEntryKey EntryKey = PCGInvalidEntryKey;
			ParamMetadataDomain->InitializeOnSet(EntryKey);
			Attribute->SetValue(EntryKey, i);
		}

		// Second InitializeOnSet should have produced a warning since Data domain only supports single entries
		REQUIRE(LogCapture.NbMessageReceived.Contains(ELogVerbosity::Warning));
	}

	REQUIRE_EQUAL(ParamMetadataDomain->GetItemCountForChild(), (int64)1);
	REQUIRE_EQUAL(Attribute->GetValueFromItemKey(PCGFirstEntryKey), 1);
}

/**
 * Test that the Data metadata domain rejects bulk AddEntries and only keeps a single entry.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Metadata::AddEntriesToData", "[PCG][Metadata]")
{
	const FName AttributeName = TEXT("MyAttr");

	UPCGParamData* ParamData = CreateData<UPCGParamData>();
	UPCGMetadata* ParamMetadata = ParamData->MutableMetadata();
	FPCGMetadataDomain* ParamMetadataDomain = ParamMetadata->GetMetadataDomain(PCGMetadataDomainID::Data);
	FPCGMetadataAttribute<int32>* Attribute = ParamMetadataDomain->CreateAttribute<int32>(AttributeName, 0, false, false);
	REQUIRE_NOT_EQUAL(Attribute, nullptr);

	{
		PCGTests::FPCGTestsLogOutputDevice LogCapture(/*bSuppressErrors=*/true);

		TStaticArray<PCGMetadataEntryKey, 2> ParentKeys = {PCGInvalidEntryKey, PCGInvalidEntryKey};
		TArray<PCGMetadataEntryKey> NewKeys = ParamMetadataDomain->AddEntries(ParentKeys);
		TStaticArray<int32, 2> Values = {5, 6};
		Attribute->SetValues(NewKeys, Values);

		REQUIRE(LogCapture.NbMessageReceived.Contains(ELogVerbosity::Warning));
	}

	REQUIRE_EQUAL(ParamMetadataDomain->GetItemCountForChild(), (int64)1);
	REQUIRE_EQUAL(Attribute->GetValueFromItemKey(PCGFirstEntryKey), 6);
}

/**
 * Test that the Data metadata domain rejects AddEntriesInPlace and only keeps a single entry.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Metadata::AddEntriesInPlaceToData", "[PCG][Metadata]")
{
	const FName AttributeName = TEXT("MyAttr");

	UPCGParamData* ParamData = CreateData<UPCGParamData>();
	UPCGMetadata* ParamMetadata = ParamData->MutableMetadata();
	FPCGMetadataDomain* ParamMetadataDomain = ParamMetadata->GetMetadataDomain(PCGMetadataDomainID::Data);
	FPCGMetadataAttribute<int32>* Attribute = ParamMetadataDomain->CreateAttribute<int32>(AttributeName, 0, false, false);
	REQUIRE_NOT_EQUAL(Attribute, nullptr);

	{
		PCGTests::FPCGTestsLogOutputDevice LogCapture(/*bSuppressErrors=*/true);

		TStaticArray<PCGMetadataEntryKey, 2> Keys = {PCGInvalidEntryKey, PCGInvalidEntryKey};
		TStaticArray<PCGMetadataEntryKey*, 2> KeysPtr = {&Keys[0], &Keys[1]};

		ParamMetadataDomain->AddEntriesInPlace(KeysPtr);
		TStaticArray<int32, 2> Values = {5, 6};
		Attribute->SetValues(Keys, Values);

		REQUIRE(LogCapture.NbMessageReceived.Contains(ELogVerbosity::Warning));
	}

	REQUIRE_EQUAL(ParamMetadataDomain->GetItemCountForChild(), (int64)1);
	REQUIRE_EQUAL(Attribute->GetValueFromItemKey(PCGFirstEntryKey), 6);
}

/**
 * Test that the Data metadata domain rejects delayed entries (AddEntryPlaceholder + AddDelayedEntries)
 * and only keeps a single entry.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Metadata::DelayedAddEntriesToData", "[PCG][Metadata]")
{
	const FName AttributeName = TEXT("MyAttr");

	UPCGParamData* ParamData = CreateData<UPCGParamData>();
	UPCGMetadata* ParamMetadata = ParamData->MutableMetadata();
	FPCGMetadataDomain* ParamMetadataDomain = ParamMetadata->GetMetadataDomain(PCGMetadataDomainID::Data);
	FPCGMetadataAttribute<int32>* Attribute = ParamMetadataDomain->CreateAttribute<int32>(AttributeName, 0, false, false);
	REQUIRE_NOT_EQUAL(Attribute, nullptr);

	{
		PCGTests::FPCGTestsLogOutputDevice LogCapture(/*bSuppressErrors=*/true);

		TArray<TTuple<PCGMetadataEntryKey, PCGMetadataEntryKey>> Mapping;
		TStaticArray<PCGMetadataEntryKey, 2> Keys;

		Keys[0] = ParamMetadataDomain->AddEntryPlaceholder();
		Mapping.Emplace(PCGInvalidEntryKey, Keys[0]);
		Keys[1] = ParamMetadataDomain->AddEntryPlaceholder();
		Mapping.Emplace(PCGInvalidEntryKey, Keys[1]);

		TStaticArray<int32, 2> Values = {5, 6};
		Attribute->SetValues(Keys, Values);

		ParamMetadataDomain->AddDelayedEntries(Mapping);

		// Both AddEntryPlaceholder (second call) and AddDelayedEntries should produce warnings
		REQUIRE(LogCapture.NbMessageReceived.Contains(ELogVerbosity::Warning));
	}

	REQUIRE_EQUAL(ParamMetadataDomain->GetItemCountForChild(), (int64)1);
	REQUIRE_EQUAL(Attribute->GetValueFromItemKey(PCGFirstEntryKey), 6);
}

/**
 * Adding entries with a non-legacy attribute type (TArray, TMap) must change the CRC.
 * These types route through the new SerializeValuesForEntryKeys path; this test catches
 * regressions where that path silently no-ops or skips entries.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Metadata::AddToCRC", "[PCG][Metadata][CRC]")
{
	auto ComputeCrc = [](const UPCGMetadata* Metadata)
	{
		FArchiveCrc32 Ar;
		Metadata->AddToCrc(Ar, /*bFullDataCrc=*/true);
		return Ar.GetCrc();
	};

	SECTION("TArray<int32>")
	{
		UPCGParamData* ParamData = CreateData<UPCGParamData>();
		UPCGMetadata* Metadata = ParamData->MutableMetadata();
		FPCGMetadataDomain* Domain = Metadata->GetDefaultMetadataDomain();
		FPCGMetadataAttributeBase* Attribute = Domain->CreateAttribute<TArray<int32>>(TEXT("ArrAttr"), TArray<int32>{}, /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false);
		REQUIRE_NOT_EQUAL(Attribute, nullptr);

		const uint32 EmptyCrc = ComputeCrc(Metadata);

		TArray<PCGMetadataEntryKey> EntryKeys = {Domain->AddEntry(), Domain->AddEntry()};
		TArray<TArray<int32>> Values = {{1, 2, 3}, {4, 5}};
		Attribute->SetValues<TArray<int32>>(MakeConstArrayView(EntryKeys), MakeConstArrayView(Values));

		const int32 CRC = ComputeCrc(Metadata);
		REQUIRE_NOT_EQUAL(CRC, EmptyCrc);
		
		// Adding a new value, changes the CRC
		PCGMetadataEntryKey NewEntry = Domain->AddEntry();
		Attribute->SetValue<TArray<int32>>(NewEntry, TArray<int32>{5, 6, 7});
		
		REQUIRE_NOT_EQUAL(ComputeCrc(Metadata), CRC);
	}

	SECTION("TMap<int32, int32>")
	{
		UPCGParamData* ParamData = CreateData<UPCGParamData>();
		UPCGMetadata* Metadata = ParamData->MutableMetadata();
		FPCGMetadataDomain* Domain = Metadata->GetDefaultMetadataDomain();
		FPCGMetadataAttributeBase* Attribute = Domain->CreateAttribute<TMap<int32, int32>>(TEXT("MapAttr"), TMap<int32, int32>{}, /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false);
		REQUIRE_NOT_EQUAL(Attribute, nullptr);

		const uint32 EmptyCrc = ComputeCrc(Metadata);

		TArray<PCGMetadataEntryKey> EntryKeys = {Domain->AddEntry(), Domain->AddEntry()};
		TArray<TMap<int32, int32>> Values;
		Values.Emplace(TMap<int32, int32>{{1, 10}, {2, 20}});
		Values.Emplace(TMap<int32, int32>{{3, 30}});
		Attribute->SetValues<TMap<int32, int32>>(MakeConstArrayView(EntryKeys), MakeConstArrayView(Values));

		const int32 CRC = ComputeCrc(Metadata);
		REQUIRE_NOT_EQUAL(CRC, EmptyCrc);
		
		// Adding a new value, changes the CRC
		PCGMetadataEntryKey NewEntry = Domain->AddEntry();
		Attribute->SetValue<TMap<int32, int32>>(NewEntry, TMap<int32, int32>{{1, 2}, {3, 4}});
		
		REQUIRE_NOT_EQUAL(ComputeCrc(Metadata), CRC);
	}
}

/**
 * Pins the CRC produced by AddToCrc for every legacy attribute type (Float through FSoftClassPath).
 * Legacy types must keep the pre-refactor byte layout so that asset and dependency-tracking CRCs
 * remain stable across the partitioning refactor.
 *
 * The Expected* values are captured prior to the refactor.
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Metadata::AddToCRC::OldTypes", "[PCG][Metadata][CRC]")
{
	auto MakeAndCrc = [this]<typename T>(const T& Default, const TArray<T>& Values) -> uint32
	{
		UPCGParamData* ParamData = CreateData<UPCGParamData>();
		UPCGMetadata* Metadata = ParamData->MutableMetadata();
		FPCGMetadataAttribute<T>* Attribute = Metadata->CreateAttribute<T>(TEXT("Attr"), Default, /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false);
		REQUIRE_NOT_EQUAL(Attribute, nullptr);

		for (const T& Value : Values)
		{
			PCGMetadataEntryKey EntryKey = PCGInvalidEntryKey;
			Metadata->InitializeOnSet(EntryKey);
			Attribute->SetValue(EntryKey, Value);
		}

		FArchiveCrc32 Ar;
		Metadata->AddToCrc(Ar, /*bFullDataCrc=*/true);
		return Ar.GetCrc();
	};

	SECTION("Float")           
	{
		CHECK_EQUAL(MakeAndCrc(0.0f, TArray<float>{1.0f, 2.0f, 3.0f}), 0x29f39e4a);
	}
	
	SECTION("Double")          
	{
		CHECK_EQUAL(MakeAndCrc(0.0, TArray<double>{1.0, 2.0, 3.0}), 0x798b98b1);
	}
	
	SECTION("Integer32")       
	{
		CHECK_EQUAL(MakeAndCrc(0, TArray<int32>{1, 2, 3}), 0x2b1d2a68);
	}
	
	SECTION("Integer64")       
	{
		CHECK_EQUAL(MakeAndCrc(int64{0}, TArray<int64>{1, 2, 3}), 0x965fd12);
	}
	
	SECTION("Vector2")         
	{
		CHECK_EQUAL(MakeAndCrc(FVector2D::ZeroVector, TArray<FVector2D>{{1.0, 2.0}, {3.0, 4.0}}), 0x1ff2046f);
	}
	
	SECTION("Vector")          
	{
		CHECK_EQUAL(MakeAndCrc(FVector::ZeroVector, TArray<FVector>{{1.0, 2.0, 3.0}, {4.0, 5.0, 6.0}}), 0xce79d432);
	}
	
	SECTION("Vector4")         
	{
		CHECK_EQUAL(MakeAndCrc(FVector4{0.0, 0.0, 0.0, 0.0}, TArray<FVector4>{FVector4{1.0, 2.0, 3.0, 4.0}, FVector4{5.0, 6.0, 7.0, 8.0}}), 0x7301a965);
	}
	
	SECTION("Quaternion")      
	{
		CHECK_EQUAL(MakeAndCrc(FQuat::Identity, TArray<FQuat>{{0.0, 0.0, 0.0, 1.0}, {1.0, 0.0, 0.0, 0.0}}), 0x6bf34d48);
	}
	
	SECTION("Transform")       
	{
		CHECK_EQUAL(MakeAndCrc(FTransform::Identity, TArray<FTransform>{FTransform{FQuat::Identity, FVector{1.0, 2.0, 3.0}}, FTransform{FQuat::Identity, FVector{4.0, 5.0, 6.0}}}), 0xb11f3858);
	}
	
	SECTION("String")          
	{
		CHECK_EQUAL(MakeAndCrc(FString{}, TArray<FString>{TEXT("hello"), TEXT("world")}), 0xc7bf62cf);
	}
	
	SECTION("Boolean")         
	{
		CHECK_EQUAL(MakeAndCrc(false, TArray<bool>{true, false, true}), 0xc331e59e);
	}
	
	SECTION("Rotator")         
	{
		CHECK_EQUAL(MakeAndCrc(FRotator::ZeroRotator, TArray<FRotator>{{10.0, 20.0, 30.0}, {40.0, 50.0, 60.0}}), 0xbf7cd55f);
	}

	// @todo_pcg: Seems like FName don't give a deterministic CRC in this case. Hard to reproduce and seems totally random.
	// Disable for now.
	// SECTION("Name")            
	// {
	// 	CHECK_EQUAL(MakeAndCrc(FName(NAME_None), TArray<FName>{FName{"Alpha"}, FName{"Beta"}}), 0xb2f49d95);
	// }
	
	SECTION("SoftObjectPath")  
	{
		CHECK_EQUAL(MakeAndCrc(FSoftObjectPath{}, TArray<FSoftObjectPath>{FSoftObjectPath{TEXT("/Game/A.A")}, FSoftObjectPath{TEXT("/Game/B.B")}}), 0x101f522b);
	}
	
	SECTION("SoftClassPath")   
	{
		CHECK_EQUAL(MakeAndCrc(FSoftClassPath{}, TArray<FSoftClassPath>{FSoftClassPath{TEXT("/Script/Engine.AActor")}, FSoftClassPath{TEXT("/Script/Engine.APawn")}}), 0x53e344b3);
	}
}

/**
 * Edge case discovered when we try to copy attributes on the same data, when those were coming from 2 different hierarchies.
 * Mimic this graph:
 * - 2 Create Points, into an Add Attribute, same name different values for both
 * - Merge Points
 * - Copy Points on another Create Points
 * - Copy Attribute, original attribute into another one
 */
TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Metadata::EdgeCase::CopyAttributeOnMergedPoints", "[PCG][Metadata]")
{
	UPCGBasePointData* FirstPointData = FPCGContext::NewPointData_AnyThread(GetContext());
	UPCGBasePointData* SecondPointData = FPCGContext::NewPointData_AnyThread(GetContext());
	
	FirstPointData->SetNumPoints(1);
	SecondPointData->SetNumPoints(1);
	
	const FName AttributeName = "Attr";
	const FName CopyAttributeName = "CopyAttr";
	FirstPointData->MutableMetadata()->CreateAttribute<double>(AttributeName, 0.0, false, false);
	SecondPointData->MutableMetadata()->CreateAttribute<double>(AttributeName, 1.0, false, false);
	
	auto RunElement = [this](FPCGDataCollection& InputData, const UPCGSettings* Settings) -> TUniquePtr<FPCGContext>
	{
		InputData.TaggedData.Emplace(FPCGTaggedData{.Data = Settings, .Pin = "Settings"});
		TUniquePtr<FPCGContext> ElementContext;
		ElementContext.Reset(Settings->GetElement()->Initialize(FPCGInitializeElementParams(&InputData, /*ExecutionSource=*/nullptr, /*Node=*/nullptr)));
		REQUIRE(ElementContext);
		ElementContext->InitializeSettings();
		ElementContext->AsyncState.NumAvailableTasks = 1;
			
		FPCGElementPtr Element = Settings->GetElement();
			
		while (!Element->Execute(ElementContext.Get())){}
			
		return ElementContext;
	};
	
	// Merge
	const UPCGData* MergeResult = nullptr; 
	{
		FPCGDataCollection InputData;
		InputData.TaggedData.Emplace(FPCGTaggedData{.Data = FirstPointData, .Pin = PCGPinConstants::DefaultInputLabel});
		InputData.TaggedData.Emplace(FPCGTaggedData{.Data = SecondPointData, .Pin = PCGPinConstants::DefaultInputLabel});
		UPCGMergeSettings* MergeSettings = FPCGContext::NewObject_AnyThread<UPCGMergeSettings>(GetContext());
		TUniquePtr<FPCGContext> MergeContext = RunElement(InputData, MergeSettings);
		REQUIRE(MergeContext);
		TArray<FPCGTaggedData> OutData = MergeContext->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);
		REQUIRE(OutData.Num() == 1);
		MergeResult = OutData[0].Data;
	}
	
	REQUIRE(MergeResult);
	
	// Copy Points
	const UPCGData* CopyPointsResult = nullptr;
	{
		UPCGBasePointData* TargetPointData = FPCGContext::NewPointData_AnyThread(GetContext());
		TargetPointData->SetNumPoints(1);
		
		FPCGDataCollection InputData;
		InputData.TaggedData.Emplace(FPCGTaggedData{.Data = MergeResult, .Pin = PCGCopyPointsConstants::SourcePointsLabel });
		InputData.TaggedData.Emplace(FPCGTaggedData{.Data = TargetPointData, .Pin = PCGCopyPointsConstants::TargetPointsLabel});
		UPCGCopyPointsSettings* CopyPointsSettings = FPCGContext::NewObject_AnyThread<UPCGCopyPointsSettings>(GetContext());
		TUniquePtr<FPCGContext> CopyPointsContext = RunElement(InputData, CopyPointsSettings);
		REQUIRE(CopyPointsContext);
		TArray<FPCGTaggedData> OutData = CopyPointsContext->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);
		REQUIRE(OutData.Num() == 1);
		CopyPointsResult = OutData[0].Data;
	}
	
	REQUIRE(CopyPointsResult);
	
	// Copy Attributes
	const UPCGData* CopyAttributesResult = nullptr;
	{
		FPCGDataCollection InputData;
		InputData.TaggedData.Emplace(FPCGTaggedData{.Data = CopyPointsResult, .Pin = PCGCopyPointsConstants::SourcePointsLabel });
		InputData.TaggedData.Emplace(FPCGTaggedData{.Data = CopyPointsResult, .Pin = PCGCopyPointsConstants::TargetPointsLabel});
		UPCGCopyAttributesSettings* CopyAttributesSettings = FPCGContext::NewObject_AnyThread<UPCGCopyAttributesSettings>(GetContext());
		CopyAttributesSettings->InputSource.SetAttributeName(AttributeName);
		CopyAttributesSettings->OutputTarget.SetAttributeName(CopyAttributeName);
		TUniquePtr<FPCGContext> CopyAttributesContext = RunElement(InputData, CopyAttributesSettings);
		REQUIRE(CopyAttributesContext);
		TArray<FPCGTaggedData> OutData = CopyAttributesContext->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);
		REQUIRE(OutData.Num() == 1);
		CopyAttributesResult = OutData[0].Data;
	}

	// Verify that the attributes are copied correctly
	const UPCGBasePointData* CopyAttributesResultPoints = Cast<UPCGBasePointData>(CopyAttributesResult);
	REQUIRE(CopyAttributesResultPoints);
	REQUIRE(CopyAttributesResultPoints->GetNumPoints() == 2);
	
	const FPCGMetadataAttribute<double>* CopyAttribute = CopyAttributesResultPoints->ConstMetadata()->GetConstDefaultMetadataDomain()->GetConstTypedAttribute<double>(CopyAttributeName);
	REQUIRE(CopyAttribute);
	
	TConstPCGValueRange<PCGMetadataEntryKey> EntryKeys = CopyAttributesResultPoints->GetConstMetadataEntryValueRange();
	REQUIRE(EntryKeys.Num() == 2);
	
	
	REQUIRE_EQUAL(CopyAttribute->GetValueFromItemKey(EntryKeys[0]), 0.0);
	REQUIRE_EQUAL(CopyAttribute->GetValueFromItemKey(EntryKeys[1]), 1.0);
}