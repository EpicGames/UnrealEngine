// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Async.h"
#include "HAL/Event.h"
#include "Templates/Function.h"

namespace PCGTests::Async
{
	template <typename Func = TFunction<void(int32 /*ThreadIndex*/)>>
	void AsyncRun(int32 NumThreads, Func&& Callback, bool& bOutHasRun, bool& bHasRunOnMultiThread)
	{
		FEvent* StartEvent = FPlatformProcess::GetSynchEventFromPool(/*bIsManualReset=*/true);
		FEvent* LastEvent = FPlatformProcess::GetSynchEventFromPool(/*bIsManualReset=*/true);
		TAtomic<uint64> ActiveCount{ 0 };
		TAtomic<bool> bSuccess = true;
		TAtomic<uint64> NumActualThreads = 0;

		for (int32 i = 0; i < NumThreads; ++i)
		{
			auto Process = [StartEvent, i, &bSuccess, &ActiveCount, LastEvent, &NumActualThreads, &Callback]()
				{
					if (StartEvent->Wait(2000))
					{
						Callback(i);
					}
					else
					{
						bSuccess = false;
					}
				
					// Verify that it was actually done in MT. 
					if (!IsInGameThread())
					{
						++NumActualThreads;
					}
				
					if (--ActiveCount == 0)
					{
						LastEvent->Trigger();
					}
				};
		
			++ActiveCount;
			::Async(i < FTaskGraphInterface::Get().GetNumWorkerThreads() ? EAsyncExecution::TaskGraph : EAsyncExecution::Thread, MoveTemp(Process));
		}
	
		StartEvent->Trigger();
		LastEvent->Wait();
	
		FPlatformProcess::ReturnSynchEventToPool(StartEvent);
		FPlatformProcess::ReturnSynchEventToPool(LastEvent);
		
		bOutHasRun = bSuccess;
		bHasRunOnMultiThread = NumActualThreads > 1;
	}
}
