// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "TestHarness.h"

#include <catch2/generators/catch_generators.hpp>

#include "Helpers/Parsing/PCGParsing.h"
#include "PCGTestsCommon.h"

TEST_CASE_METHOD(PCGTests::FPCGBaseTestWithBasicContext, "PCG::Indexing::Basic", "[PCG][Indexing]")
{
	using PCGIndexing::FPCGIndexCollection;

	SECTION("Empty array")
	{
		FPCGIndexCollection Collection(10);
		TArray<int32> EmptyIndices;
		Collection += EmptyIndices;
		REQUIRE_EQUAL(Collection.GetTotalIndexCount(), 0);
	}

	SECTION("Empty array construction")
	{
		TArray<int32> EmptyIndices;
		FPCGIndexCollection Collection(EmptyIndices, 10);
		REQUIRE(Collection.IsValid());
		REQUIRE(Collection.IsEmpty());
		REQUIRE_EQUAL(Collection.GetArraySize(), 10);
	}

	SECTION("Array construction")
	{
		TArray<int32> Indices = {0, 1, 2, 3, 4};
		FPCGIndexCollection Collection(Indices, 10);
		REQUIRE_EQUAL(Collection.GetTotalIndexCount(), 5);
		REQUIRE_EQUAL(Collection.GetTotalRangeCount(), 1);
		REQUIRE_EQUAL(Collection.GetArraySize(), 10);
	}

	SECTION("Single index addition")
	{
		FPCGIndexCollection Collection(10);
		Collection += {5};
		REQUIRE_EQUAL(Collection.GetTotalIndexCount(), 1);
		REQUIRE(Collection.ContainsIndex(5));
		REQUIRE_FALSE(Collection.ContainsIndex(6));
	}

	SECTION("Single index via array")
	{
		FPCGIndexCollection Collection(10);
		TArray<int32> Indices = {5};
		Collection += Indices;
		REQUIRE_EQUAL(Collection.GetTotalIndexCount(), 1);
		REQUIRE(Collection.ContainsIndex(5));
	}

	SECTION("Consecutive indices merge")
	{
		FPCGIndexCollection Collection(10);
		Collection += {3, 4, 5};
		REQUIRE_EQUAL(Collection.GetTotalIndexCount(), 3);
		REQUIRE_EQUAL(Collection.GetTotalRangeCount(), 1);
	}

	SECTION("Multiple, non-consecutive")
	{
		FPCGIndexCollection Collection(10);
		Collection += {2, 4, 6};
		REQUIRE_EQUAL(Collection.GetTotalIndexCount(), 3);
		REQUIRE_EQUAL(Collection.GetTotalRangeCount(), 3);
		REQUIRE((Collection.ContainsIndex(2) && Collection.ContainsIndex(4) && Collection.ContainsIndex(6)));
		REQUIRE((!Collection.ContainsIndex(1) && !Collection.ContainsIndex(5) && !Collection.ContainsIndex(7)));
	}

	SECTION("Sorted, consecutive indices")
	{
		FPCGIndexCollection Collection(20);
		TArray<int32> Indices = {0, 1, 2, 3, 4, 5, 6, 7, 8, 9};
		Collection += Indices;
		REQUIRE_EQUAL(Collection.GetTotalIndexCount(), 10);
		REQUIRE_EQUAL(Collection.GetTotalRangeCount(), 1);
	}

	SECTION("Sorted, non-consecutive indices")
	{
		FPCGIndexCollection Collection(20);
		TArray<int32> Indices = {0, 1, 2, 5, 6, 7, 10, 15};
		Collection += Indices;
		REQUIRE_EQUAL(Collection.GetTotalIndexCount(), 8);
		REQUIRE_EQUAL(Collection.GetTotalRangeCount(), 4);
		REQUIRE((Collection.ContainsIndex(0) && Collection.ContainsIndex(7) &&
			Collection.ContainsIndex(10) && Collection.ContainsIndex(15)));
		REQUIRE_FALSE((Collection.ContainsIndex(3) || Collection.ContainsIndex(8)));
	}

	SECTION("Unsorted array")
	{
		FPCGIndexCollection Collection(20);
		TArray<int32> Indices = {9, 2, 5, 1, 7, 3};
		Collection += Indices;
		REQUIRE_EQUAL(Collection.GetTotalIndexCount(), 6);
		REQUIRE((Collection.ContainsIndex(1) && Collection.ContainsIndex(9)));
	}

	SECTION("Duplicates")
	{
		FPCGIndexCollection Collection(20);
		TArray<int32> Indices = {5, 5, 5, 7, 7, 9};
		Collection += Indices;
		REQUIRE_EQUAL(Collection.GetTotalIndexCount(), 3);
		REQUIRE((Collection.ContainsIndex(5) && Collection.ContainsIndex(7) && Collection.ContainsIndex(9)));
		REQUIRE((!Collection.ContainsIndex(2) && !Collection.ContainsIndex(4) && !Collection.ContainsIndex(6)));
	}

	SECTION("Negative index")
	{
		FPCGIndexCollection Collection(10);
		Collection += {-1};
		REQUIRE(Collection.ContainsIndex(9));
		REQUIRE_EQUAL(Collection.GetTotalIndexCount(), 1);
	}

	SECTION("Multiple negative indices")
	{
		FPCGIndexCollection Collection(10);
		Collection += {-1, -2};
		REQUIRE_EQUAL(Collection.GetTotalIndexCount(), 2);
		REQUIRE_EQUAL(Collection.GetTotalRangeCount(), 1);
		REQUIRE((Collection.ContainsIndex(8) && Collection.ContainsIndex(9)));
		REQUIRE_FALSE(Collection.ContainsIndex(7));
	}

	SECTION("All negative indices spanning full array")
	{
		FPCGIndexCollection Collection(5);
		Collection += {-1, -2, -3, -4, -5};
		REQUIRE_EQUAL(Collection.GetTotalIndexCount(), 5);
		REQUIRE_EQUAL(Collection.GetTotalRangeCount(), 1);
		REQUIRE((Collection.ContainsIndex(0) && Collection.ContainsIndex(4)));
	}

	SECTION("Negative adjacent to positive after resolution")
	{
		FPCGIndexCollection Collection(10);
		Collection += {8, -1};
		REQUIRE_EQUAL(Collection.GetTotalIndexCount(), 2);
		REQUIRE_EQUAL(Collection.GetTotalRangeCount(), 1);
		REQUIRE((Collection.ContainsIndex(8) && Collection.ContainsIndex(9)));
	}

	SECTION("Mixed positive and negative indices")
	{
		FPCGIndexCollection Collection(10);
		Collection += {0, 1, -1};
		REQUIRE_EQUAL(Collection.GetTotalIndexCount(), 3);
		REQUIRE_EQUAL(Collection.GetTotalRangeCount(), 2);
		REQUIRE((Collection.ContainsIndex(0) && Collection.ContainsIndex(1) && Collection.ContainsIndex(9)));
		REQUIRE((!Collection.ContainsIndex(2) && !Collection.ContainsIndex(8)));
	}

	SECTION("Negative index exceeding array size")
	{
		FPCGIndexCollection Collection(10);
		Collection += {-11};
		REQUIRE_EQUAL(Collection.GetTotalIndexCount(), 0);
	}

	SECTION("Array construction with negative indices")
	{
		TArray<int32> Indices = {3, -1};
		FPCGIndexCollection Collection(Indices, 10);
		REQUIRE_EQUAL(Collection.GetTotalIndexCount(), 2);
		REQUIRE_EQUAL(Collection.GetTotalRangeCount(), 2);
		REQUIRE((Collection.ContainsIndex(3) && Collection.ContainsIndex(9)));
		REQUIRE_FALSE(Collection.ContainsIndex(4));
	}

	SECTION("Grow size with array with positive index first")
	{
		FPCGIndexCollection Collection(10);
		Collection += {10, -11};
		// Grow sets ArraySize = max(10 + 1, 10) = 11. Index 10 and 0 are now valid.
		REQUIRE_EQUAL(Collection.GetArraySize(), 11);
		REQUIRE_EQUAL(Collection.GetTotalIndexCount(), 2);
		REQUIRE((Collection.ContainsIndex(0) && Collection.ContainsIndex(10)));
	}

	SECTION("Grow size with array with negative index first")
	{
		FPCGIndexCollection Collection(10);
		Collection += {-11, 10};
		// Order shouldn't matter here, so functionally the same as above
		REQUIRE_EQUAL(Collection.GetArraySize(), 11);
		REQUIRE_EQUAL(Collection.GetTotalIndexCount(), 2);
		REQUIRE((Collection.ContainsIndex(0) && Collection.ContainsIndex(10)));
	}
}
