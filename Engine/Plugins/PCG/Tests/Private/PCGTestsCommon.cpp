// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGTestsCommon.h"

#include "PCGModule.h"
#include "Logging/StructuredLog.h"

namespace PCGTests
{
	FPCGBaseTest::FPCGBaseTest()
	{
		PCGModule = MakeUnique<FPCGModule>();
		PCGModule->StartupModule();
	}

	FPCGBaseTest::~FPCGBaseTest()
	{
		PCGModule->ShutdownModule();
	}

	UPCGBasePointData* FPCGBaseTestWithContext::CreatePointData()
	{
		return FPCGContext::NewPointData_AnyThread(GetContext());
	}

	FPCGTestsLogOutputDevice::FPCGTestsLogOutputDevice(bool bInSuppressErrors):
		bSuppressErrors(bInSuppressErrors),
		OldContext(GWarn)
	{
		GWarn = this;
		GLog->AddOutputDevice(this);
	}

	FPCGTestsLogOutputDevice::~FPCGTestsLogOutputDevice()
	{
		GLog->RemoveOutputDevice(this);
		GWarn = OldContext;
	}

	void FPCGTestsLogOutputDevice::Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category)
	{
		if (Verbosity <= ELogVerbosity::Warning)
		{
			if (NbMessageReceived.Contains(Verbosity))
			{
				NbMessageReceived[Verbosity]++;
			}
			else
			{
				NbMessageReceived.Add(Verbosity, 1);
			}
			if (!bSuppressErrors)
			{
				OldContext->Serialize(V, Verbosity, Category);
			}
		}
		else
		{
			OldContext->Serialize(V, Verbosity, Category);
		}
	}

	void FPCGTestsLogOutputDevice::SerializeRecord(const UE::FLogRecord& Record)
	{
		// Making sure we catch any re-entry
		thread_local bool bReentryGuard = false;

		if (bReentryGuard)
		{
			return;
		}

		bReentryGuard = true;

		const ELogVerbosity::Type Verbosity = ResolveVerbosity(Record.GetVerbosity());
		if (Verbosity <= ELogVerbosity::Warning)
		{
			if (NbMessageReceived.Contains(Verbosity))
			{
				NbMessageReceived[Verbosity]++;
			}
			else
			{
				NbMessageReceived.Add(Verbosity, 1);
			}
			if (!bSuppressErrors)
			{
				OldContext->SerializeRecord(Record);
			}
		}
		else
		{
			OldContext->SerializeRecord(Record);
		}

		bReentryGuard = false;
	}
}
