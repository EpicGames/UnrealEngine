// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"

#include "CoreMinimal.h"
#include "TestHarness.h"

#include <catch2/catch_approx.hpp>
#include <catch2/matchers/catch_matchers_all.hpp>

#include "Misc/FeedbackContext.h"

class FPCGModule;
class UPCGBasePointData;

namespace PCGTests
{
	template<typename VecType>
	struct EqualsVecMatcher : Catch::Matchers::MatcherGenericBase {
		EqualsVecMatcher(VecType const& Vec):
			Vec{ Vec }
		{}

		template<typename OtherVecType>
		bool match(OtherVecType const& Other) const {
			return Vec.Equals(Other);
		}

		virtual std::string describe() const override {
			return std::string("Equals: ") + std::string(TCHAR_TO_UTF8(*(Vec.ToString())));
		}

	private:
		const VecType& Vec;
	};

	template <typename T> 
	concept CHasEqual = requires(const T& V1, const T& V2) {{V1.Equals(V2)} -> std::same_as<bool>;};
	template <typename T> 
	concept CHasToString = requires(const T& V) { { V.ToString() } -> std::same_as<FString>; };

	template<typename DataType>
	auto ApproxMatcher(const DataType& Val)
	{
		return Catch::Matchers::WithinRel(Val);
	}

	template<CHasEqual DataType>
	auto ApproxMatcher(const DataType& Val)
	{
		return EqualsVecMatcher(Val);
	}
}

namespace Catch {
	template<PCGTests::CHasToString T>
	struct StringMaker<T> {
		static std::string convert( T const& V ) {
			return std::string(TCHAR_TO_UTF8(*V.ToString()));
		}
	};
}


namespace PCGTests
{
class FPCGBaseTest
{
public:
	FPCGBaseTest();
	virtual ~FPCGBaseTest();
	
protected:
	TUniquePtr<FPCGModule> PCGModule;
};

/** Simple fixture that provide helper functions to create new objects with a FPCGContext. */
class FPCGBaseTestWithContext : public FPCGBaseTest
{
public:
	using FPCGBaseTest::FPCGBaseTest;
	
protected:
	virtual FPCGContext* GetContext() = 0;;
	UPCGBasePointData* CreatePointData();

	template <typename T> requires std::is_base_of_v<UPCGData, T>
	T* CreateData()
	{
		return FPCGContext::NewObject_AnyThread<T>(GetContext());
	}

	template <typename T, typename ...Args> requires std::is_base_of_v<UObject, T>
	T* CreateObject(Args&& ...InArgs)
	{
		return FPCGContext::NewObject_AnyThread<T>(GetContext(), std::forward<Args>(InArgs)...);
	}
	
};
	
/** Simple fixture that provide a non-dynamic FPCGContext. */
class FPCGBaseTestWithBasicContext : public FPCGBaseTestWithContext
{
public:
	using FPCGBaseTestWithContext::FPCGBaseTestWithContext;

protected:
	virtual FPCGContext* GetContext() override {return &Context;};

	FPCGContext Context;
};
	
/** Utility class to gather errors and warnings*/
struct FPCGTestsLogOutputDevice : public FFeedbackContext
{
	explicit FPCGTestsLogOutputDevice(bool bInSuppressErrors);

	virtual ~FPCGTestsLogOutputDevice();

	// FOutputDevice
	virtual bool IsMemoryOnly() const override { return true; }
	virtual bool CanBeUsedOnMultipleThreads() const override { return true; }
	virtual bool CanBeUsedOnAnyThread() const override { return true; }
	virtual void SerializeRecord(const UE::FLogRecord& Record) override;
	
	//~ @todo_pcg: When this becomes deprecated, can be safely removed, as SerializeRecord is the new way of capturing logs.
	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override;

	bool bSuppressErrors;
	FFeedbackContext* OldContext = nullptr;
	using MessageMap = TMap<ELogVerbosity::Type, int32>;
	MessageMap NbMessageReceived;
};
	
/** Base templated class to execute a PCG element.*/
template<typename PCGSettingsType>
class FPCGSingleElementBaseTest: public PCGTests::FPCGBaseTestWithContext
{
public:
	FPCGSingleElementBaseTest(): 
		FPCGBaseTestWithContext()
	{
		TypedSettings = NewObject<PCGSettingsType>();
		check(TypedSettings);

		TypedSettings->Seed = 42;

		InputData.TaggedData.Emplace_GetRef().Data = TypedSettings;
		InputData.TaggedData.Last().Pin = FName(TEXT("Settings"));
		
		TestElement = TypedSettings->GetElement();
	}

	virtual FPCGContext* GetContext() override {return Context.Get();};
	
	class FSuppressErrorsScope
	{
		public:
		explicit FSuppressErrorsScope(FPCGSingleElementBaseTest& InTestReference): TestReference(InTestReference)
		{
			TestReference.bSuppressErrors = true;
		}
		~FSuppressErrorsScope()
		{
			TestReference.bSuppressErrors = false;
		}
		
	private:
		FPCGSingleElementBaseTest& TestReference;
	};

	void ExecuteElement(IPCGGraphExecutionSource* ExecutionSource = nullptr, const UPCGNode* Node = nullptr)
	{
		Context.Reset(TypedSettings->GetElement()->Initialize(FPCGInitializeElementParams(&InputData, ExecutionSource, Node)));
		check(Context);
		Context->InitializeSettings();
		Context->AsyncState.NumAvailableTasks = 1;
		
		PCGTests::FPCGTestsLogOutputDevice LogCapture(bSuppressErrors);

		while (!TestElement->Execute(Context.Get()))
		{
		}	
	
		NumErrors = LogCapture.NbMessageReceived.Contains(ELogVerbosity::Error) ? LogCapture.NbMessageReceived[ELogVerbosity::Error] : 0;
		NumWarnings = LogCapture.NbMessageReceived.Contains(ELogVerbosity::Warning) ? LogCapture.NbMessageReceived[ELogVerbosity::Warning] : 0;
	}

	PCGSettingsType* TypedSettings;
	FPCGElementPtr TestElement;
	FPCGDataCollection InputData;
	TUniquePtr<FPCGContext> Context;
	bool bSuppressErrors = false;
	int NumErrors = 0;
	int NumWarnings = 0;
};	
}
