// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "TestHarness.h"

#include "PCGContext.h"
#include "Data/PCGPointData.h"
#include "SpatialAlgo/PCGOctreeQueries.h"

/**
* For all the tests, the point data will be points scattered on a straight line (Direction {1, 1, 1}), and evenly spaced.
* Query will be on the first point, at the origin.
*/
TEST_CASE("PCG::OctreeQueries", "[PCG][OctreeQueries]")
{
	// Setup code for this test case
	static constexpr double Distance = 100.0;
	static constexpr int32 NumPoints = 10;
	FPCGContext Context{};

	UPCGBasePointData* InputPointData = FPCGContext::NewPointData_AnyThread(&Context);
	InputPointData->SetNumPoints(NumPoints);
	InputPointData->SetDensity(1.0f);
	InputPointData->SetSeed(42);

	TPCGValueRange<FTransform> TransformRange = InputPointData->GetTransformValueRange();

	for (int32 i = 0; i < NumPoints; ++i)
	{
		FVector Location = FVector::OneVector * Distance * i;
		TransformRange[i] = FTransform(Location);
	}

	SECTION("Sphere")
	{
		TArray<int32> ExpectedIndexes = { 0, 1, 2 };
		int32 CountFound = 0;
		UPCGOctreeQueries::ForEachPointInsideSphere(InputPointData, FVector::ZeroVector, 350.0, [&ExpectedIndexes, &CountFound](const UPCGBasePointData*, int32 PointIndex, double)
		{
			if (ExpectedIndexes.Contains(PointIndex))
			{
				++CountFound;
			}
		});

		REQUIRE_EQUAL(CountFound, ExpectedIndexes.Num());
	}

	SECTION("ClosestPoint")
	{
		const int32 PointIndex = UPCGOctreeQueries::GetClosestPointIndex(InputPointData, FVector::ZeroVector, /*bDiscardCenter=*/false, 350.0);
		REQUIRE_NOT_EQUAL(PointIndex, (int32)INDEX_NONE);
		REQUIRE_EQUAL(PointIndex, 0);
	}

	SECTION("ClosestPointFromOtherPoint")
	{
		const int32 PointIndex = UPCGOctreeQueries::GetClosestPointIndexFromOtherPointIndex(InputPointData, 0, 350.0);
		REQUIRE_NOT_EQUAL(PointIndex, (int32)INDEX_NONE);
		REQUIRE_EQUAL(PointIndex, 1);
	}

	SECTION("FarthestPoint")
	{
		const int32 PointIndex = UPCGOctreeQueries::GetFarthestPointIndex(InputPointData, FVector::ZeroVector, 350.0);
		REQUIRE_NOT_EQUAL(PointIndex, (int32)INDEX_NONE);
		REQUIRE_EQUAL(PointIndex, 2);
	}

	SECTION("FarthestPointFromOtherPoint")
	{
		const int32 PointIndex = UPCGOctreeQueries::GetFarthestPointIndexFromOtherPointIndex(InputPointData, 0, 10000.0);
		REQUIRE_NOT_EQUAL(PointIndex, (int32)INDEX_NONE);
		REQUIRE_EQUAL(PointIndex, 9);
	}
}
